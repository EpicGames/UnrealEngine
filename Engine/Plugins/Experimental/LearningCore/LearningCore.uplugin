{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "0.1",
	"FriendlyName": "Learning Core",
	"Description": "Learning Core is a machine learning library for Unreal Engine.",
	"Category": "AI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Learning",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LearningTraining",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeBasicCpu",
			"Enabled": true
		}
	],
	"PythonRequirements": [
		{
			"Platform": "All",
			"Requirements": [
				"boto3==1.40.11 --hash=sha256:9d2d211d9cb3efc9a2b2ceec3c510b4e62e389618fd5c871e74d2cbca4561ff5",
				"botocore==1.40.11 --hash=sha256:4beca0c5f92201da1bf1bc0a55038538ad2defded32ab0638cb68f5631dcc665",
				"jmespath==1.0.1 --hash=sha256:02e2e4cc71b5bcab88332eebf907519190dd9e6e82107fa7f83b1003a6252980",
				"python_dateutil==2.9.0 --hash=sha256:cbf2f1da5e6083ac2fbfd4da39a25f34312230110440f424a14c7558bb85d82e",
				"s3transfer==0.13.1 --hash=sha256:a981aa7429be23fe6dfc13e80e4020057cbab622b08c0315288758d67cabc724",
				"six==1.16.0 --hash=sha256:8abb2f1d86890a2dfb989f9a77cfcfd3e47c2a354b01111771326f8aa26e0254",
				"smart_open==7.3.0 --hash=sha256:0fc37c7144868e2619a088537094aa3cb0fe5fd8c17162149fa6c983fa8efe6a",
				"wrapt==1.17.2 --hash=sha256:0a6e821770cf99cc586d33833b2ff32faebdbe886bd6322395606cf55153246c --hash=sha256:0b929ac182f5ace000d459c59c2c9c33047e20e935f8e39371fa6e3b85d56f4a --hash=sha256:1a7ed2d9d039bd41e889f6fb9364554052ca21ce823580f6a07c4ec245c1f5d6 --hash=sha256:2696993ee1eebd20b8e4ee4356483c4cb696066ddc24bd70bcbb80fa56ff9061 --hash=sha256:36ccae62f64235cf8ddb682073a60519426fdd4725524ae38874adf72b5f2aeb --hash=sha256:3d57c572081fed831ad2d26fd430d565b76aa277ed1d30ff4d40670b1c0dd984 --hash=sha256:3fc7cb4c1c744f8c05cd5f9438a3caa6ab94ce8344e952d7c45a8ed59dd88392 --hash=sha256:4db983e7bca53819efdbd64590ee96c9213894272c776966ca6306b73e4affda --hash=sha256:58705da316756681ad3c9c73fd15499aa4d8c69f9fd38dc8a35e06c12468582f --hash=sha256:5bb1d0dbf99411f3d871deb6faa9aabb9d4e744d67dcaaa05399af89d847a91d --hash=sha256:612dff5db80beef9e649c6d803a8d50c409082f1fedc9dbcdfde2983b2025b82 --hash=sha256:62c2caa1585c82b3f7a7ab56afef7b3602021d6da34fbc1cf234ff139fed3cd9 --hash=sha256:80dd7db6a7cb57ffbc279c4394246414ec99537ae81ffd702443335a61dbf3a7 --hash=sha256:8fdbdb757d5390f7c675e558fd3186d590973244fab0c5fe63d373ade3e99d40 --hash=sha256:99039fa9e6306880572915728d7f6c24a86ec57b0a83f6b2491e1d8ab0235b9a --hash=sha256:9abc77a4ce4c6f2a3168ff34b1da9b0f311a8f1cfd694ec96b0603dff1c79438 --hash=sha256:acc130bc0375999da18e3d19e5a86403667ac0c4042a094fefb7eec8ebac7cf3 --hash=sha256:b18f2d1533a71f069c7f82d524a52599053d4c7166e9dd374ae2136b7f40f7c8 --hash=sha256:b5e251054542ae57ac7f3fba5d10bfff615b6c2fb09abeb37d2f1463f841ae22 --hash=sha256:b870b5df5b71d8c3359d21be8f0d6c485fa0ebdb6477dda51a1ea54a9b558061 --hash=sha256:bc570b5f14a79734437cb7b0500376b6b791153314986074486e0b0fa8d71d98 --hash=sha256:d5e2439eecc762cd85e7bd37161d4714aa03a33c5ba884e26c81559817ca0925 --hash=sha256:ec89ed91f2fa8e3f52ae53cd3cf640d6feff92ba90d62236a81e4e563ac0e991 --hash=sha256:fc78a84e2dfbc27afe4b2bd7c80c8db9bca75cc5b85df52bfe634596a1da846b --hash=sha256:ff04ef6eec3eee8a5efef2401495967a916feaa353643defcc03fc74fe213b58"
			]
		}
	]
}