<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="TMultiArray&lt;*,*,TFixedAllocator&lt;*&gt;&gt;">
    <Expand>
      <ArrayItems>
        <Direction>Forward</Direction>
        <Rank>$T1</Rank>
        <Size>ArrayShape.Nums[$i]</Size>
        <ValuePointer>(TArray&lt;$T1,TFixedAllocator&lt;$T2&gt; &gt;::ElementType*)AllocatorInstance.InlineData</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="TMultiArray&lt;*,*,TSizedInlineAllocator&lt;*,*,*&gt;&gt;">
    <Expand>
      <ArrayItems>
        <Direction>Forward</Direction>
        <Rank>$T1</Rank>
        <Size>ArrayShape.Nums[$i]</Size>
        <ValuePointer Condition="AllocatorInstance.SecondaryData.Data == 0">($T2*)AllocatorInstance.InlineData</ValuePointer>
        <ValuePointer Condition="AllocatorInstance.SecondaryData.Data != 0">($T2*)AllocatorInstance.SecondaryData.Data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="TMultiArray&lt;*,*,TMemoryImageAllocator&lt;*&gt;&gt;">
    <Expand>
      <ArrayItems>
        <Direction>Forward</Direction>
        <Rank>$T1</Rank>
        <Size>ArrayShape.Nums[$i]</Size>
        <ValuePointer Condition="(AllocatorInstance.Data.Frozen.Packed &amp; 1) != 0">($T2*)( (char*)&amp;AllocatorInstance.Data + (int64(AllocatorInstance.Data.Frozen.Packed) >> 24))</ValuePointer>
        <ValuePointer Condition="(AllocatorInstance.Data.Frozen.Packed &amp; 1) == 0">($T2*)AllocatorInstance.Data.UnfrozenPtr</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="TMultiArray&lt;*,*,*&gt;">
    <Expand>
      <ArrayItems>
        <Direction>Forward</Direction>
        <Rank>$T1</Rank>
        <Size>ArrayShape.Nums[$i]</Size>
        <ValuePointer>($T2*)AllocatorInstance.Data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="TMultiArrayView&lt;*,*&gt;">
    <Expand>
      <ArrayItems>
        <Direction>Forward</Direction>
        <Rank>$T1</Rank>
        <Size>ArrayShape.Nums[$i]</Size>
        <ValuePointer>($T2*)DataPtr</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

</AutoVisualizer>
