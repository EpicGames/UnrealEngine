// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_MAX_FLT
#define UE_MAX_FLT			(3.402823466e+38F)
#endif

#ifndef INDEX_NONE
#define INDEX_NONE		(-1)
#endif

#ifndef UE_SMALL_NUMBER
#define UE_SMALL_NUMBER (1.e-8f)
#endif

#ifndef UE_KINDA_SMALL_NUMBER
#define UE_KINDA_SMALL_NUMBER (1.e-4f)
#endif

#ifndef UE_PI
#define UE_PI 					(3.1415926535897932f)	/* Extra digits if needed: 3.1415926535897932384626433832795f */
#endif

inline float LearningSigmoid(const float X)
{
	return 1.0f / (1.0f + exp(-X));
}

inline float LearningSquare(float X)
{
	return X * X;
}

inline float LearningInvExpApprox(float X)
{
	uniform const float A = 1.00746054f; // 1 / 1! in Taylor series
	uniform const float B = 0.45053901f; // 1 / 2! in Taylor series
	uniform const float C = 0.25724632f; // 1 / 3! in Taylor series
	return 1 / (1 + A * X + B * X * X + C * X * X * X);
}

inline float LearningApproxAcos(float X)
{
	const float Y = abs(X);
	const float P = -0.1565827 * Y + 1.570796;
	const float Q = P * sqrt(max(1.0 - Y, 0.0));
	return X >= 0.0 ? Q : UE_PI - Q;
}

inline float LearningInertializationPartialIntegral(
	const float X,
	const float V,
	const float Time,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = clamp(Time / Z, 0.0f, 1.0f);
	return
		X * ((Z / 2) * T * T * T * T - Z * T * T * T + Z * T) +
		V * (((Z * Z) / 4) * T * T * T * T - ((2 * Z * Z) / 3) * T * T * T + ((Z * Z) / 2) * T * T);
}

inline float LearningInertializationFullIntegral(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	return X * (Z / 2) + V * ((Z * Z) / 12);
}

inline float LearningInertializationPosition(
	const float X,
	const float V,
	const float Time,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = clamp(Time / Z, 0.0f, 1.0f);
	return X * (2 * T * T * T - 3 * T * T + 1) + V * (Z * T * T * T - Z * 2 * T * T + Z * T);
}

inline float LearningInertializationVelocity(
	const float X,
	const float V,
	const float Time,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = clamp(Time / Z, 0.0f, 1.0f);
	return X * ((6 / Z) * T * T - (6 / Z) * T) + V * (3 * T * T - 4 * T + 1);
}

inline float LearningInertializationPositionIntersectionTime(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	return (-Z * X) / (2 * X + Z * V);
}

inline float LearningInertializationVelocityIntersectionTime(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	return (Z * Z * V) / (6 * X + 3 * Z * V);
}

inline float LearningInertializationAccelerationIntersectionTime(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	return (3 * Z * X + 2 * Z * Z * V) / (6 * X + 3 * Z * V);
}

inline float LearningInertializationMaximumPositionMagnitude(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = LearningInertializationVelocityIntersectionTime(X, V, Z);

	return T > 0.0f && T < Z ?
		max(abs(X), abs(LearningInertializationPosition(X, V, T, Z))) : abs(X);
}

inline float LearningInertializationMaximumVelocityMagnitude(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = LearningInertializationAccelerationIntersectionTime(X, V, Z);

	return T > 0.0f && T < Z ?
		max(abs(V), abs(LearningInertializationVelocity(X, V, T, Z))) : abs(V);
}

inline float LearningInertializationIntegralMagnitude(
	const float X,
	const float V,
	uniform const float BlendTime)
{
	uniform const float Z = max(BlendTime, UE_SMALL_NUMBER);
	const float T = LearningInertializationPositionIntersectionTime(X, V, Z);
	const float IntegralT = LearningInertializationPartialIntegral(X, V, T, Z);
	const float IntegralZ = LearningInertializationFullIntegral(X, V, Z);

	return T > 0.0f && T < Z ?
		abs(IntegralZ - IntegralT) + abs(IntegralT) :
		abs(IntegralZ);
}

/////////////////////////////////////////////////////////
// 
// Functions matching those in LearningRandom.cpp
// 
/////////////////////////////////////////////////////////

inline float LearningRandomUniformToGaussian(const float R1, const float R2)
{
	return sqrt(-2.0f * log(max(R1, 1e-8f))) * cos(R2 * 6.28318530717f);
}

inline uint32 LearningRandomInt(const uint32 State)
{
	uint32 X = State ^ 0xb74eaecf;
	X = ((X >> 16) ^ X) * 0x45d9f3b;
	X = ((X >> 16) ^ X) * 0x45d9f3b;
	return (X >> 16) ^ X;
}

inline uniform uint32 LearningRandomInt(uniform const uint32 State)
{
	uniform uint32 X = State ^ 0xb74eaecf;
	X = ((X >> 16) ^ X) * 0x45d9f3b;
	X = ((X >> 16) ^ X) * 0x45d9f3b;
	return (X >> 16) ^ X;
}

inline float LearningRandomFloat(const uint32 State)
{
	return floatbits(0x3F800000U | (LearningRandomInt(State ^ 0x1c89a74a) >> 9)) - 1.0f;
}

inline uniform float LearningRandomFloat(uniform const uint32 State)
{
	return floatbits(0x3F800000U | (LearningRandomInt(State ^ 0x1c89a74a) >> 9)) - 1.0f;
}

inline int32 LearningRandomIntInRange(const uint32 State, const int32 Min, const int32 Max)
{
	const int32 Range = (Max - Min) + 1;
	return Min + ((Range > 0) ? (int32)(trunc(LearningRandomFloat(State ^ 0x7d3b208a) * (float)(Range))) : 0);
}

inline uniform int32 LearningRandomIntInRange(uniform const uint32 State, uniform const int32 Min, uniform const int32 Max)
{
	uniform const int32 Range = (Max - Min) + 1;
	return Min + ((Range > 0) ? (uniform int32)(LearningRandomFloat(State ^ 0x7d3b208a) * (uniform float)(Range)) : 0);
}

inline float LearningRandomUniform(
	const uint32 State,
	const float Min = 0.0f,
	const float Max = 1.0f)
{
	return (Max - Min) * LearningRandomFloat(State ^ 0x72404541) + Min;
}

inline float LearningRandomGaussian(
	const uint32 State,
	const float Mean = 0.0f,
	const float Std = 1.0f)
{
	return Std * LearningRandomUniformToGaussian(
		LearningRandomFloat(State ^ 0x4855e88f),
		LearningRandomFloat(State ^ 0x0eedb850)) + Mean;
}

inline float LearningRandomClippedGaussian(
	const uint32 State,
	const float Mean = 0.0f,
	const float Std = 1.0f,
	const float Clip = 3.0f)
{
	return clamp(LearningRandomGaussian(State ^ 0xf3ce904a, Mean, Std), Mean - Std * Clip, Mean + Std * Clip);
}

export void LearningRandomIntArray(
	uniform uint32 Output[],
	uniform const int DimNum,
	uniform const uint32 State)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		Output[DimIdx] = LearningRandomInt(State ^ 0xbed25b30 ^ LearningRandomInt(DimIdx ^ 0xb521a8d9));
	}
}

export void LearningRandomFloatArray(
	uniform float Output[],
	uniform const int DimNum,
	uniform const uint32 State)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		Output[DimIdx] = LearningRandomFloat(State ^ 0xf955fac7 ^ LearningRandomInt(DimIdx ^ 0xcd989d6f));
	}
}

export void LearningRandomUniformArray(
	uniform float Output[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Min,
	uniform const float Max)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		Output[DimIdx] = LearningRandomUniform(State ^ 0x5f15554c ^ LearningRandomInt(DimIdx ^ 0x242735e0), Min, Max);
	}
}

export void LearningRandomGaussianArray(
	uniform float Output[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Mean,
	uniform const float Std)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		Output[DimIdx] = LearningRandomGaussian(State ^ 0x7b5d5f62 ^ LearningRandomInt(DimIdx ^ 0x546ab965), Mean, Std);
	}
}

export void LearningRandomClippedGaussianArray(
	uniform float Output[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Mean,
	uniform const float Std,
	uniform const float Clip)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		Output[DimIdx] = LearningRandomClippedGaussian(State ^ 0x11a760f7 ^ LearningRandomInt(DimIdx ^ 0x1dfc9b77), Mean, Std, Clip);
	}
}

export void LearningRandomDistributionIndependantNormal(
	uniform float Output[],
	uniform const float Mean[],
	uniform const float LogStd[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Scale)
{
	if (Scale < UE_SMALL_NUMBER)
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = Mean[DimIdx];
		}
	}
	else
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = LearningRandomGaussian(State ^ 0x7db0e4e9 ^ LearningRandomInt(DimIdx ^ 0xf4976a00), Mean[DimIdx], Scale * exp(min(LogStd[DimIdx], 10.0f)));
		}
	}
}

export void LearningRandomDistributionIndependantNormalMasked(
	uniform float Output[],
	uniform const float Mean[],
	uniform const float LogStd[],
	uniform const bool Masked[],
	uniform const float MaskedValues[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Scale)
{
	if (Scale < UE_SMALL_NUMBER)
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(Masked[DimIdx], MaskedValues[DimIdx], Mean[DimIdx]);
		}
	}
	else
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(Masked[DimIdx], MaskedValues[DimIdx], LearningRandomGaussian(State ^ 0x0a006c6b ^ LearningRandomInt(DimIdx ^ 0x6fe1d121), Mean[DimIdx], Scale * exp(min(LogStd[DimIdx], 10.0f))));
		}
	}
}

export void LearningRandomDistributionBernoulli(
	uniform float Output[],
	uniform const float Logits[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Scale)
{
	if (Scale < UE_SMALL_NUMBER)
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(Logits[DimIdx] > 0.0f, 1.0f, 0.0f);
		}
	}
	else
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(LearningSigmoid(Logits[DimIdx] / Scale) > LearningRandomUniform(State ^ 0xf4021a46 ^ LearningRandomInt(DimIdx ^ 0x7a8cc64e)), 1.0f, 0.0f);
		}
	}
}

export void LearningRandomDistributionBernoulliMasked(
	uniform float Output[],
	uniform const float Logits[],
	uniform const bool Masked[],
	uniform const int DimNum,
	uniform const uint32 State,
	uniform const float Scale)
{
	if (Scale < UE_SMALL_NUMBER)
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(!Masked[DimIdx] && Logits[DimIdx] > 0.0f, 1.0f, 0.0f);
		}
	}
	else
	{
		foreach(DimIdx = 0 ... DimNum)
		{
			Output[DimIdx] = select(!Masked[DimIdx] && LearningSigmoid(Logits[DimIdx] / Scale) > LearningRandomUniform(State ^ 0x8c992b7f ^ LearningRandomInt(DimIdx ^ 0x1953d0fc)), 1.0f, 0.0f);
		}
	}
}

/////////////////////////////////////////////////////////
// 
// Functions used by various different Optimizers
// 
/////////////////////////////////////////////////////////

export void LearningAdjustCMAMean(
	uniform float InOutMean[],
	uniform float InOutOldMean[],
	uniform const float Weights[],
	uniform const int LossRanking[],
	uniform const float Samples[],
	uniform const int Mu,
	uniform const int DimNum)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		InOutOldMean[DimIdx] = InOutMean[DimIdx];
		InOutMean[DimIdx] = 0.0f;
	}

	for (uniform int LossIdx = 0; LossIdx < Mu; LossIdx++)
	{
		const float LossWeight = Weights[LossIdx];
		const int ControlIdx = LossRanking[LossIdx];

		foreach (DimIdx = 0 ... DimNum)
		{
			InOutMean[DimIdx] += LossWeight * Samples[ControlIdx * DimNum + DimIdx];
		}
	}
}

export void LearningUpdateCMACovariance(
	uniform float InOutCovariance[],
	uniform const float PathCovariance[],
	uniform const float Scale,
	uniform const float C1,
	uniform const int DimNum)
{
	foreach (DimX = 0 ... DimNum, DimY = 0 ... DimNum)
	{
		InOutCovariance[DimX * DimNum + DimY] = Scale * InOutCovariance[DimX * DimNum + DimY] + C1 * PathCovariance[DimX] * PathCovariance[DimY];
	}
}

export void LearningComputeCMAUpdateDirection(
	uniform float OutUpdateDirection[],
	uniform const float Mean[],
	uniform const float OldMean[],
	uniform const float CovarianceInverseSqrt[],
	uniform const int DimNum)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		OutUpdateDirection[DimIdx] = 0.0f;
	}
	
	foreach (RowIdx = 0 ... DimNum, ColIdx = 0 ... DimNum)
	{
		OutUpdateDirection[ColIdx] += (Mean[RowIdx] - OldMean[RowIdx]) * CovarianceInverseSqrt[RowIdx * DimNum + ColIdx];
	}
}

export void LearningTransformCMASamples(
	uniform float Output[],
	const uniform float Samples[],
	const uniform float Mean[],
	const uniform float Covariance[],
	const uniform int SampleNum,
	const uniform int DimNum,
	const uniform float Sigma)
{
	foreach (SampleIdx = 0 ... SampleNum, ColIdx = 0 ... DimNum)
	{
		Output[SampleIdx * DimNum + ColIdx] = Mean[ColIdx];
	}
	
	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		for (uniform int RowIdx = 0; RowIdx < DimNum; RowIdx++)
		{
			uniform const float Value = Sigma * Samples[SampleIdx * DimNum + RowIdx];

			foreach (ColIdx = 0 ... DimNum)
			{
				Output[SampleIdx * DimNum + ColIdx] += Value * Covariance[RowIdx * DimNum + ColIdx];
			}
		}
	}
}

export void LearningUpdateAdamEstimate(
	uniform int32 InOutIterations[],
	uniform float InOutEstimate[],
	uniform float InOutM0[],
	uniform float InOutM1[],
	uniform float InOutM1HatMax[],
	uniform const float Gradient[],
	uniform const float LearningRate,
	uniform const float Beta1,
	uniform const float Beta2,
	uniform const int DimNum)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		InOutIterations[DimIdx]++;

		InOutM0[DimIdx] = Beta1 * InOutM0[DimIdx] + (1.0f - Beta1) * Gradient[DimIdx];
		InOutM1[DimIdx] = Beta2 * InOutM1[DimIdx] + (1.0f - Beta2) * (Gradient[DimIdx] * Gradient[DimIdx]);

		const float M0Hat = InOutM0[DimIdx] / (1.0f - pow(Beta1, InOutIterations[DimIdx]));
		const float M1Hat = InOutM1[DimIdx] / (1.0f - pow(Beta2, InOutIterations[DimIdx]));

		InOutM1HatMax[DimIdx] = max(InOutM1HatMax[DimIdx], M1Hat);
		InOutEstimate[DimIdx] -= (LearningRate * M0Hat) / (sqrt(InOutM1HatMax[DimIdx]) + 1e-8f);
	}
}

export void LearningEstimateGradient(
	uniform float OutGradient[],
	uniform const float Samples[],
	uniform const float Losses[],
	uniform const int SampleNum,
	uniform const int DimNum)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		OutGradient[DimIdx] = 0.0f;
	}

	for (uniform int SampleIdx = 1; SampleIdx < SampleNum; SampleIdx++)
	{
		float Length = 0.0f;
		foreach (DimIdx = 0 ... DimNum)
		{
			const float Difference = Samples[SampleIdx * DimNum + DimIdx] - Samples[0 * DimNum + DimIdx];
			Length += Difference * Difference;
		}
		uniform const float SampleLength = max(sqrt(reduce_add(Length)), 1e-8f);

		uniform const float LossGradient = (Losses[SampleIdx] - Losses[0]) / SampleLength;

		foreach (DimIdx = 0 ... DimNum)
		{
			const float ControlGradient = (Samples[SampleIdx * DimNum + DimIdx] - Samples[0 * DimNum + DimIdx]) / SampleLength;

			OutGradient[DimIdx] += (DimNum * LossGradient * ControlGradient) / (SampleNum - 1);
		}
	}
}

export void LearningSampleAdamOptimizer(
	uniform float OutSamples[],
	uniform const float GaussianSamples[],
	uniform const float Estimate[],
	uniform const float FiniteDifferenceStd,
	uniform const int SampleNum,
	uniform const int DimNum)
{
	foreach (DimIdx = 0 ... DimNum)
	{
		OutSamples[0 * DimNum + DimIdx] = Estimate[DimIdx];
	}

	foreach (SampleIdx = 1 ... SampleNum, DimIdx = 0 ... DimNum)
	{
		OutSamples[SampleIdx * DimNum + DimIdx] = Estimate[DimIdx] + FiniteDifferenceStd * GaussianSamples[SampleIdx * DimNum + DimIdx];
	}
}

export void LearningUpdatePSOVelocitiesSamples(
	uniform float InOutVelocities[],
	uniform float InOutSamples[],
	uniform const float UniformSamplesLocal[],
	uniform const float UniformSamplesGlobal[],
	uniform const float LocalBestPositions[],
	uniform const float GlobalBestPosition[],
	uniform const int SampleNum,
	uniform const int DimNum,
	uniform const float Momentum,
	uniform const float LocalGain,
	uniform const float GlobalGain)
{
	foreach (SampleIdx = 0 ... SampleNum, DimIdx = 0 ... DimNum)
	{
		InOutVelocities[SampleIdx * DimNum + DimIdx] = (
			Momentum * InOutVelocities[SampleIdx * DimNum + DimIdx] +
			LocalGain * UniformSamplesLocal[SampleIdx * DimNum + DimIdx] * (LocalBestPositions[SampleIdx * DimNum + DimIdx] - InOutSamples[SampleIdx * DimNum + DimIdx]) +
			GlobalGain * UniformSamplesGlobal[SampleIdx * DimNum + DimIdx] * (GlobalBestPosition[DimIdx] - InOutSamples[SampleIdx * DimNum + DimIdx]));

		InOutSamples[SampleIdx * DimNum + DimIdx] += InOutVelocities[SampleIdx * DimNum + DimIdx];
	}
}

export void LearningUpdatePSOSamples(
	uniform float InOutSamples[],
	uniform const float Velocities[],
	uniform const int SampleNum,
	uniform const int DimNum)
{
	foreach (SampleIdx = 0 ... SampleNum, DimIdx = 0 ... DimNum)
	{
		InOutSamples[SampleIdx * DimNum + DimIdx] += Velocities[SampleIdx * DimNum + DimIdx];
	}
}

export void LearningUpdatePSOBest(
	uniform float& InOutBestGlobalLoss,
	uniform float InOutGlobalBestPosition[],
	uniform float InOutLocalBestLoss[],
	uniform float InOutLocalBestPositions[],
	uniform const float Losses[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int DimNum)
{
	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		if (Losses[SampleIdx] < InOutLocalBestLoss[SampleIdx])
		{
			InOutLocalBestLoss[SampleIdx] = Losses[SampleIdx];
			
			foreach (DimIdx = 0 ... DimNum)
			{
				InOutLocalBestPositions[SampleIdx * DimNum + DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
			}

			if (Losses[SampleIdx] < InOutBestGlobalLoss)
			{
				InOutBestGlobalLoss = Losses[SampleIdx];

				foreach (DimIdx = 0 ... DimNum)
				{
					InOutGlobalBestPosition[DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
				}
			}
		}
	}
}

/////////////////////////////////////////////////////////
// 
// Functions related to K-Means Clustering
// 
/////////////////////////////////////////////////////////

export void LearningKMeansInitCenters(
	uniform float OutCenters[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum,
	uniform const uint32 Seed)
{
	for (uniform int ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
	{
		uniform const int32 SampleIdx = LearningRandomIntInRange(LearningRandomInt(Seed ^ ClusterIdx), 0, SampleNum - 1);

		foreach (DimIdx = 0 ... DimNum)
		{
			OutCenters[ClusterIdx * DimNum + DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
		}
	}
}

export void LearningKMeansInitBounds(
	uniform float OutMins[],
	uniform float OutMaxs[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum,
	uniform const uint32 Seed)
{
	for (uniform int ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
	{
		uniform const int32 SampleIdx = LearningRandomIntInRange(LearningRandomInt(Seed ^ ClusterIdx), 0, SampleNum - 1);

		foreach (DimIdx = 0 ... DimNum)
		{
			OutMins[ClusterIdx * DimNum + DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
			OutMaxs[ClusterIdx * DimNum + DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
		}
	}
}

export void LearningKMeansUpdateAssignmentsFromCenters(
	uniform int32 OutAssignments[],
	uniform const float Centers[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum)
{
	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		uniform float BestDistance = UE_MAX_FLT;
		uniform int32 BestIndex = INDEX_NONE;

		for (uniform int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			float Distance = 0.0f;
			foreach (DimIdx = 0 ... DimNum)
			{
				const float Difference = Centers[ClusterIdx * DimNum + DimIdx] - Samples[SampleIdx * DimNum + DimIdx];
				Distance += Difference * Difference;
			}
			uniform const float CurrDistance = reduce_add(Distance);

			if (CurrDistance < BestDistance)
			{
				BestDistance = CurrDistance;
				BestIndex = ClusterIdx;
			}
		}

		OutAssignments[SampleIdx] = BestIndex;
	}
}

export void LearningKMeansUpdateAssignmentsFromBounds(
	uniform int32 OutAssignments[],
	uniform const float Mins[],
	uniform const float Maxs[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum)
{
	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		uniform float BestDistance = UE_MAX_FLT;
		uniform int32 BestIndex = INDEX_NONE;

		for (uniform int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			float Distance = 0.0f;
			foreach (DimIdx = 0 ... DimNum)
			{
				const float Center = (Maxs[ClusterIdx * DimNum + DimIdx] - Mins[ClusterIdx * DimNum + DimIdx]) / 2.0f + Mins[ClusterIdx * DimNum + DimIdx];
				Distance += abs(Center - Samples[SampleIdx * DimNum + DimIdx]);
			}
			uniform const float CurrDistance = reduce_add(Distance);

			if (CurrDistance < BestDistance)
			{
				BestDistance = CurrDistance;
				BestIndex = ClusterIdx;
			}
		}

		OutAssignments[SampleIdx] = BestIndex;
	}
}

export void LearningKMeansUpdateCenters(
	uniform float OutCenters[],
	uniform const int32 Assignments[],
	uniform const int32 AssignmentCounts[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum)
{
	foreach (ClusterIdx = 0 ... ClusterNum, DimIdx = 0 ... DimNum)
	{
		OutCenters[ClusterIdx * DimNum + DimIdx] = 0.0f;
	}

	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		uniform const int32 CenterIdx = Assignments[SampleIdx];
		uniform int32 AssignmentCount = AssignmentCounts[CenterIdx];

		foreach (DimIdx = 0 ... DimNum)
		{
			OutCenters[CenterIdx * DimNum + DimIdx] += Samples[SampleIdx * DimNum + DimIdx] / AssignmentCount;
		}
	}
}

export void LearningKMeansUpdateBounds(
	uniform float OutMins[],
	uniform float OutMaxs[],
	uniform const int32 Assignments[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum)
{
	foreach(ClusterIdx = 0 ... ClusterNum, DimIdx = 0 ... DimNum)
	{
		OutMins[ClusterIdx * DimNum + DimIdx] = +UE_MAX_FLT;
		OutMaxs[ClusterIdx * DimNum + DimIdx] = -UE_MAX_FLT;
	}

	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		uniform const int32 CenterIdx = Assignments[SampleIdx];

		foreach(DimIdx = 0 ... DimNum)
		{
			OutMins[CenterIdx * DimNum + DimIdx] = min(OutMins[CenterIdx * DimNum + DimIdx], Samples[SampleIdx * DimNum + DimIdx]);
			OutMaxs[CenterIdx * DimNum + DimIdx] = max(OutMaxs[CenterIdx * DimNum + DimIdx], Samples[SampleIdx * DimNum + DimIdx]);
		}
	}
}

export void LearningKMeansComputeClusteredIndex(
	uniform float OutClusteredSamples[],
	uniform int32 OutClusterStarts[],
	uniform int32 OutClusterLengths[],
	uniform int32 OutSampleMapping[],
	uniform int32 OutInverseSampleMapping[],
	uniform const int32 Assignments[],
	uniform const int32 AssignmentCounts[],
	uniform const float Samples[],
	uniform const int SampleNum,
	uniform const int ClusterNum,
	uniform const int DimNum)
{
	uniform int32 ClusterOffset = 0;
	for (uniform int ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
	{
		OutClusterStarts[ClusterIdx] = ClusterOffset;
		ClusterOffset += AssignmentCounts[ClusterIdx];
	}

	// Keep running total of assignments here
	foreach (ClusterIdx = 0 ... ClusterNum)
	{
		OutClusterLengths[ClusterIdx] = 0;
	}

	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		uniform const int32 ClusterIdx = Assignments[SampleIdx];
		uniform const int32 SortedIdx = OutClusterStarts[ClusterIdx] + OutClusterLengths[ClusterIdx];

		foreach (DimIdx = 0 ... DimNum)
		{
			OutClusteredSamples[SortedIdx * DimNum + DimIdx] = Samples[SampleIdx * DimNum + DimIdx];
		}
		OutClusterLengths[ClusterIdx]++;

		OutSampleMapping[SampleIdx] = SortedIdx;
		OutInverseSampleMapping[SortedIdx] = SampleIdx;
	}
}

export void LearningComputeRangeOffsets(
	uniform int32 Offsets[],
	uniform const int32 Lengths[],
	uniform const int RangeNum,
	uniform const int32 InitialOffset)
{
	uniform int32 Offset = InitialOffset;
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		Offsets[RangeIdx] = Offset;
		Offset += Lengths[RangeIdx];
	}
}

export uniform bool LearningContainsSorted(
	uniform const int32 Elements[], 
	uniform const int32 Element,
	uniform const int ElementNum)
{
	if (ElementNum <= 512)
	{
		for (uniform int Idx = 0; Idx < ElementNum; Idx++)
		{
			if (Elements[Idx] == Element) { return true; }
		}
		return false;
	}

	uniform int Low = 0;
	uniform int High = ElementNum - 1;

	while (Low <= High)
	{
		uniform const int Mid = Low + (High - Low) / 2;

		if (Elements[Mid] == Element)
		{
			return true;
		}
		else if (Elements[Mid] < Element)
		{
			Low = Mid + 1;
		}
		else
		{
			High = Mid - 1;
		}
	}

	return false;
}

export uniform int32 LearningFindSorted(
	uniform const int32 Elements[], 
	uniform const int32 Element,
	uniform const int ElementNum)
{
	if (ElementNum <= 512)
	{
		for (uniform int Idx = 0; Idx < ElementNum; Idx++)
		{
			if (Elements[Idx] == Element) { return Idx; }
		}
		return INDEX_NONE;
	}

	uniform int Low = 0;
	uniform int High = ElementNum - 1;

	while (Low <= High)
	{
		uniform const int Mid = Low + (High - Low) / 2;

		if (Elements[Mid] == Element)
		{
			return Mid;
		}
		else if (Elements[Mid] < Element)
		{
			Low = Mid + 1;
		}
		else
		{
			High = Mid - 1;
		}
	}

	return INDEX_NONE;
}

export void LearningFramesCopy(
	uniform int32 DstFrames[],
	uniform const int32 SrcFrames[],
	uniform const int FrameNum)
{
	foreach (FrameIdx = 0 ... FrameNum)
	{
		DstFrames[FrameIdx] = SrcFrames[FrameIdx];
	}
}

export void LearningRangesCopy(
	uniform int32 DstStarts[],
	uniform int32 DstLengths[],
	uniform const int32 SrcStarts[],
	uniform const int32 SrcLengths[],
	uniform const int RangeNum)
{
	foreach (RangeIdx = 0 ... RangeNum)
	{
		DstStarts[RangeIdx] = SrcStarts[RangeIdx];
		DstLengths[RangeIdx] = SrcLengths[RangeIdx];
	}
}

export void LearningRangesCopyFrames(
	uniform int32 DstStarts[],
	uniform int32 DstLengths[],
	uniform const int32 SrcFrames[],
	uniform const int RangeNum)
{
	foreach(RangeIdx = 0 ... RangeNum)
	{
		DstStarts[RangeIdx] = SrcFrames[RangeIdx];
		DstLengths[RangeIdx] = 1;
	}
}

export uniform bool LearningRangesEqual(
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform int LhsRangeNum,
	uniform int RhsRangeNum)
{
	if (LhsRangeNum != RhsRangeNum) { return false; }

	for (uniform int RangeIdx = 0; RangeIdx < LhsRangeNum; RangeIdx++)
	{
		if (LhsStarts[RangeIdx] != RhsStarts[RangeIdx] || LhsLengths[RangeIdx] != RhsLengths[RangeIdx])
		{
			return false;
		}
	}

	return true;
}

export uniform bool LearningRangesContains(
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform const int32 Frame,
	uniform const int RangeNum)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		if (Frame >= Starts[RangeIdx] && Frame < Starts[RangeIdx] + Lengths[RangeIdx])
		{
			return true;
		}
	}
	return false;
}

export uniform bool LearningRangesFind(
	uniform int32& OutRangeIdx,
	uniform int32& OutRangeFrame,
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform const int32 Frame,
	uniform const int RangeNum)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		if (Frame >= Starts[RangeIdx] && Frame < Starts[RangeIdx] + Lengths[RangeIdx])
		{
			OutRangeIdx = RangeIdx;
			OutRangeFrame = Frame - Starts[RangeIdx];
			return true;
		}
	}

	OutRangeIdx = INDEX_NONE;
	OutRangeFrame = INDEX_NONE;
	return false;
}

export uniform bool LearningRangesContainsTime(
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform int32 RangeNum,
	uniform const float Time,
	uniform const float FrameDeltaTime,
	uniform const float Tolerance = UE_KINDA_SMALL_NUMBER)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		uniform const float StartTime = Starts[RangeIdx] * FrameDeltaTime;
		uniform const float EndTime = (Starts[RangeIdx] + Lengths[RangeIdx] - 1) * FrameDeltaTime;

		if (Time >= StartTime - Tolerance && Time < EndTime + Tolerance)
		{
			return true;
		}
	}
	return false;
}

export uniform bool LearningRangesIntersectsRange(
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform const int32 Start,
	uniform const int32 Length,
	uniform const int RangeNum)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		if (Starts[RangeIdx] < Start + Length && Start < Starts[RangeIdx] + Lengths[RangeIdx])
		{
			return true;
		}
	}
	return false;
}

export uniform bool LearningRangesContainsRange(
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform const int32 Start,
	uniform const int32 Length,
	uniform const int RangeNum)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		if (Starts[RangeIdx] <= Start && Start + Length <= Starts[RangeIdx] + Lengths[RangeIdx])
		{
			return true;
		}
	}
	return false;
}

export uniform bool LearningRangesFindTime(
	uniform int32& OutRangeIdx,
	uniform float& OutRangeTime,
	uniform const int32 Starts[],
	uniform const int32 Lengths[],
	uniform const int RangeNum,
	uniform const float Time,
	uniform const float FrameDeltaTime,
	uniform const float Tolerance = UE_KINDA_SMALL_NUMBER)
{
	for (uniform int RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
	{
		uniform const float StartTime = Starts[RangeIdx] * FrameDeltaTime;
		uniform const float EndTime = (Starts[RangeIdx] + Lengths[RangeIdx] - 1) * FrameDeltaTime;
		uniform const float Duration = (Lengths[RangeIdx] - 1) * FrameDeltaTime;

		if (Time >= StartTime - Tolerance && Time < EndTime + Tolerance)
		{
			OutRangeIdx = RangeIdx;
			OutRangeTime = clamp(Time - StartTime, 0.0f, Duration);
			return true;
		}
	}

	OutRangeIdx = INDEX_NONE;
	OutRangeTime = -1.0f;
	return false;
}


export uniform int32 LearningFramesRangesUnion(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const uniform int32 LhsFrames[],
	uniform const uniform int32 RhsStarts[],
	uniform const uniform int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0)
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			RhsStarts,
			RhsLengths,
			RhsNum);

		return RhsNum;
	}

	if (RhsNum == 0)
	{
		LearningRangesCopyFrames(
			OutStarts,
			OutLengths,
			LhsFrames,
			LhsNum);

		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsFrames[LhsIndex / 2] : LhsFrames[LhsIndex / 2] + 1;
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs is coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext && !bRhsActive)
			{

				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsIndex++;
		}
		// Event from rhs is coming first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActive && !bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && (bLhsActiveNext || bRhsActiveNext))
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !(bLhsActiveNext || bRhsActiveNext))
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// Process any remaining lhs events
	while (LhsIndex < LhsNum * 2)
	{
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const int32 LhsT = bLhsActiveNext ? LhsFrames[LhsIndex / 2] : LhsFrames[LhsIndex / 2] + 1;

		// Activate output
		if (!bOutActive && bLhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = LhsT;
		}
		// Deactivate output
		else if (bOutActive && !bLhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bLhsActive = bLhsActiveNext;
		LhsIndex++;
	}

	// Process any remaining rhs events
	while (RhsIndex < RhsNum * 2)
	{
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Activate output
		if (!bOutActive && bRhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = RhsT;
		}
		// Deactivate output
		else if (bOutActive && !bRhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bRhsActive = bRhsActiveNext;
		RhsIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningFramesRangesIntersection(
	uniform int32 OutFrames[],
	uniform const int32 LhsFrames[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{

	if (LhsNum == 0) { return 0; }
	if (RhsNum == 0) { return 0; }

	// Activation state of ranges
	uniform bool bRhsActive = false;

	// Event index for each list
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum && RhsIndex < RhsNum * 2)
	{
		// Are the rhs events active or inactive
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = LhsFrames[LhsIndex];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Add to output if range is active
			if (bRhsActive)
			{
				OutFrames[OutIndex] = LhsT;
				OutIndex++;
			}

			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			bRhsActive = bRhsActiveNext;
			RhsIndex++;

			// Add to output if range is active
			if (bRhsActive)
			{
				OutFrames[OutIndex] = LhsT;
				OutIndex++;
			}

			LhsIndex++;
		}
	}

	// Return number of frames added
	return OutIndex;
}

export uniform int32 LearningFramesRangesDifference(
	uniform int32 OutFrames[],
	uniform const int32 LhsFrames[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0) { return 0; }

	if (RhsNum == 0)
	{
		LearningFramesCopy(OutFrames, LhsFrames, LhsNum);
		return LhsNum;
	}

	// Activation state of ranges
	uniform bool bRhsActive = false;

	// Event index for each list
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum && RhsIndex < RhsNum * 2)
	{
		// Are the rhs events active or inactive
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = LhsFrames[LhsIndex];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Add to output if range is active
			if (!bRhsActive)
			{
				OutFrames[OutIndex] = LhsT;
				OutIndex++;
			}

			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			bRhsActive = bRhsActiveNext;
			RhsIndex++;

			// Add to output if range is active
			if (!bRhsActive)
			{
				OutFrames[OutIndex] = LhsT;
				OutIndex++;
			}

			LhsIndex++;
		}
	}

	// Return number of events added
	return OutIndex;
}


export uniform int32 LearningRangesFramesDifference(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsFrames[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{

	if (LhsNum == 0) { return 0; }

	if (RhsNum == 0)
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsFrames[RhsIndex / 2] : RhsFrames[RhsIndex / 2] + 1;

		// Event coming from lhs first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && !bRhsActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsIndex++;
		}
		// Event coming from rhs first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActive && !bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext && !bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// Process any remaining lhs events
	while (LhsIndex < LhsNum * 2)
	{
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

		// Activate output
		if (!bOutActive && bLhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = LhsT;
		}
		// Deactivate output
		else if (bOutActive && !bLhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bLhsActive = bLhsActiveNext;
		LhsIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}


export uniform int32 LearningRangesUnion(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0)
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			RhsStarts,
			RhsLengths,
			RhsNum);

		return RhsNum;
	}

	if (RhsNum == 0)
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	if (LearningRangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths, LhsNum, RhsNum))
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs is coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext && !bRhsActive)
			{

				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsIndex++;
		}
		// Event from rhs is coming first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActive && !bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && (bLhsActiveNext || bRhsActiveNext))
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !(bLhsActiveNext || bRhsActiveNext))
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// Process any remaining lhs events
	while (LhsIndex < LhsNum * 2)
	{
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

		// Activate output
		if (!bOutActive && bLhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = LhsT;
		}
		// Deactivate output
		else if (bOutActive && !bLhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bLhsActive = bLhsActiveNext;
		LhsIndex++;
	}

	// Process any remaining rhs events
	while (RhsIndex < RhsNum * 2)
	{
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Activate output
		if (!bOutActive && bRhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = RhsT;
		}
		// Deactivate output
		else if (bOutActive && !bRhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bRhsActive = bRhsActiveNext;
		RhsIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningRangesIntersection(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0) { return 0; }
	if (RhsNum == 0) { return 0; }

	if (LearningRangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths, LhsNum, RhsNum))
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bRhsActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActive && bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && !bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && (bLhsActiveNext && bRhsActiveNext))
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && (!bLhsActiveNext || !bRhsActiveNext))
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningRangesTotalFrameNum(uniform const int32 RangeLengths[], uniform const int RangeNum)
{
	int32 Total = 0;

	foreach (RangeIdx = 0 ... RangeNum)
	{
		Total += RangeLengths[RangeIdx];
	}

	return reduce_add(Total);
}

export uniform int32 LearningRangesIntersectionWithOffsets(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform int32 OutLhsOffsets[],
	uniform int32 OutRhsOffsets[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int32 LhsEntryOffset,
	uniform const int32 RhsEntryOffset,
	uniform const int32 LhsNum,
	uniform const int32 RhsNum)
{
	if (LhsNum == 0) { return 0; }
	if (RhsNum == 0) { return 0; }

	if (LearningRangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths, LhsNum, RhsNum))
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		LearningComputeRangeOffsets(OutLhsOffsets, LhsLengths, LhsNum, LhsEntryOffset);
		LearningComputeRangeOffsets(OutRhsOffsets, RhsLengths, RhsNum, RhsEntryOffset);
		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// Offsets
	uniform int32 LhsOffset = LhsEntryOffset;
	uniform int32 RhsOffset = RhsEntryOffset;
	uniform int32 LhsActiveStart = INDEX_NONE;
	uniform int32 RhsActiveStart = INDEX_NONE;
	uniform int32 LhsActiveOffset = INDEX_NONE;
	uniform int32 RhsActiveOffset = INDEX_NONE;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bRhsActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
				OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsActiveStart = bLhsActive ? LhsStarts[LhsIndex / 2] : INDEX_NONE;
			LhsActiveOffset = bLhsActive ? LhsOffset : INDEX_NONE;
			LhsOffset += bLhsActive ? 0 : LhsLengths[LhsIndex / 2];
			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActive && bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && !bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
				OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsActiveStart = bRhsActive ? RhsStarts[RhsIndex / 2] : INDEX_NONE;
			RhsActiveOffset = bRhsActive ? RhsOffset : INDEX_NONE;
			RhsOffset += bRhsActive ? 0 : RhsLengths[RhsIndex / 2];
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && (bLhsActiveNext && bRhsActiveNext))
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && (!bLhsActiveNext || !bRhsActiveNext))
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
				OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsActiveStart = bLhsActive ? LhsStarts[LhsIndex / 2] : INDEX_NONE;
			LhsActiveOffset = bLhsActive ? LhsOffset : INDEX_NONE;
			bRhsActive = bRhsActiveNext;
			RhsActiveStart = bRhsActive ? RhsStarts[RhsIndex / 2] : INDEX_NONE;
			RhsActiveOffset = bRhsActive ? RhsOffset : INDEX_NONE;
			LhsOffset += bLhsActive ? 0 : LhsLengths[LhsIndex / 2];
			RhsOffset += bRhsActive ? 0 : RhsLengths[RhsIndex / 2];
			LhsIndex++; RhsIndex++;
		}
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningRangesDifference(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int32 LhsNum,
	uniform const int32 RhsNum)
{
	if (LhsNum == 0) { return 0; }

	if (RhsNum == 0)
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	if (LearningRangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths, LhsNum, RhsNum))
	{
		return 0;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bLhsActive = false;
	uniform bool bRhsActive = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event coming from lhs first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && !bRhsActive && bLhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			LhsIndex++;
		}
		// Event coming from rhs first
		else if (RhsT < LhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActive && !bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = RhsT;
			}
			// Deactivate output
			else if (bOutActive && bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext && !bRhsActiveNext)
			{
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && bRhsActiveNext)
			{
				bOutActive = false;
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			bLhsActive = bLhsActiveNext;
			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// Process any remaining lhs events
	while (LhsIndex < LhsNum * 2)
	{
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

		// Activate output
		if (!bOutActive && bLhsActiveNext)
		{
			bOutActive = true;
			OutStarts[OutIndex] = LhsT;
		}
		// Deactivate output
		else if (bOutActive && !bLhsActiveNext)
		{
			bOutActive = false;
			OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
			OutIndex++;
		}

		bLhsActive = bLhsActiveNext;
		LhsIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningRangesIntersects(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsStarts[],
	uniform const int32 RhsLengths[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0) { return 0; }
	if (RhsNum == 0) { return 0; }

	if (LearningRangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths, LhsNum, RhsNum))
	{
		LearningRangesCopy(
			OutStarts,
			OutLengths,
			LhsStarts,
			LhsLengths,
			LhsNum);

		return LhsNum;
	}

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bRhsActive = false;
	uniform bool bIntersects = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both ranges have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
	{
		// Are the next lhs, and rhs events active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;
		uniform const bool bRhsActiveNext = RhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bIntersects = bRhsActive;
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				if (bIntersects)
				{
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}
			}

			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			// Mark as intersecting
			if (bOutActive && bRhsActiveNext)
			{
				bIntersects = true;
			}

			bRhsActive = bRhsActiveNext;
			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bIntersects = bRhsActiveNext;
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				if (bIntersects)
				{
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}
			}

			bRhsActive = bRhsActiveNext;
			LhsIndex++; RhsIndex++;
		}
	}

	// While the output is still active and intersecting
	if (bOutActive && bIntersects)
	{
		// Next lhs event should be inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;

		// Time of the next lhs event
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

		// Finish writing range
		OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
		OutIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}

export uniform int32 LearningRangesFramesIntersects(
	uniform int32 OutStarts[],
	uniform int32 OutLengths[],
	uniform const int32 LhsStarts[],
	uniform const int32 LhsLengths[],
	uniform const int32 RhsFrames[],
	uniform const int LhsNum,
	uniform const int RhsNum)
{
	if (LhsNum == 0) { return 0; }
	if (RhsNum == 0) { return 0; }

	// Activation state of each list of ranges
	uniform bool bOutActive = false;
	uniform bool bIntersects = false;

	// Event index for each list of ranges
	uniform int32 LhsIndex = 0;
	uniform int32 RhsIndex = 0;
	uniform int32 OutIndex = 0;

	// While both have events to process
	while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum)
	{
		// Is the next lhs event active or inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;

		// Time of the next lhs, and rhs events
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
		uniform const int32 RhsT = RhsFrames[RhsIndex];

		// Event from lhs coming first
		if (LhsT < RhsT)
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bIntersects = false;
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				if (bIntersects)
				{
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}
			}

			LhsIndex++;
		}
		// Event from rhs coming first
		else if (RhsT < LhsT)
		{
			// Mark as intersecting
			if (bOutActive)
			{
				bIntersects = true;
			}

			RhsIndex++;
		}
		// Event from lhs and rhs coming at same time
		else
		{
			// Activate output
			if (!bOutActive && bLhsActiveNext)
			{
				bIntersects = true;
				bOutActive = true;
				OutStarts[OutIndex] = LhsT;
			}
			// Deactivate output
			else if (bOutActive && !bLhsActiveNext)
			{
				bOutActive = false;
				if (bIntersects)
				{
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}
			}

			LhsIndex++; RhsIndex++;
		}
	}

	// While the output is still active and intersecting
	if (bOutActive && bIntersects)
	{
		// Next lhs event should be inactive
		uniform const bool bLhsActiveNext = LhsIndex % 2 == 0;

		// Time of the next lhs event
		uniform const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

		// Finish writing range
		OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
		OutIndex++;
	}

	// Return number of ranges added
	return OutIndex;
}

export void LearningFilterWithKernel(
	uniform float Out[], 
	uniform const float In[], 
	uniform const float Kernel[],
	uniform const int ValueNum,
	uniform const int KernelNum)
{
	uniform const int KernelHalfWidth = KernelNum / 2;

	foreach (ValueIdx = 0 ... ValueNum)
	{
		float Total = 0.0f;
		Out[ValueIdx] = 0.0f;

		for (int Offset = -KernelHalfWidth; Offset <= +KernelHalfWidth; Offset++)
		{
			if (ValueIdx + Offset >= 0 && ValueIdx + Offset < ValueNum)
			{
				const float Weight = Kernel[Offset + KernelHalfWidth];
				Out[ValueIdx] += Weight * In[ValueIdx + Offset];
				Total += Weight;
			}
		}

		Out[ValueIdx] = Out[ValueIdx] / Total;
	}
}

export void LearningFilterMajorityVote(
	uniform float Out[],
	uniform const float In[],
	uniform const int ValueNum,
	uniform const int FilterHalfWidth)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		int Total = 0;

		for (int Offset = -FilterHalfWidth; Offset <= +FilterHalfWidth; Offset++)
		{
			if (ValueIdx + Offset >= 0 && ValueIdx + Offset < ValueNum)
			{
				Total += In[ValueIdx + Offset] ? 1 : -1;
			}
		}

		Out[ValueIdx] = Total > 0;
	}
}

export void LearningFrameAttributeZero(
	uniform float Out[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = 0.0f;
	}
}

export void LearningFrameAttributeSet(
	uniform float Out[],
	uniform int ValueNum,
	uniform float Value)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = Value;
	}
}

export void LearningFrameAttributeCopy(
	uniform float Out[],
	uniform const float In[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = In[ValueIdx];
	}
}

export uniform bool LearningFrameAttributeEqual(
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	for (uniform int ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
	{
		if (Lhs[ValueIdx] != Rhs[ValueIdx]) { return false; }
	}
	return true;
}

export void LearningFrameAttributeSelect(
	uniform float Out[], 
	uniform const float Cond[],
	uniform const float True[],
	uniform const float False[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = select(Cond[ValueIdx], True[ValueIdx], False[ValueIdx]);
	}
}

export void LearningFrameAttributeAdd(
	uniform float Out[], 
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = Lhs[ValueIdx] + Rhs[ValueIdx];
	}
}

export void LearningFrameAttributeSub(
	uniform float Out[],
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = Lhs[ValueIdx] - Rhs[ValueIdx];
	}
}

export void LearningFrameAttributeMul(
	uniform float Out[],
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = Lhs[ValueIdx] * Rhs[ValueIdx];
	}
}

export void LearningFrameAttributeDiv(
	uniform float Out[],
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = Lhs[ValueIdx] / Rhs[ValueIdx];
	}
}

export void LearningFrameAttributeMax(
	uniform float Out[],
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = max(Lhs[ValueIdx], Rhs[ValueIdx]);
	}
}

export void LearningFrameAttributeMin(
	uniform float Out[],
	uniform const float Lhs[],
	uniform const float Rhs[],
	uniform int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		Out[ValueIdx] = min(Lhs[ValueIdx], Rhs[ValueIdx]);
	}
}

export void LearningFrameAttributeAddConstantInplace(
	uniform float InOut[],
	uniform const float Value,
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] += Value;
	}
}

export void LearningFrameAttributeSubConstantInplace(
	uniform float InOut[],
	uniform const float Value,
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] -= Value;
	}
}

export void LearningFrameAttributeMulConstantInplace(
	uniform float InOut[], 
	uniform const float Value,
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] *= Value;
	}
}

export void LearningFrameAttributeDivConstantInplace(
	uniform float InOut[],
	uniform const float Value,
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] /= Value;
	}
}

export void LearningFrameAttributeConstantDot(
	uniform float InOut[], 
	uniform const float Lhs, 
	uniform const float Rhs[],
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] += Lhs * Rhs[ValueIdx];
	}
}

export void LearningFrameAttributeConstantDistSquared(
	uniform float InOut[], 
	uniform const float Lhs, 
	uniform const float Rhs[],
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] += LearningSquare(Lhs - Rhs[ValueIdx]);
	}
}

export void LearningFrameAttributeApproxAcosInplace(
	uniform float InOut[],
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] = LearningApproxAcos(InOut[ValueIdx]);
	}
}

export void LearningFrameAttributeSqrtInplace(
	uniform float InOut[],
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] = sqrt(InOut[ValueIdx]);
	}
}

export void LearningFrameAttributeLengthSquared(
	uniform float InOut[], 
	uniform const float In[], 
	uniform const int ValueNum)
{
	foreach(ValueIdx = 0 ... ValueNum)
	{
		InOut[ValueIdx] += In[ValueIdx] * In[ValueIdx];
	}
}

export void LearningInertializationDistanceConstant(
	uniform float OutValues[],
	uniform const float Locations[],
	uniform const float Velocities[],
	uniform const int ValueNum,
	uniform const float Location,
	uniform const float Velocity,
	uniform const float BlendTime,
	uniform const float LocWeight,
	uniform const float VelWeight)
{
	foreach (ValueIdx = 0 ... ValueNum)
	{
		const float Integral = LearningInertializationIntegralMagnitude(Locations[ValueIdx] - Location, Velocities[ValueIdx] - Velocity, BlendTime);
		const float MaxVelocity = LearningInertializationMaximumVelocityMagnitude(Locations[ValueIdx] - Location, Velocities[ValueIdx] - Velocity, BlendTime);
		OutValues[ValueIdx] = LocWeight * Integral + VelWeight * MaxVelocity;
	}
}

export uniform bool LearningFrameAttributeFindMinimum(
	uniform int32& OutChannelIdx,
	uniform int32& OutFrameIdx,
	uniform float& OutMinimumValue,
	uniform const float Values[],
	uniform const int ChannelNum,
	uniform const int FrameNum)
{
	OutChannelIdx = INDEX_NONE;
	OutFrameIdx = INDEX_NONE;
	OutMinimumValue = UE_MAX_FLT;

	for (uniform int ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
	{
		// TODO: Vectorize
		for (uniform int FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			if (Values[ChannelIdx * FrameNum + FrameIdx] < OutMinimumValue)
			{
				OutChannelIdx = ChannelIdx;
				OutFrameIdx = FrameIdx;
				OutMinimumValue = Values[ChannelIdx * FrameNum + FrameIdx];
			}
		}
	}

	return OutChannelIdx != INDEX_NONE;
}

export uniform bool LearningFrameAttributeFindMaximum(
	uniform int32& OutChannelIdx,
	uniform int32& OutFrameIdx,
	uniform float& OutMaximumValue,
	uniform const float Values[],
	uniform const int ChannelNum,
	uniform const int FrameNum)
{
	OutChannelIdx = INDEX_NONE;
	OutFrameIdx = INDEX_NONE;
	OutMaximumValue = -UE_MAX_FLT;

	for (uniform int ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
	{
		// TODO: Vectorize
		for (uniform int FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			if (Values[ChannelIdx * FrameNum + FrameIdx] > OutMaximumValue)
			{
				OutChannelIdx = ChannelIdx;
				OutFrameIdx = FrameIdx;
				OutMaximumValue = Values[ChannelIdx * FrameNum + FrameIdx];
			}
		}
	}

	return OutChannelIdx != INDEX_NONE;
}


export uniform bool LearningInt32ArraysEqual(
	uniform const int32 Lhs[],
	uniform const int32 Rhs[],
	uniform const int ValueNum)
{
	// TODO: Vectorize
	for (uniform int ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
	{
		if (Lhs[ValueIdx] != Rhs[ValueIdx]) { return false; }
	}

	return true;
}