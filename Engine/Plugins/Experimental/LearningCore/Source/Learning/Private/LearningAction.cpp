// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAction.h"

#include "LearningRandom.h"

#include "NNERuntimeBasicCpuBuilder.h"

namespace UE::Learning::Action
{
	namespace Private
	{
		template<typename T>
		static inline void ArraySerializeRaw(FArchive& Ar, TArray<T>& Array)
		{
			if (Ar.IsSaving())
			{
				int32 Num = Array.Num();
				Ar << Num;
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(T));
			}
			else if (Ar.IsLoading())
			{
				int32 Num = 0;
				Ar << Num;
				Array.SetNumUninitialized(Num);
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(T));
			}
		}

		static inline bool ContainsDuplicates(const TArrayView<const FName> ElementNames)
		{
			TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<32>> ElementNameSet;
			ElementNameSet.Append(ElementNames);
			return ElementNames.Num() != ElementNameSet.Num();
		}

		static inline bool CheckAllValid(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			for (const FSchemaElement SubElement : Elements)
			{
				if (!Schema.IsValid(SubElement)) { return false; }
			}
			return true;
		}

		static inline int32 GetMaxActionVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size = FMath::Max(Size, Schema.GetActionVectorSize(SubElement));
			}
			return Size;
		}

		static inline int32 GetTotalActionVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetActionVectorSize(SubElement);
			}
			return Size;
		}

		static inline int32 GetTotalEncodedActionVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetEncodedVectorSize(SubElement);
			}
			return Size;
		}

		static inline int32 GetTotalActionDistributionVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetActionDistributionVectorSize(SubElement);
			}
			return Size;
		}

		static inline int32 GetTotalActionModifierVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetActionModifierVectorSize(SubElement);
			}
			return Size;
		}

		static inline bool CheckAllValid(const FObject& Object, const TArrayView<const FObjectElement> Elements)
		{
			for (const FObjectElement SubElement : Elements)
			{
				if (!Object.IsValid(SubElement)) { return false; }
			}
			return true;
		}

		static inline bool CheckPriorProbabilitiesExclusive(const TArrayView<const float> PriorProbabilities, const float Epsilon = UE_KINDA_SMALL_NUMBER)
		{
			if (PriorProbabilities.Num() == 0) { return true; }

			for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
			{
				if (PriorProbabilities[Idx] < 0.0f || PriorProbabilities[Idx] > 1.0f)
				{
					return false;
				}
			}

			float Total = 0.0f;
			for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
			{
				Total += PriorProbabilities[Idx];
			}

			return FMath::Abs(Total - 1.0f) < Epsilon;
		}

		static inline bool CheckPriorProbabilitiesInclusive(const TArrayView<const float> PriorProbabilities)
		{
			if (PriorProbabilities.Num() == 0) { return true; }

			for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
			{
				if (PriorProbabilities[Idx] < 0.0f || PriorProbabilities[Idx] > 1.0f)
				{
					return false;
				}
			}

			return true;
		}

		static inline bool CheckAllValid(const FModifier& Object, const TArrayView<const FModifierElement> Elements)
		{
			for (const FModifierElement SubElement : Elements)
			{
				if (!Object.IsValid(SubElement)) { return false; }
			}
			return true;
		}

		static inline bool CheckExclusiveMaskValid(const TArrayView<const bool> Mask)
		{
			for (int32 MaskIdx = 0; MaskIdx < Mask.Num(); MaskIdx++)
			{
				if (!Mask[MaskIdx]) { return true; }
			}
			return false;
		}

		static inline float Logit(const float X)
		{
			return FMath::Loge(FMath::Max(X / FMath::Max(1.0f - X, FLT_MIN), FLT_MIN));
		}

		struct FContinuousDataDeprecated
		{
			int32 Num = INDEX_NONE;
			float Scale = 0.0f;
		};
	}

	FSchemaElement FSchema::CreateNull(const FName Tag)
	{
		const int32 Index = Types.Add(EType::Null);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(0);
		ActionVectorSizes.Add(0);
		ActionDistributionVectorSizes.Add(0);
		ActionModifierVectorSizes.Add(1);
		TypeDataIndices.Add(INDEX_NONE);

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateContinuous(const FSchemaContinuousParameters Parameters, const FName Tag)
	{
		check(Parameters.Num >= 0);

		FContinuousData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.NormalizationOption = Parameters.NormalizationOption;

		if (Parameters.NormalizationOption == ENormalization::Manual)
		{
			check(Parameters.Scales.Num() == Parameters.Num or Parameters.Scales.IsEmpty());
			check(Parameters.Offsets.Num() == Parameters.Num or Parameters.Offsets.IsEmpty());

			ElementData.NormalizationIndexOffset = NormalizationScales.Num();

			// Use provided arrays or identity (std=1, mean=0)
			if (Parameters.Scales.Num() == Parameters.Num)
			{
				NormalizationScales.Append(Parameters.Scales.GetData(), Parameters.Scales.Num());
			}
			else
			{
				for (int32 i = 0; i < Parameters.Num; i++)
				{
					NormalizationScales.Add(1.0f);
				}
			}

			if (Parameters.Offsets.Num() == Parameters.Num)
			{
				NormalizationOffsets.Append(Parameters.Offsets.GetData(), Parameters.Offsets.Num());
			}
			else
			{
				for (int32 i = 0; i < Parameters.Num; i++)
				{
					NormalizationOffsets.Add(0.0f);
				}
			}
		}

		const int32 Index = Types.Add(EType::Continuous);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(2 * Parameters.Num);
		ActionVectorSizes.Add(Parameters.Num);
		ActionDistributionVectorSizes.Add(2 * Parameters.Num);
		ActionModifierVectorSizes.Add(1 + 2 * Parameters.Num);
		TypeDataIndices.Add(ContinuousData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateDiscreteExclusive(const FSchemaDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.PriorProbabilities.Num() == Parameters.Num);
		check(Private::CheckPriorProbabilitiesExclusive(Parameters.PriorProbabilities));

		FDiscreteExclusiveData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();

		PriorProbabilities.Append(Parameters.PriorProbabilities);

		const int32 Index = Types.Add(EType::DiscreteExclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Parameters.Num);
		ActionVectorSizes.Add(Parameters.Num);
		ActionDistributionVectorSizes.Add(Parameters.Num);
		ActionModifierVectorSizes.Add(1 + Parameters.Num);
		TypeDataIndices.Add(DiscreteExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateDiscreteInclusive(const FSchemaDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.PriorProbabilities.Num() == Parameters.Num);
		check(Private::CheckPriorProbabilitiesInclusive(Parameters.PriorProbabilities));

		FDiscreteInclusiveData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();

		PriorProbabilities.Append(Parameters.PriorProbabilities);

		const int32 Index = Types.Add(EType::DiscreteInclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Parameters.Num);
		ActionVectorSizes.Add(Parameters.Num);
		ActionDistributionVectorSizes.Add(Parameters.Num);
		ActionModifierVectorSizes.Add(1 + Parameters.Num);
		TypeDataIndices.Add(DiscreteInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateNamedDiscreteExclusive(const FSchemaNamedDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.PriorProbabilities.Num() == Parameters.ElementNames.Num());
		check(Private::CheckPriorProbabilitiesExclusive(Parameters.PriorProbabilities));
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		FNamedDiscreteExclusiveData ElementData;
		ElementData.Num = Parameters.ElementNames.Num();
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();
		ElementData.ElementsOffset = SubElementNames.Num();

		PriorProbabilities.Append(Parameters.PriorProbabilities);
		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < ElementData.Num; Idx++) { SubElementObjects.Add(FSchemaElement()); }

		const int32 Index = Types.Add(EType::NamedDiscreteExclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Parameters.ElementNames.Num());
		ActionVectorSizes.Add(Parameters.ElementNames.Num());
		ActionDistributionVectorSizes.Add(Parameters.ElementNames.Num());
		ActionModifierVectorSizes.Add(1 + Parameters.ElementNames.Num());
		TypeDataIndices.Add(NamedDiscreteExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateNamedDiscreteInclusive(const FSchemaNamedDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.PriorProbabilities.Num() == Parameters.ElementNames.Num());
		check(Private::CheckPriorProbabilitiesInclusive(Parameters.PriorProbabilities));
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		FNamedDiscreteInclusiveData ElementData;
		ElementData.Num = Parameters.ElementNames.Num();
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();
		ElementData.ElementsOffset = SubElementNames.Num();

		PriorProbabilities.Append(Parameters.PriorProbabilities);
		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < ElementData.Num; Idx++) { SubElementObjects.Add(FSchemaElement()); }

		const int32 Index = Types.Add(EType::NamedDiscreteInclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Parameters.ElementNames.Num());
		ActionVectorSizes.Add(Parameters.ElementNames.Num());
		ActionDistributionVectorSizes.Add(Parameters.ElementNames.Num());
		ActionModifierVectorSizes.Add(1 + Parameters.ElementNames.Num());
		TypeDataIndices.Add(NamedDiscreteInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateAnd(const FSchemaAndParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		FAndData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		const int32 Index = Types.Add(EType::And);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Private::GetTotalEncodedActionVectorSize(*this, Parameters.Elements));
		ActionVectorSizes.Add(Private::GetTotalActionVectorSize(*this, Parameters.Elements));
		ActionDistributionVectorSizes.Add(Private::GetTotalActionDistributionVectorSize(*this, Parameters.Elements));
		ActionModifierVectorSizes.Add(1 + Private::GetTotalActionModifierVectorSize(*this, Parameters.Elements));
		TypeDataIndices.Add(AndData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateOrExclusive(const FSchemaOrExclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));
		check(Parameters.PriorProbabilities.Num() == Parameters.Elements.Num());
		check(Private::CheckPriorProbabilitiesExclusive(Parameters.PriorProbabilities));

		FOrExclusiveData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);
		PriorProbabilities.Append(Parameters.PriorProbabilities);

		const int32 Index = Types.Add(EType::OrExclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Private::GetTotalEncodedActionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionVectorSizes.Add(Private::GetMaxActionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionDistributionVectorSizes.Add(Private::GetTotalActionDistributionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionModifierVectorSizes.Add(1 + Parameters.Elements.Num() + Private::GetTotalActionModifierVectorSize(*this, Parameters.Elements));
		TypeDataIndices.Add(OrExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateOrInclusive(const FSchemaOrInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));
		check(Parameters.PriorProbabilities.Num() == Parameters.Elements.Num());
		check(Private::CheckPriorProbabilitiesInclusive(Parameters.PriorProbabilities));

		FOrInclusiveData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();
		ElementData.PriorProbabilitiesOffset = PriorProbabilities.Num();

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);
		PriorProbabilities.Append(Parameters.PriorProbabilities);

		const int32 Index = Types.Add(EType::OrInclusive);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(Private::GetTotalEncodedActionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionVectorSizes.Add(Private::GetTotalActionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionDistributionVectorSizes.Add(Private::GetTotalActionDistributionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		ActionModifierVectorSizes.Add(1 + Parameters.Elements.Num() + Private::GetTotalActionModifierVectorSize(*this, Parameters.Elements));
		TypeDataIndices.Add(OrInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateArray(const FSchemaArrayParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));
		check(Parameters.Num >= 0);

		FArrayData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.ElementIndex = SubElementObjects.Num();

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Array);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(GetEncodedVectorSize(Parameters.Element) * Parameters.Num);
		ActionVectorSizes.Add(GetActionVectorSize(Parameters.Element) * Parameters.Num);
		ActionDistributionVectorSizes.Add(GetActionDistributionVectorSize(Parameters.Element) * Parameters.Num);
		ActionModifierVectorSizes.Add(1 + GetActionModifierVectorSize(Parameters.Element) * Parameters.Num);
		TypeDataIndices.Add(ArrayData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateEncoding(const FSchemaEncodingParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		FEncodingData ElementData;
		ElementData.EncodingSize = Parameters.EncodingSize;
		ElementData.LayerNum = Parameters.LayerNum;
		ElementData.ActivationFunction = Parameters.ActivationFunction;
		ElementData.ElementIndex = SubElementObjects.Num();

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Encoding);
		Tags.Add(Tag);
		EncodedVectorSizes.Add(ElementData.EncodingSize);
		ActionVectorSizes.Add(GetActionVectorSize(Parameters.Element));
		ActionDistributionVectorSizes.Add(GetActionDistributionVectorSize(Parameters.Element));
		ActionModifierVectorSizes.Add(1 + GetActionModifierVectorSize(Parameters.Element));
		TypeDataIndices.Add(EncodingData.Add(ElementData));

		return { Index, Generation };
	}

	bool FSchema::IsValid(const FSchemaElement Element) const
	{
		return Element.Generation == Generation && Element.Index != INDEX_NONE;
	}

	EType FSchema::GetType(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return Types[Element.Index];
	}

	FName FSchema::GetTag(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return Tags[Element.Index];
	}

	int32 FSchema::GetEncodedVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return EncodedVectorSizes[Element.Index];
	}

	int32 FSchema::GetActionVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return ActionVectorSizes[Element.Index];
	}

	int32 FSchema::GetActionDistributionVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return ActionDistributionVectorSizes[Element.Index];
	}

	int32 FSchema::GetActionModifierVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return ActionModifierVectorSizes[Element.Index];
	}

	FSchemaContinuousParameters FSchema::GetContinuous(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Continuous);
		const FContinuousData& ElementData = ContinuousData[TypeDataIndices[Element.Index]];

		FSchemaContinuousParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.NormalizationOption = ElementData.NormalizationOption;

		if (Parameters.NormalizationOption == ENormalization::Manual)
		{
			Parameters.Scales = TArrayView<const float>(NormalizationScales.GetData() + ElementData.NormalizationIndexOffset, ElementData.Num);
			Parameters.Offsets = TArrayView<const float>(NormalizationOffsets.GetData() + ElementData.NormalizationIndexOffset, ElementData.Num);
		}
				
		return Parameters;
	}

	FSchemaDiscreteExclusiveParameters FSchema::GetDiscreteExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteExclusive);
		const FDiscreteExclusiveData& ElementData = DiscreteExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaDiscreteExclusiveParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaDiscreteInclusiveParameters FSchema::GetDiscreteInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteInclusive);
		const FDiscreteInclusiveData& ElementData = DiscreteInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaDiscreteInclusiveParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaNamedDiscreteExclusiveParameters FSchema::GetNamedDiscreteExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteExclusive);
		const FNamedDiscreteExclusiveData& ElementData = NamedDiscreteExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaNamedDiscreteExclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaNamedDiscreteInclusiveParameters FSchema::GetNamedDiscreteInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteInclusive);
		const FNamedDiscreteInclusiveData& ElementData = NamedDiscreteInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaNamedDiscreteInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaAndParameters FSchema::GetAnd(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::And);
		const FAndData& ElementData = AndData[TypeDataIndices[Element.Index]];

		FSchemaAndParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaOrExclusiveParameters FSchema::GetOrExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrExclusive);
		const FOrExclusiveData& ElementData = OrExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaOrExclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaOrInclusiveParameters FSchema::GetOrInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrInclusive);
		const FOrInclusiveData& ElementData = OrInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaOrInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.PriorProbabilities = TArrayView<const float>(PriorProbabilities.GetData() + ElementData.PriorProbabilitiesOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaArrayParameters FSchema::GetArray(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Array);
		const FArrayData& ElementData = ArrayData[TypeDataIndices[Element.Index]];

		FSchemaArrayParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		return Parameters;
	}

	FSchemaEncodingParameters FSchema::GetEncoding(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Encoding);
		const FEncodingData& ElementData = EncodingData[TypeDataIndices[Element.Index]];

		FSchemaEncodingParameters Parameters;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		Parameters.EncodingSize = ElementData.EncodingSize;
		Parameters.LayerNum = ElementData.LayerNum;
		Parameters.ActivationFunction = ElementData.ActivationFunction;
		return Parameters;
	}

	void FSchema::SetGeneration(const uint32 InGeneration)
	{
		Generation = InGeneration;
	}

	uint32 FSchema::GetGeneration() const
	{
		return Generation;
	}

	void FSchema::Empty()
	{
		Types.Empty();
		Tags.Empty();
		EncodedVectorSizes.Empty();
		ActionVectorSizes.Empty();
		ActionDistributionVectorSizes.Empty();
		ActionModifierVectorSizes.Empty();
		TypeDataIndices.Empty();

		ContinuousData.Empty();
		DiscreteExclusiveData.Empty();
		DiscreteInclusiveData.Empty();
		NamedDiscreteExclusiveData.Empty();
		NamedDiscreteInclusiveData.Empty();
		AndData.Empty();
		OrExclusiveData.Empty();
		OrInclusiveData.Empty();
		ArrayData.Empty();
		EncodingData.Empty();

		SubElementNames.Empty();
		SubElementObjects.Empty();
		PriorProbabilities.Empty();
		NormalizationScales.Empty();
		NormalizationOffsets.Empty();

		Generation++;
	}

	bool FSchema::IsEmpty() const
	{
		return Types.IsEmpty();
	}

	void FSchema::Reset()
	{
		Types.Reset();
		Tags.Reset();
		EncodedVectorSizes.Reset();
		ActionVectorSizes.Reset();
		ActionDistributionVectorSizes.Reset();
		ActionModifierVectorSizes.Reset();
		TypeDataIndices.Reset();

		ContinuousData.Reset();
		DiscreteExclusiveData.Reset();
		DiscreteInclusiveData.Reset();
		NamedDiscreteExclusiveData.Reset();
		NamedDiscreteInclusiveData.Reset();		
		AndData.Reset();
		OrExclusiveData.Reset();
		OrInclusiveData.Reset();
		ArrayData.Reset();
		EncodingData.Reset();

		SubElementNames.Reset();
		SubElementObjects.Reset();
		PriorProbabilities.Reset();
		NormalizationScales.Reset();
		NormalizationOffsets.Reset();

		Generation++;
	}

	bool FSchema::Serialize(FArchive& Ar)
	{
		static constexpr int32 Magic = 0x25344a5c;
		static constexpr int32 Version = 1;

		bool bSerializeDeprecatedContinuousData = false;

		if (Ar.IsLoading())
		{
			int64 Offset = Ar.Tell();
			int32 LoadMagic = 0;
			int32 LoadVersion = 0;
			Ar << LoadMagic;
			if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); Empty(); return false; }
			Ar << LoadVersion;

			if (LoadVersion == 0) { bSerializeDeprecatedContinuousData = true; }
			else if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); Empty(); return false; }
		}
		else
		{
			int32 SaveMagic = Magic;
			int32 SaveVersion = Version;
			Ar << SaveMagic;
			Ar << SaveVersion;
		}

		Ar << Generation;

		Ar << Types;
		Ar << Tags;
		Ar << EncodedVectorSizes;
		Ar << ActionVectorSizes;
		Ar << ActionDistributionVectorSizes;
		Ar << ActionModifierVectorSizes;
		Ar << TypeDataIndices;

		if (bSerializeDeprecatedContinuousData)
		{
			TArray<Private::FContinuousDataDeprecated> DeprecatedContinuousData;
			Private::ArraySerializeRaw(Ar, DeprecatedContinuousData);
						
			for (const Private::FContinuousDataDeprecated& DeprecatedData : DeprecatedContinuousData)
			{
				FContinuousData CurrentData;

				CurrentData.Num = DeprecatedData.Num;
				CurrentData.NormalizationOption = ENormalization::Manual;
				CurrentData.NormalizationIndexOffset = NormalizationScales.Num();

				for (int32 i = 0; i < DeprecatedData.Num; ++i)
				{
					NormalizationScales.Add(DeprecatedData.Scale);
					NormalizationOffsets.Add(0.0f);
				}

				ContinuousData.Add(CurrentData);
			}
		}
		else
		{
			Private::ArraySerializeRaw(Ar, ContinuousData);
		}

		Private::ArraySerializeRaw(Ar, DiscreteExclusiveData);
		Private::ArraySerializeRaw(Ar, DiscreteInclusiveData);
		Private::ArraySerializeRaw(Ar, NamedDiscreteExclusiveData);
		Private::ArraySerializeRaw(Ar, NamedDiscreteInclusiveData);
		Private::ArraySerializeRaw(Ar, AndData);
		Private::ArraySerializeRaw(Ar, OrExclusiveData);
		Private::ArraySerializeRaw(Ar, OrInclusiveData);
		Private::ArraySerializeRaw(Ar, ArrayData);
		Private::ArraySerializeRaw(Ar, EncodingData);

		Ar << SubElementNames;
		Ar << SubElementObjects;

		Ar << PriorProbabilities;

		if (!bSerializeDeprecatedContinuousData)
		{
			Private::ArraySerializeRaw(Ar, NormalizationScales);
			Private::ArraySerializeRaw(Ar, NormalizationOffsets);
		}

		return true;
	}

	FObjectElement FObject::CreateNull(const FName Tag)
	{
		const int32 Index = Types.Add(EType::Null);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(0);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateContinuous(const FObjectContinuousParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::Continuous);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(Parameters.Values.Num());

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(0);

		ContinuousValues.Append(Parameters.Values);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateDiscreteExclusive(const FObjectDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(1);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(0);

		DiscreteValues.Add(Parameters.DiscreteIndex);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateDiscreteInclusive(const FObjectDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(Parameters.DiscreteIndices.Num());

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(0);

		DiscreteValues.Append(Parameters.DiscreteIndices);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateNamedDiscreteExclusive(const FObjectNamedDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::NamedDiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(1);

		SubElementObjects.Add(FObjectElement());
		SubElementNames.Add(Parameters.ElementName);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateNamedDiscreteInclusive(const FObjectNamedDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::NamedDiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(Parameters.ElementNames.Num());

		for (int32 Idx = 0; Idx < Parameters.ElementNames.Num(); Idx++) { SubElementObjects.Add(FObjectElement()); }
		SubElementNames.Append(Parameters.ElementNames);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateAnd(const FObjectAndParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::And);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		SubElementObjects.Append(Parameters.Elements);
		SubElementNames.Append(Parameters.ElementNames);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateOrExclusive(const FObjectOrExclusiveParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::OrExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(1);

		SubElementObjects.Add(Parameters.Element);
		SubElementNames.Add(Parameters.ElementName);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateOrInclusive(const FObjectOrInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::OrInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		SubElementObjects.Append(Parameters.Elements);
		SubElementNames.Append(Parameters.ElementNames);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateArray(const FObjectArrayParameters Parameters, const FName Tag)
	{
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::Array);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			SubElementNames.Add(NAME_None);
		}
		SubElementObjects.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateEncoding(const FObjectEncodingParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::Encoding);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementObjects.Num());
		ElementDataNums.Add(1);

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		return { Index, Generation };
	}

	bool FObject::IsValid(const FObjectElement Element) const
	{
		return Element.Generation == Generation && Element.Index != INDEX_NONE;
	}

	EType FObject::GetType(const FObjectElement Element) const
	{
		check(IsValid(Element));
		return Types[Element.Index];
	}
	
	FName FObject::GetTag(const FObjectElement Element) const
	{
		check(IsValid(Element));
		return Tags[Element.Index];
	}

	FObjectContinuousParameters FObject::GetContinuous(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Continuous);

		FObjectContinuousParameters Parameters;
		Parameters.Values = TArrayView<const float>(ContinuousValues.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		return Parameters;
	}

	FObjectDiscreteExclusiveParameters FObject::GetDiscreteExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteExclusive);

		FObjectDiscreteExclusiveParameters Parameters;
		Parameters.DiscreteIndex = DiscreteValues[DiscreteDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectDiscreteInclusiveParameters FObject::GetDiscreteInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteInclusive);

		FObjectDiscreteInclusiveParameters Parameters;
		Parameters.DiscreteIndices = TArrayView<const int32>(DiscreteValues.GetData() + DiscreteDataOffsets[Element.Index], DiscreteDataNums[Element.Index]);
		return Parameters;
	}

	FObjectNamedDiscreteExclusiveParameters FObject::GetNamedDiscreteExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteExclusive);

		FObjectNamedDiscreteExclusiveParameters Parameters;
		Parameters.ElementName = SubElementNames[ElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectNamedDiscreteInclusiveParameters FObject::GetNamedDiscreteInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteInclusive);

		FObjectNamedDiscreteInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectAndParameters FObject::GetAnd(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::And);

		FObjectAndParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectOrExclusiveParameters FObject::GetOrExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrExclusive);

		FObjectOrExclusiveParameters Parameters;
		Parameters.ElementName = SubElementNames[ElementDataOffsets[Element.Index]];
		Parameters.Element = SubElementObjects[ElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectOrInclusiveParameters FObject::GetOrInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrInclusive);

		FObjectOrInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectArrayParameters FObject::GetArray(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Array);
		
		FObjectArrayParameters Parameters;
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectEncodingParameters FObject::GetEncoding(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Encoding);

		FObjectEncodingParameters Parameters;
		Parameters.Element = SubElementObjects[ElementDataOffsets[Element.Index]];
		return Parameters;
	}

	void FObject::SetGeneration(const uint32 InGeneration)
	{
		Generation = InGeneration;
	}

	uint32 FObject::GetGeneration() const
	{
		return Generation;
	}

	void FObject::Empty()
	{
		Types.Empty();
		Tags.Empty();
		ContinuousDataOffsets.Empty();
		ContinuousDataNums.Empty();
		DiscreteDataOffsets.Empty();
		DiscreteDataNums.Empty();
		ElementDataOffsets.Empty();
		ElementDataNums.Empty();

		ContinuousValues.Empty();
		DiscreteValues.Empty();
		SubElementObjects.Empty();
		SubElementNames.Empty();

		Generation++;
	}

	bool FObject::IsEmpty() const
	{
		return Types.IsEmpty();
	}

	void FObject::Reset()
	{
		Types.Reset();
		Tags.Reset();
		ContinuousDataOffsets.Reset();
		ContinuousDataNums.Reset();
		DiscreteDataOffsets.Reset();
		DiscreteDataNums.Reset();
		ElementDataOffsets.Reset();
		ElementDataNums.Reset();

		ContinuousValues.Reset();
		DiscreteValues.Reset();
		SubElementObjects.Reset();
		SubElementNames.Reset();

		Generation++;
	}

	bool FObject::Serialize(FArchive& Ar)
	{
		static constexpr int32 Magic = 0x535a63fb;
		static constexpr int32 Version = 0;

		if (Ar.IsLoading())
		{
			int64 Offset = Ar.Tell();
			int32 LoadMagic = 0;
			int32 LoadVersion = 0;
			Ar << LoadMagic;
			if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); Empty(); return false; }
			Ar << LoadVersion;
			if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); Empty(); return false; }
		}
		else
		{
			int32 SaveMagic = Magic;
			int32 SaveVersion = Version;
			Ar << SaveMagic;
			Ar << SaveVersion;
		}

		Ar << Generation;

		Ar << Types;
		Ar << Tags;

		Ar << ContinuousDataOffsets;
		Ar << ContinuousDataNums;

		Ar << DiscreteDataOffsets;
		Ar << DiscreteDataNums;

		Ar << ElementDataOffsets;
		Ar << ElementDataNums;

		Ar << ContinuousValues;
		Ar << DiscreteValues;
		Ar << SubElementObjects;
		Ar << SubElementNames;

		return true;
	}

	FModifierElement FModifier::CreateNull(const FName Tag)
	{
		const int32 Index = Types.Add(EType::Null);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(0);

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateContinuous(const FModifierContinuousParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::Continuous);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(Parameters.MaskedValues.Num());

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(0);

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		ContinuousMaskeds.Append(Parameters.Masked);
		ContinuousMaskedValues.Append(Parameters.MaskedValues);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateDiscreteExclusive(const FModifierDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(Parameters.MaskedIndices.Num());

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(0);

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		DiscreteValues.Append(Parameters.MaskedIndices);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateDiscreteInclusive(const FModifierDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(Parameters.MaskedIndices.Num());

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(0);

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		DiscreteValues.Append(Parameters.MaskedIndices);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateNamedDiscreteExclusive(const FModifierNamedDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::NamedDiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.MaskedElementNames.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		SubElementNames.Append(Parameters.MaskedElementNames);
		for (int32 Idx = 0; Idx < Parameters.MaskedElementNames.Num(); Idx++) { SubElementModifiers.Add(FModifierElement()); }

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateNamedDiscreteInclusive(const FModifierNamedDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::NamedDiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.MaskedElementNames.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		SubElementNames.Append(Parameters.MaskedElementNames);
		for (int32 Idx = 0; Idx < Parameters.MaskedElementNames.Num(); Idx++) { SubElementModifiers.Add(FModifierElement()); }

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateAnd(const FModifierAndParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::And);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		SubElementModifiers.Append(Parameters.Elements);
		SubElementNames.Append(Parameters.ElementNames);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateOrExclusive(const FModifierOrExclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(!Private::ContainsDuplicates(Parameters.MaskedElements));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::OrExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(Parameters.MaskedElements.Num());

		SubElementModifiers.Append(Parameters.Elements);
		SubElementNames.Append(Parameters.ElementNames);

		MaskedElementNames.Append(Parameters.MaskedElements);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateOrInclusive(const FModifierOrInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(!Private::ContainsDuplicates(Parameters.MaskedElements));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::OrInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(Parameters.MaskedElements.Num());

		SubElementModifiers.Append(Parameters.Elements);
		SubElementNames.Append(Parameters.ElementNames);

		MaskedElementNames.Append(Parameters.MaskedElements);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateArray(const FModifierArrayParameters Parameters, const FName Tag)
	{
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::Array);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(Parameters.Elements.Num());

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			SubElementNames.Add(NAME_None);
		}
		SubElementModifiers.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FModifierElement FModifier::CreateEncoding(const FModifierEncodingParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::Encoding);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousMaskeds.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		ElementDataOffsets.Add(SubElementModifiers.Num());
		ElementDataNums.Add(1);

		MaskedDataOffsets.Add(MaskedElementNames.Num());
		MaskedDataNums.Add(0);

		SubElementNames.Add(NAME_None);
		SubElementModifiers.Add(Parameters.Element);

		return { Index, Generation };
	}

	bool FModifier::IsValid(const FModifierElement Element) const
	{
		return Element.Generation == Generation && Element.Index != INDEX_NONE;
	}

	EType FModifier::GetType(const FModifierElement Element) const
	{
		check(IsValid(Element));
		return Types[Element.Index];
	}

	FName FModifier::GetTag(const FModifierElement Element) const
	{
		check(IsValid(Element));
		return Tags[Element.Index];
	}

	FModifierContinuousParameters FModifier::GetContinuous(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Continuous);

		FModifierContinuousParameters Parameters;
		Parameters.Masked = TArrayView<const bool>(ContinuousMaskeds.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		Parameters.MaskedValues = TArrayView<const float>(ContinuousMaskedValues.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		return Parameters;
	}

	FModifierDiscreteExclusiveParameters FModifier::GetDiscreteExclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteExclusive);

		FModifierDiscreteExclusiveParameters Parameters;
		Parameters.MaskedIndices = TArrayView<const int32>(DiscreteValues.GetData() + DiscreteDataOffsets[Element.Index], DiscreteDataNums[Element.Index]);
		return Parameters;
	}

	FModifierDiscreteInclusiveParameters FModifier::GetDiscreteInclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteInclusive);

		FModifierDiscreteInclusiveParameters Parameters;
		Parameters.MaskedIndices = TArrayView<const int32>(DiscreteValues.GetData() + DiscreteDataOffsets[Element.Index], DiscreteDataNums[Element.Index]);
		return Parameters;
	}

	FModifierNamedDiscreteExclusiveParameters FModifier::GetNamedDiscreteExclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteExclusive);

		FModifierNamedDiscreteExclusiveParameters Parameters;
		Parameters.MaskedElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FModifierNamedDiscreteInclusiveParameters FModifier::GetNamedDiscreteInclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteInclusive);

		FModifierNamedDiscreteInclusiveParameters Parameters;
		Parameters.MaskedElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FModifierAndParameters FModifier::GetAnd(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::And);

		FModifierAndParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FModifierElement>(SubElementModifiers.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FModifierOrExclusiveParameters FModifier::GetOrExclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrExclusive);

		FModifierOrExclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FModifierElement>(SubElementModifiers.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.MaskedElements = TArrayView<const FName>(MaskedElementNames.GetData() + MaskedDataOffsets[Element.Index], MaskedDataNums[Element.Index]);
		return Parameters;
	}

	FModifierOrInclusiveParameters FModifier::GetOrInclusive(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrInclusive);

		FModifierOrInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FModifierElement>(SubElementModifiers.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		Parameters.MaskedElements = TArrayView<const FName>(MaskedElementNames.GetData() + MaskedDataOffsets[Element.Index], MaskedDataNums[Element.Index]);
		return Parameters;
	}

	FModifierArrayParameters FModifier::GetArray(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Array);

		FModifierArrayParameters Parameters;
		Parameters.Elements = TArrayView<const FModifierElement>(SubElementModifiers.GetData() + ElementDataOffsets[Element.Index], ElementDataNums[Element.Index]);
		return Parameters;
	}

	FModifierEncodingParameters FModifier::GetEncoding(const FModifierElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Encoding);

		FModifierEncodingParameters Parameters;
		Parameters.Element = SubElementModifiers[ElementDataOffsets[Element.Index]];
		return Parameters;
	}

	void FModifier::SetGeneration(const uint32 InGeneration)
	{
		Generation = InGeneration;
	}

	uint32 FModifier::GetGeneration() const
	{
		return Generation;
	}

	void FModifier::Empty()
	{
		Types.Empty();
		Tags.Empty();
		ContinuousDataOffsets.Empty();
		ContinuousDataNums.Empty();
		DiscreteDataOffsets.Empty();
		DiscreteDataNums.Empty();
		ElementDataOffsets.Empty();
		ElementDataNums.Empty();
		MaskedDataOffsets.Empty();
		MaskedDataNums.Empty();

		ContinuousMaskeds.Empty();
		ContinuousMaskedValues.Empty();
		DiscreteValues.Empty();
		SubElementModifiers.Empty();
		SubElementNames.Empty();
		MaskedElementNames.Empty();

		Generation++;
	}

	bool FModifier::IsEmpty() const
	{
		return Types.IsEmpty();
	}

	void FModifier::Reset()
	{
		Types.Reset();
		Tags.Reset();
		ContinuousDataOffsets.Reset();
		ContinuousDataNums.Reset();
		DiscreteDataOffsets.Reset();
		DiscreteDataNums.Reset();
		ElementDataOffsets.Reset();
		ElementDataNums.Reset();
		MaskedDataOffsets.Reset();
		MaskedDataNums.Reset();

		ContinuousMaskeds.Reset();
		ContinuousMaskedValues.Reset();
		DiscreteValues.Reset();
		SubElementModifiers.Reset();
		SubElementNames.Reset();
		MaskedElementNames.Reset();

		Generation++;
	}

	bool FModifier::Serialize(FArchive& Ar)
	{
		static constexpr int32 Magic = 0x5342479b;
		static constexpr int32 Version = 0;

		if (Ar.IsLoading())
		{
			int64 Offset = Ar.Tell();
			int32 LoadMagic = 0;
			int32 LoadVersion = 0;
			Ar << LoadMagic;
			if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); Empty(); return false; }
			Ar << LoadVersion;
			if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); Empty(); return false; }
		}
		else
		{
			int32 SaveMagic = Magic;
			int32 SaveVersion = Version;
			Ar << SaveMagic;
			Ar << SaveVersion;
		}

		Ar << Generation;

		Ar << Types;
		Ar << Tags;
		Ar << ContinuousDataOffsets;
		Ar << ContinuousDataNums;
		Ar << DiscreteDataOffsets;
		Ar << DiscreteDataNums;
		Ar << ElementDataOffsets;
		Ar << ElementDataNums;
		Ar << MaskedDataOffsets;
		Ar << MaskedDataNums;

		Ar << ContinuousMaskeds;
		Ar << ContinuousMaskedValues;
		Ar << DiscreteValues;
		Ar << SubElementModifiers;
		Ar << SubElementNames;
		Ar << MaskedElementNames;

		return true;
	}

	namespace Private
	{
		static inline NNE::RuntimeBasic::FModelBuilder::EActivationFunction GetNNEActivationFunction(const EEncodingActivationFunction ActivationFunction)
		{
			switch (ActivationFunction)
			{
			case EEncodingActivationFunction::ReLU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
			case EEncodingActivationFunction::ELU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ELU;
			case EEncodingActivationFunction::TanH: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::TanH;
			case EEncodingActivationFunction::GELU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::GELU;
			default: checkNoEntry(); return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
			}
		}

		static inline int32 HashFNameStable(const FName Name)
		{
			const FString NameString = Name.ToString().ToLower();
			return (int32)CityHash32(
				(const char*)NameString.GetCharArray().GetData(),
				sizeof(NameString[0]) * NameString.GetCharArray().Num());
		}

		static inline int32 HashInt(const int32 Int)
		{
			return (int32)CityHash32((const char*)&Int, sizeof(int32));
		}

		static inline int32 HashCombine(const TArrayView<const int32> Hashes)
		{
			return (int32)CityHash32((const char*)Hashes.GetData(), Hashes.Num() * Hashes.GetTypeSize());
		}

		static inline int32 HashElements(
			const FSchema& Schema,
			const TArrayView<const FName> SchemaElementNames,
			const int32 Salt)
		{
			// Note: Here we xor all entries together. 
			// This makes the hash in invariant to the ordering of names which is actually what we want 
			// since this array is representing a set-like structure and it is fine to pass elements in a different order.

			int32 Hash = 0x9de53147;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaElementNames.Num(); SchemaElementIdx++)
			{
				Hash ^= HashFNameStable(SchemaElementNames[SchemaElementIdx]);
			}

			return Hash;
		}

		static inline int32 HashElements(
			const FSchema& Schema,
			const TArrayView<const FName> SchemaElementNames,
			const TArrayView<const FSchemaElement> SchemaElements,
			const int32 Salt)
		{
			// Note: Here we xor all entries together. 
			// This makes the hash in invariant to the ordering of pairs of names and elements 
			// which is actually what we want since these two arrays are representing a map-like 
			// structure and it is fine to pass keys and values in a different order.

			int32 Hash = 0x5b3bbe4d;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaElements.Num(); SchemaElementIdx++)
			{
				Hash ^= HashCombine({ HashFNameStable(SchemaElementNames[SchemaElementIdx]), GetSchemaObjectsCompatibilityHash(Schema, SchemaElements[SchemaElementIdx], Salt) });
			}

			return Hash;
		}
	}

	int32 GetSchemaObjectsCompatibilityHash(
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const int32 Salt)
	{
		check(Schema.IsValid(SchemaElement));
		const EType SchemaElementType = Schema.GetType(SchemaElement);

		const int32 Hash = Private::HashCombine({ Salt, Private::HashInt((int32)SchemaElementType) });

		switch (SchemaElementType)
		{
		case EType::Null: return Hash;

		case EType::Continuous: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetContinuous(SchemaElement).Num) });

		case EType::DiscreteExclusive: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetDiscreteExclusive(SchemaElement).Num) });

		case EType::DiscreteInclusive: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetDiscreteInclusive(SchemaElement).Num) });

		case EType::NamedDiscreteExclusive:
		{
			const FSchemaNamedDiscreteExclusiveParameters Parameters = Schema.GetNamedDiscreteExclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Salt) });
		}

		case EType::NamedDiscreteInclusive:
		{
			const FSchemaNamedDiscreteInclusiveParameters Parameters = Schema.GetNamedDiscreteInclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Salt) });
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashInt(Parameters.Num), GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt) });
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);
			return GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt);
		}

		default:
		{
			checkNoEntry();
			return 0;
		}

		}
	}

	bool AreSchemaObjectsCompatible(
		const FSchema& SchemaA,
		const FSchemaElement SchemaElementA,
		const FSchema& SchemaB,
		const FSchemaElement SchemaElementB)
	{
		check(SchemaA.IsValid(SchemaElementA));
		check(SchemaB.IsValid(SchemaElementB));

		const EType SchemaElementTypeA = SchemaA.GetType(SchemaElementA);
		const EType SchemaElementTypeB = SchemaB.GetType(SchemaElementB);

		// If any element is an encoding element we forward the comparison to the sub-element since encoding elements don't affect compatibility
		if (SchemaElementTypeA == EType::Encoding) { return AreSchemaObjectsCompatible(SchemaA, SchemaA.GetEncoding(SchemaElementA).Element, SchemaB, SchemaElementB); }
		if (SchemaElementTypeB == EType::Encoding) { return AreSchemaObjectsCompatible(SchemaA, SchemaElementA, SchemaB, SchemaB.GetEncoding(SchemaElementB).Element); }

		// Otherwise if types don't match we immediately know elements are incompatible
		if (SchemaElementTypeA != SchemaElementTypeB) { return false; }

		// This is an early-out since if the input sizes are different we are definitely incompatible
		if (SchemaA.GetActionVectorSize(SchemaElementA) != SchemaB.GetActionVectorSize(SchemaElementB)) { return false; }

		switch (SchemaElementTypeA)
		{
		case EType::Null: return true;

		case EType::Continuous: return SchemaA.GetContinuous(SchemaElementA).Num == SchemaB.GetContinuous(SchemaElementB).Num;

		case EType::DiscreteExclusive: return SchemaA.GetDiscreteExclusive(SchemaElementA).Num == SchemaB.GetDiscreteExclusive(SchemaElementB).Num;

		case EType::DiscreteInclusive: return SchemaA.GetDiscreteInclusive(SchemaElementA).Num == SchemaB.GetDiscreteInclusive(SchemaElementB).Num;

		case EType::NamedDiscreteExclusive:
		{
			const FSchemaNamedDiscreteExclusiveParameters ParametersA = SchemaA.GetNamedDiscreteExclusive(SchemaElementA);
			const FSchemaNamedDiscreteExclusiveParameters ParametersB = SchemaB.GetNamedDiscreteExclusive(SchemaElementB);

			if (ParametersA.ElementNames.Num() != ParametersB.ElementNames.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.ElementNames.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
			}

			return true;
		}

		case EType::NamedDiscreteInclusive:
		{
			const FSchemaNamedDiscreteInclusiveParameters ParametersA = SchemaA.GetNamedDiscreteInclusive(SchemaElementA);
			const FSchemaNamedDiscreteInclusiveParameters ParametersB = SchemaB.GetNamedDiscreteInclusive(SchemaElementB);

			if (ParametersA.ElementNames.Num() != ParametersB.ElementNames.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.ElementNames.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
			}

			return true;
		}

		case EType::And:
		{
			const FSchemaAndParameters ParametersA = SchemaA.GetAnd(SchemaElementA);
			const FSchemaAndParameters ParametersB = SchemaB.GetAnd(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters ParametersA = SchemaA.GetOrExclusive(SchemaElementA);
			const FSchemaOrExclusiveParameters ParametersB = SchemaB.GetOrExclusive(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters ParametersA = SchemaA.GetOrInclusive(SchemaElementA);
			const FSchemaOrInclusiveParameters ParametersB = SchemaB.GetOrInclusive(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters ParametersA = SchemaA.GetArray(SchemaElementA);
			const FSchemaArrayParameters ParametersB = SchemaB.GetArray(SchemaElementB);

			return (ParametersA.Num == ParametersB.Num) && AreSchemaObjectsCompatible(SchemaA, ParametersA.Element, SchemaB, ParametersB.Element);
		}

		case EType::Encoding:
		{
			checkf(false, TEXT("Encoding elements should always be forwarded..."));
			return false;
		}

		default:
		{
			checkNoEntry();
			return false;
		}

		}
	}

	void MakeDecoderNetworkModelBuilderElementFromSchema(
		NNE::RuntimeBasic::FModelBuilderElement& OutElement,
		NNE::RuntimeBasic::FModelBuilder& Builder,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings)
	{
		const EType SchemaElementType = Schema.GetType(SchemaElement);

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			OutElement = Builder.MakeCopy(0);
			break;
		}

		case EType::Continuous:
		{
			const int32 ValueNum = Schema.GetContinuous(SchemaElement).Num * 2;

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::DiscreteExclusive:
		{
			const FSchemaDiscreteExclusiveParameters Parameters = Schema.GetDiscreteExclusive(SchemaElement);

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.Num; Idx++)
			{
				// Clamp zero probabilities to the smallest (positive) float. This is approximately equal to a probability of 1:1e38
				LogPriorProbabilities[Idx] = FMath::Loge(FMath::Max(LogPriorProbabilities[Idx], FLT_MIN));
			}

			OutElement = Builder.MakeDenormalize(
				Parameters.Num,
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(Parameters.Num));
			break;
		}

		case EType::DiscreteInclusive:
		{
			const FSchemaDiscreteInclusiveParameters Parameters = Schema.GetDiscreteInclusive(SchemaElement);

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.Num; Idx++)
			{
				LogPriorProbabilities[Idx] = Private::Logit(LogPriorProbabilities[Idx]);
			}

			OutElement = Builder.MakeDenormalize(
				Parameters.Num,
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(Parameters.Num));
			break;
		}

		case EType::NamedDiscreteExclusive:
		{
			const FSchemaNamedDiscreteExclusiveParameters Parameters = Schema.GetNamedDiscreteExclusive(SchemaElement);

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.ElementNames.Num(); Idx++)
			{
				// Clamp zero probabilities to the smallest (positive) float. This is approximately equal to a probability of 1:1e38
				LogPriorProbabilities[Idx] = FMath::Loge(FMath::Max(LogPriorProbabilities[Idx], FLT_MIN));
			}

			OutElement = Builder.MakeDenormalize(
				Parameters.ElementNames.Num(),
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(Parameters.ElementNames.Num()));
			break;
		}

		case EType::NamedDiscreteInclusive:
		{
			const FSchemaNamedDiscreteInclusiveParameters Parameters = Schema.GetNamedDiscreteInclusive(SchemaElement);

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.ElementNames.Num(); Idx++)
			{
				LogPriorProbabilities[Idx] = Private::Logit(LogPriorProbabilities[Idx]);
			}

			OutElement = Builder.MakeDenormalize(
				Parameters.ElementNames.Num(),
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(Parameters.ElementNames.Num()));
			break;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
			BuilderLayers.Reserve(Parameters.Elements.Num());
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeDecoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderLayers.Emplace(BuilderSubElement);
			}

			OutElement = Builder.MakeConcat(BuilderLayers);
			break;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
			BuilderLayers.Reserve(Parameters.Elements.Num() + 1);
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeDecoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderLayers.Emplace(BuilderSubElement);
			}

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.PriorProbabilities.Num(); Idx++)
			{
				// Clamp zero probabilities to the smallest (positive) float. This is approximately equal to a probability of 1:1e38
				LogPriorProbabilities[Idx] = FMath::Loge(FMath::Max(LogPriorProbabilities[Idx], FLT_MIN));
			}

			BuilderLayers.Emplace(Builder.MakeDenormalize(
				LogPriorProbabilities.Num(),
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(LogPriorProbabilities.Num())));

			OutElement = Builder.MakeConcat(BuilderLayers);
			break;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
			BuilderLayers.Reserve(Parameters.Elements.Num() + 1);
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeDecoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderLayers.Emplace(BuilderSubElement);
			}

			TArray<float, TInlineAllocator<16>> LogPriorProbabilities;
			LogPriorProbabilities.Append(Parameters.PriorProbabilities);
			for (int32 Idx = 0; Idx < Parameters.PriorProbabilities.Num(); Idx++)
			{
				LogPriorProbabilities[Idx] = Private::Logit(LogPriorProbabilities[Idx]);
			}

			BuilderLayers.Emplace(Builder.MakeDenormalize(
				LogPriorProbabilities.Num(),
				Builder.MakeValuesCopy(LogPriorProbabilities),
				Builder.MakeValuesOne(LogPriorProbabilities.Num())));

			OutElement = Builder.MakeConcat(BuilderLayers);
			break;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeDecoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);
			OutElement = Builder.MakeArray(Parameters.Num, BuilderSubElement);
			break;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);

			const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(Parameters.Element);

			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeDecoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);

			NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
			LinearLayerSettings.Type = NetworkSettings.bUseCompressedLinearLayers ?
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

			switch (NetworkSettings.WeightInitialization)
			{
			case EWeightInitialization::KaimingGaussian: LinearLayerSettings.WeightInitializationSettings.Type = 
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
			case EWeightInitialization::KaimingUniform: LinearLayerSettings.WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
			default: checkNoEntry();
			}

			OutElement = Builder.MakeSequence({
				Builder.MakeActivation(Parameters.EncodingSize, Private::GetNNEActivationFunction(Parameters.ActivationFunction)),
				Builder.MakeMLP(
					Parameters.EncodingSize,
					SubElementEncodedSize,
					Parameters.EncodingSize,
					Parameters.LayerNum + 1,  // Add 1 to account for input layer 
					Private::GetNNEActivationFunction(Parameters.ActivationFunction),
					false,
					LinearLayerSettings),
				BuilderSubElement,
				});

			break;
		}

		default:
		{
			checkNoEntry();
		}
		}

		checkf(OutElement.GetInputSize() == Schema.GetEncodedVectorSize(SchemaElement),
			TEXT("Decoder Network Input unexpected size. Got %i, expected %i according to Schema."),
			OutElement.GetInputSize(), Schema.GetEncodedVectorSize(SchemaElement));

		checkf(OutElement.GetOutputSize() == Schema.GetActionDistributionVectorSize(SchemaElement),
			TEXT("Decoder Network Output unexpected size. Got %i, expected %i according to Schema."),
			OutElement.GetOutputSize(), Schema.GetActionDistributionVectorSize(SchemaElement));
	}

	void GenerateDecoderNetworkFileDataFromSchema(
		TArray<uint8>& OutFileData,
		uint32& OutInputSize,
		uint32& OutOutputSize,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings,
		const uint32 Seed)
	{
		check(Schema.IsValid(SchemaElement));

		NNE::RuntimeBasic::FModelBuilder Builder(Seed);
		NNE::RuntimeBasic::FModelBuilderElement Element;
		MakeDecoderNetworkModelBuilderElementFromSchema(Element, Builder, Schema, SchemaElement, NetworkSettings);
		Builder.WriteFileDataAndReset(OutFileData, OutInputSize, OutOutputSize, Element);
	}

	void SampleVectorFromDistributionVector(
		uint32& InOutRandomState,
		TLearningArrayView<1, float> OutActionVector,
		const TLearningArrayView<1, const float> ActionDistributionVector,
		const TLearningArrayView<1, const float> ActionModifierVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const float ActionNoiseScale)
	{
		check(Schema.IsValid(SchemaElement));

		const EType SchemaElementType = Schema.GetType(SchemaElement);

		switch (SchemaElementType)
		{
		case EType::Null: break;

		case EType::Continuous:
		{
			const int32 ValueNum = Schema.GetContinuous(SchemaElement).Num;
			check(ValueNum == OutActionVector.Num());
			check(ValueNum * 2 == ActionDistributionVector.Num());
			check(1 + ValueNum * 2 == ActionModifierVector.Num());

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ValueNum });
				for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
				{
					Masked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				}

				Random::SampleDistributionIndependantNormalMasked(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector.Slice(0, ValueNum),
					ActionDistributionVector.Slice(ValueNum, ValueNum),
					Masked,
					ActionModifierVector.Slice(1 + ValueNum, ValueNum),
					ActionNoiseScale);
			}
			else
			{
				Random::SampleDistributionIndependantNormal(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector.Slice(0, ValueNum),
					ActionDistributionVector.Slice(ValueNum, ValueNum),
					ActionNoiseScale);
			}

			break;
		}

		case EType::DiscreteExclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteExclusive(SchemaElement).Num;
			check(ValueNum == OutActionVector.Num());
			check(ValueNum == ActionDistributionVector.Num());
			check(1 + ValueNum == ActionModifierVector.Num());

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ValueNum });
				for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
				{
					Masked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				}
				check(Private::CheckExclusiveMaskValid(Masked));

				Random::SampleDistributionMultinoulliMasked(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					Masked,
					ActionNoiseScale);
			}
			else
			{
				Random::SampleDistributionMultinoulli(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					ActionNoiseScale);
			}

			break;
		}

		case EType::DiscreteInclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteInclusive(SchemaElement).Num;
			check(ValueNum == OutActionVector.Num());
			check(ValueNum == ActionDistributionVector.Num());
			check(1 + ValueNum == ActionModifierVector.Num());

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ValueNum });
				for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
				{
					Masked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				}

				Random::SampleDistributionBernoulliMasked(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					Masked,
					ActionNoiseScale);
			}
			else
			{
				Random::SampleDistributionBernoulli(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					ActionNoiseScale);
			}

			break;
		}

		case EType::NamedDiscreteExclusive:
		{
			const int32 ValueNum = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num();
			check(ValueNum == OutActionVector.Num());
			check(ValueNum == ActionDistributionVector.Num());
			check(1 + ValueNum == ActionModifierVector.Num());

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ValueNum });
				for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
				{
					Masked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				}
				check(Private::CheckExclusiveMaskValid(Masked));

				Random::SampleDistributionMultinoulliMasked(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					Masked,
					ActionNoiseScale);
			}
			else
			{
				Random::SampleDistributionMultinoulli(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					ActionNoiseScale);
			}

			break;
		}

		case EType::NamedDiscreteInclusive:
		{
			const int32 ValueNum = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num();
			check(ValueNum == OutActionVector.Num());
			check(ValueNum == ActionDistributionVector.Num());
			check(1 + ValueNum == ActionModifierVector.Num());

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ValueNum });
				for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
				{
					Masked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				}

				Random::SampleDistributionBernoulliMasked(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					Masked,
					ActionNoiseScale);
			}
			else
			{
				Random::SampleDistributionBernoulli(
					OutActionVector,
					InOutRandomState,
					ActionDistributionVector,
					ActionNoiseScale);
			}

			break;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			int32 SubElementActionVectorOffset = 0;
			int32 SubElementActionDistributionVectorOffset = 0;
			int32 SubElementActionModifierVectorOffset = 1;

			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				const int32 SubElementActionVectorSize = Schema.GetActionVectorSize(SubElement);
				const int32 SubElementActionDistributionVectorSize = Schema.GetActionDistributionVectorSize(SubElement);
				const int32 SubElementActionModifierVectorSize = Schema.GetActionModifierVectorSize(SubElement);

				SampleVectorFromDistributionVector(
					InOutRandomState,
					OutActionVector.Slice(SubElementActionVectorOffset, SubElementActionVectorSize),
					ActionDistributionVector.Slice(SubElementActionDistributionVectorOffset, SubElementActionDistributionVectorSize),
					ActionModifierVector.Slice(SubElementActionModifierVectorOffset, SubElementActionModifierVectorSize),
					Schema,
					SubElement,
					ActionNoiseScale);

				SubElementActionVectorOffset += SubElementActionVectorSize;
				SubElementActionDistributionVectorOffset += SubElementActionDistributionVectorSize;
				SubElementActionModifierVectorOffset += SubElementActionModifierVectorSize;
			}

			check(SubElementActionVectorOffset == OutActionVector.Num());
			check(SubElementActionDistributionVectorOffset == ActionDistributionVector.Num());
			check(SubElementActionModifierVectorOffset == ActionModifierVector.Num());

			break;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

			const int32 SubElementActionVectorMax = Private::GetMaxActionVectorSize(Schema, Parameters.Elements);
			const int32 SubElementActionDistributionVectorTotal = Private::GetTotalActionDistributionVectorSize(Schema, Parameters.Elements);
			const int32 SubElementActionModifierVectorTotal = Private::GetTotalActionModifierVectorSize(Schema, Parameters.Elements);
			const int32 ElementNum = Parameters.Elements.Num();

			check(SubElementActionVectorMax + ElementNum == OutActionVector.Num());
			check(SubElementActionDistributionVectorTotal + ElementNum == ActionDistributionVector.Num());
			check(1 + ElementNum + SubElementActionModifierVectorTotal == ActionModifierVector.Num());

			// Zero main part of vector
			Array::Zero(OutActionVector.Slice(0, SubElementActionVectorMax));

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ElementNum });
				for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
				{
					Masked[ElementIdx] = ActionModifierVector[1 + ElementIdx] == 1.0f;
				}
				check(Private::CheckExclusiveMaskValid(Masked));

				// Sample which sub-element to generate
				Random::SampleDistributionMultinoulliMasked(
					OutActionVector.Slice(SubElementActionVectorMax, ElementNum),
					InOutRandomState,
					ActionDistributionVector.Slice(SubElementActionDistributionVectorTotal, ElementNum),
					Masked,
					ActionNoiseScale);
			}
			else
			{
				// Sample which sub-element to generate
				Random::SampleDistributionMultinoulli(
					OutActionVector.Slice(SubElementActionVectorMax, ElementNum),
					InOutRandomState,
					ActionDistributionVector.Slice(SubElementActionDistributionVectorTotal, ElementNum),
					ActionNoiseScale);
			}

			int32 SubElementsSampled = 0;
			int32 SubElementActionDistributionVectorOffset = 0;
			int32 SubElementActionModifierVectorOffset = 1 + ElementNum;

			for (int32 SubElementIdx = 0; SubElementIdx < ElementNum; SubElementIdx++)
			{
				const FSchemaElement SubElement = Parameters.Elements[SubElementIdx];
				const int32 SubElementActionVectorSize = Schema.GetActionVectorSize(SubElement);
				const int32 SubElementActionDistributionVectorSize = Schema.GetActionDistributionVectorSize(SubElement);
				const int32 SubElementActionModifierVectorSize = Schema.GetActionModifierVectorSize(SubElement);

				check(SubElementActionVectorSize <= SubElementActionVectorMax);

				if (OutActionVector[SubElementActionVectorMax + SubElementIdx])
				{
					// Sample Sub-Element
					SampleVectorFromDistributionVector(
						InOutRandomState,
						OutActionVector.Slice(0, SubElementActionVectorSize),
						ActionDistributionVector.Slice(SubElementActionDistributionVectorOffset, SubElementActionDistributionVectorSize),
						ActionModifierVector.Slice(SubElementActionModifierVectorOffset, SubElementActionModifierVectorSize),
						Schema,
						SubElement,
						ActionNoiseScale);

					SubElementsSampled++;
				}

				SubElementActionDistributionVectorOffset += SubElementActionDistributionVectorSize;
				SubElementActionModifierVectorOffset += SubElementActionModifierVectorSize;
			}

			check(SubElementsSampled == 1); // Exactly one sub-element should have been sampled
			check(SubElementActionDistributionVectorOffset == SubElementActionDistributionVectorTotal);
			check(SubElementActionModifierVectorOffset == 1 + ElementNum + SubElementActionModifierVectorTotal);

			break;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

			const int32 SubElementActionVectorTotal = Private::GetTotalActionVectorSize(Schema, Parameters.Elements);
			const int32 SubElementActionDistributionVectorTotal = Private::GetTotalActionDistributionVectorSize(Schema, Parameters.Elements);
			const int32 SubElementActionModifierVectorTotal = Private::GetTotalActionModifierVectorSize(Schema, Parameters.Elements);
			const int32 ElementNum = Parameters.Elements.Num();

			check(SubElementActionVectorTotal + ElementNum == OutActionVector.Num());
			check(SubElementActionDistributionVectorTotal + ElementNum == ActionDistributionVector.Num());
			check(1 + ElementNum + SubElementActionModifierVectorTotal == ActionModifierVector.Num());

			// Zero main part of vector
			Array::Zero(OutActionVector.Slice(0, SubElementActionVectorTotal));

			if (ActionModifierVector[0])
			{
				TLearningArray<1, bool, TInlineAllocator<32>> Masked;
				Masked.SetNumUninitialized({ ElementNum });
				for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
				{
					Masked[ElementIdx] = ActionModifierVector[1 + ElementIdx] == 1.0f;
				}

				// Sample which sub-elements to generate
				Random::SampleDistributionBernoulliMasked(
					OutActionVector.Slice(SubElementActionVectorTotal, ElementNum),
					InOutRandomState,
					ActionDistributionVector.Slice(SubElementActionDistributionVectorTotal, ElementNum),
					Masked,
					ActionNoiseScale);
			}
			else
			{
				// Sample which sub-elements to generate
				Random::SampleDistributionBernoulli(
					OutActionVector.Slice(SubElementActionVectorTotal, ElementNum),
					InOutRandomState,
					ActionDistributionVector.Slice(SubElementActionDistributionVectorTotal, ElementNum),
					ActionNoiseScale);
			}

			int32 SubElementActionVectorOffset = 0;
			int32 SubElementActionDistributionVectorOffset = 0;
			int32 SubElementActionModifierVectorOffset = 1 + ElementNum;

			for (int32 SubElementIdx = 0; SubElementIdx < ElementNum; SubElementIdx++)
			{
				const FSchemaElement SubElement = Parameters.Elements[SubElementIdx];
				const int32 SubElementActionVectorSize = Schema.GetActionVectorSize(SubElement);
				const int32 SubElementActionDistributionVectorSize = Schema.GetActionDistributionVectorSize(SubElement);
				const int32 SubElementActionModifierVectorSize = Schema.GetActionModifierVectorSize(SubElement);

				if (OutActionVector[SubElementActionVectorTotal + SubElementIdx])
				{
					// Sample sub-elements
					SampleVectorFromDistributionVector(
						InOutRandomState,
						OutActionVector.Slice(SubElementActionVectorOffset, SubElementActionVectorSize),
						ActionDistributionVector.Slice(SubElementActionDistributionVectorOffset, SubElementActionDistributionVectorSize),
						ActionModifierVector.Slice(SubElementActionModifierVectorOffset, SubElementActionModifierVectorSize),
						Schema,
						SubElement,
						ActionNoiseScale);
				}

				SubElementActionVectorOffset += SubElementActionVectorSize;
				SubElementActionDistributionVectorOffset += SubElementActionDistributionVectorSize;
				SubElementActionModifierVectorOffset += SubElementActionModifierVectorSize;
			}

			check(SubElementActionVectorOffset == SubElementActionVectorTotal);
			check(SubElementActionDistributionVectorOffset == SubElementActionDistributionVectorTotal);
			check(SubElementActionModifierVectorOffset == 1 + ElementNum + SubElementActionModifierVectorTotal);

			break;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			const int32 SubElementActionVectorSize = Schema.GetActionVectorSize(Parameters.Element);
			const int32 SubElementActionDistributionVectorSize = Schema.GetActionDistributionVectorSize(Parameters.Element);
			const int32 SubElementActionModifierVectorSize = Schema.GetActionModifierVectorSize(Parameters.Element);

			check(SubElementActionVectorSize * Parameters.Num == OutActionVector.Num());
			check(SubElementActionDistributionVectorSize * Parameters.Num == ActionDistributionVector.Num());
			check(1 + SubElementActionModifierVectorSize * Parameters.Num == ActionModifierVector.Num());

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				SampleVectorFromDistributionVector(
					InOutRandomState,
					OutActionVector.Slice(ElementIdx * SubElementActionVectorSize, SubElementActionVectorSize),
					ActionDistributionVector.Slice(ElementIdx * SubElementActionDistributionVectorSize, SubElementActionDistributionVectorSize),
					ActionModifierVector.Slice(1 + ElementIdx * SubElementActionModifierVectorSize, SubElementActionModifierVectorSize),
					Schema,
					Parameters.Element,
					ActionNoiseScale);
			}

			break;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);
			const int32 SubElementActionModifierVectorSize = Schema.GetActionModifierVectorSize(Parameters.Element);

			SampleVectorFromDistributionVector(
				InOutRandomState,
				OutActionVector,
				ActionDistributionVector,
				ActionModifierVector.Slice(1, SubElementActionModifierVectorSize),
				Schema,
				Parameters.Element,
				ActionNoiseScale);

			break;
		}

		default:
		{
			checkNoEntry();
			return;
		}

		}
	}
		
	void SetVectorFromObject(
		TLearningArrayView<1, float> OutActionVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FObject& Object,
		const FObjectElement ObjectElement)
	{
		check(Schema.IsValid(SchemaElement));
		check(Object.IsValid(ObjectElement));
		check(OutActionVector.Num() == Schema.GetActionVectorSize(SchemaElement));

		// Check that the types match

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const EType ObjectElementType = Object.GetType(ObjectElement);
		check(ObjectElementType == SchemaElementType);

		// Zero Action Vector

		Array::Zero(OutActionVector);

		// Logic for each specific element type

		switch (SchemaElementType)
		{

		case EType::Null: return;

		case EType::Continuous:
		{
			// Check the input sizes match

			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);

			TArrayView<const float> ActionValues = Object.GetContinuous(ObjectElement).Values;
			check(Schema.GetActionVectorSize(SchemaElement) == ActionValues.Num());
			check(Schema.GetActionVectorSize(SchemaElement) == OutActionVector.Num());
			check(Schema.GetActionVectorSize(SchemaElement) == SchemaParameters.Num);

			// Copy in and scale the values from the action object

			const int32 ValueNum = SchemaParameters.Num;

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				float Value = ActionValues[ValueIdx];				
				if (SchemaParameters.NormalizationOption == ENormalization::Manual)
				{
					Value -= SchemaParameters.Offsets[ValueIdx];
					const float StdDev = FMath::Max(SchemaParameters.Scales[ValueIdx], UE_SMALL_NUMBER);
					Value /= StdDev;
				}
				OutActionVector[ValueIdx] = Value;
			}

			return;
		}

		case EType::DiscreteExclusive:
		{
			const int32 ActionValue = Object.GetDiscreteExclusive(ObjectElement).DiscreteIndex;
			check(Schema.GetActionVectorSize(SchemaElement) > ActionValue && ActionValue >= 0);
			check(Schema.GetActionVectorSize(SchemaElement) == OutActionVector.Num());

			// Set the single value in the action vector

			OutActionVector[ActionValue] = 1.0f;
			return;
		}

		case EType::DiscreteInclusive:
		{
			const TArrayView<const int32> ActionValues = Object.GetDiscreteInclusive(ObjectElement).DiscreteIndices;
			check(Schema.GetActionVectorSize(SchemaElement) >= ActionValues.Num());
			check(Schema.GetActionVectorSize(SchemaElement) == OutActionVector.Num());

			// Set values in the action vector

			for (int32 ActionValueIdx = 0; ActionValueIdx < ActionValues.Num(); ActionValueIdx++)
			{
				check(Schema.GetActionVectorSize(SchemaElement) > ActionValues[ActionValueIdx] && ActionValues[ActionValueIdx] >= 0);
				OutActionVector[ActionValues[ActionValueIdx]] = 1.0f;
			}

			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			const FName ActionValue = Object.GetNamedDiscreteExclusive(ObjectElement).ElementName;
			check(Schema.GetActionVectorSize(SchemaElement) == OutActionVector.Num());

			// Set the single value in the action vector
			const int32 ActionIndex = SchemaNames.Find(ActionValue);
			check(ActionIndex != INDEX_NONE);
			OutActionVector[ActionIndex] = 1.0f;
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			const TArrayView<const FName> ActionValues = Object.GetNamedDiscreteInclusive(ObjectElement).ElementNames;
			check(Schema.GetActionVectorSize(SchemaElement) >= ActionValues.Num());
			check(Schema.GetActionVectorSize(SchemaElement) == OutActionVector.Num());

			// Set values in the action vector
			for (int32 ActionValueIdx = 0; ActionValueIdx < ActionValues.Num(); ActionValueIdx++)
			{
				const int32 ActionIndex = SchemaNames.Find(ActionValues[ActionValueIdx]);
				check(ActionIndex != INDEX_NONE);
				OutActionVector[ActionIndex] = 1.0f;
			}
			return;
		}

		case EType::And:
		{
			// Check the number of sub-elements match

			const FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const FObjectAndParameters ObjectParameters = Object.GetAnd(ObjectElement);
			check(SchemaParameters.Elements.Num() == ObjectParameters.Elements.Num());

			// Set the Sub-elements

			int32 SubElementOffset = 0;

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 ObjectElementIndex = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);
				check(ObjectElementIndex != INDEX_NONE);

				const int32 SubElementSize = Schema.GetActionVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				SetVectorFromObject(
					OutActionVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIndex]);

				SubElementOffset += SubElementSize;
			}

			check(SubElementOffset == OutActionVector.Num());
			return;
		}

		case EType::OrExclusive:
		{
			// Check only one sub-element is given and index is valid

			const FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const FObjectOrExclusiveParameters ObjectParameters = Object.GetOrExclusive(ObjectElement);

			const int32 SchemaElementIndex = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);
			check(SchemaElementIndex != INDEX_NONE);

			// Set the sub-element

			const int32 SubElementSize = Schema.GetActionVectorSize(SchemaParameters.Elements[SchemaElementIndex]);

			SetVectorFromObject(
				OutActionVector.Slice(0, SubElementSize),
				Schema,
				SchemaParameters.Elements[SchemaElementIndex],
				Object,
				ObjectParameters.Element);

			// Set Mask

			const int32 MaxSubElementSize = Private::GetMaxActionVectorSize(Schema, SchemaParameters.Elements);

			OutActionVector[MaxSubElementSize + SchemaElementIndex] = 1.0f;

			check(OutActionVector.Num() == MaxSubElementSize + SchemaParameters.Elements.Num());
			return;
		}

		case EType::OrInclusive:
		{
			// Check all indices are in range

			const FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const FObjectOrInclusiveParameters ObjectParameters = Object.GetOrInclusive(ObjectElement);
			check(ObjectParameters.Elements.Num() <= SchemaParameters.Elements.Num());

			// Update sub-elements

			int32 SubElementOffset = 0;

			for (int32 ObjectElementIdx = 0; ObjectElementIdx < ObjectParameters.Elements.Num(); ObjectElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectElementIdx]);
				check(SchemaElementIdx != INDEX_NONE);

				const int32 SubElementSize = Schema.GetActionVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				SetVectorFromObject(
					OutActionVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIdx]);

				SubElementOffset += SubElementSize;
			}

			// Set Mask

			check(SubElementOffset + SchemaParameters.Elements.Num() == OutActionVector.Num());

			for (int32 ObjectElementIdx = 0; ObjectElementIdx < ObjectParameters.Elements.Num(); ObjectElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectElementIdx]);
				check(SchemaElementIdx != INDEX_NONE);

				OutActionVector[SubElementOffset + SchemaElementIdx] = 1.0f;
			}

			return;
		}

		case EType::Array:
		{
			// Check number of array elements is correct

			const FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const FObjectArrayParameters ObjectParameters = Object.GetArray(ObjectElement);
			check(SchemaParameters.Num == ObjectParameters.Elements.Num());

			// Update sub-elements

			const int32 SubElementSize = Schema.GetActionVectorSize(SchemaParameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < SchemaParameters.Num; ElementIdx++)
			{
				SetVectorFromObject(
					OutActionVector.Slice(ElementIdx * SubElementSize, SubElementSize),
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]);
			}

			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const FObjectEncodingParameters ObjectParameters = Object.GetEncoding(ObjectElement);

			SetVectorFromObject(
				OutActionVector,
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);

			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}

		}
	}

	void GetObjectFromVector(
		FObject& OutObject,
		FObjectElement& OutObjectElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ActionVector)
	{
		check(Schema.IsValid(SchemaElement));

		// Check that the types match

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const FName SchemaElementTag = Schema.GetTag(SchemaElement);

		// Get Action Vector Size

		const int32 ActionVectorSize = ActionVector.Num();
		check(ActionVectorSize == Schema.GetActionVectorSize(SchemaElement));

		// Logic for each specific element type

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			OutObjectElement = OutObject.CreateNull(SchemaElementTag);
			return;
		}

		case EType::Continuous:
		{
			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);
			check(ActionVectorSize == SchemaParameters.Num);

			const int32 ValueNum = SchemaParameters.Num;

			TLearningArray<1, float, TInlineAllocator<32>> ActionValues;
			ActionValues.SetNumUninitialized({ ValueNum });

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				float Value = ActionVector[ValueIdx];
				
				if (SchemaParameters.NormalizationOption == ENormalization::Manual)
				{
					const float StdDev = FMath::Max(SchemaParameters.Scales[ValueIdx], UE_SMALL_NUMBER);
					Value *= StdDev;
					Value += SchemaParameters.Offsets[ValueIdx];
				}
				ActionValues[ValueIdx] = Value;
			}


			OutObjectElement = OutObject.CreateContinuous({ MakeArrayView(ActionValues.GetData(), ActionValues.Num()) }, SchemaElementTag);
			return;
		}

		case EType::DiscreteExclusive:
		{
			check(ActionVectorSize == Schema.GetDiscreteExclusive(SchemaElement).Num);

			// Find Index
			int32 ExclusiveIndex = INDEX_NONE;
			for (int32 Idx = 0; Idx < ActionVectorSize; Idx++)
			{
				check(ActionVector[Idx] == 0.0f || ActionVector[Idx] == 1.0f);
				if (ActionVector[Idx])
				{
					ExclusiveIndex = Idx;
					break;
				}
			}
			check(ExclusiveIndex != INDEX_NONE);

			OutObjectElement = OutObject.CreateDiscreteExclusive({ ExclusiveIndex }, SchemaElementTag);
			return;
		}

		case EType::DiscreteInclusive:
		{
			check(ActionVectorSize == Schema.GetDiscreteInclusive(SchemaElement).Num);

			// Find Indices
			TArray<int32, TInlineAllocator<8>> InclusiveIndices;
			InclusiveIndices.Reserve(ActionVectorSize);
			for (int32 Idx = 0; Idx < ActionVectorSize; Idx++)
			{
				check(ActionVector[Idx] == 0.0f || ActionVector[Idx] == 1.0f);
				if (ActionVector[Idx])
				{
					InclusiveIndices.Add(Idx);
				}
			}

			OutObjectElement = OutObject.CreateDiscreteInclusive({ InclusiveIndices }, SchemaElementTag);
			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			check(ActionVectorSize == Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num());

			// Find Name
			FName ExclusiveName = NAME_None;
			for (int32 Idx = 0; Idx < ActionVectorSize; Idx++)
			{
				check(ActionVector[Idx] == 0.0f || ActionVector[Idx] == 1.0f);
				if (ActionVector[Idx])
				{
					ExclusiveName = SchemaNames[Idx];
					break;
				}
			}
			check(ExclusiveName != NAME_None);

			OutObjectElement = OutObject.CreateNamedDiscreteExclusive({ ExclusiveName }, SchemaElementTag);
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			check(ActionVectorSize == Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num());

			// Find Names
			TArray<FName, TInlineAllocator<8>> InclusiveNames;
			InclusiveNames.Reserve(ActionVectorSize);
			for (int32 Idx = 0; Idx < ActionVectorSize; Idx++)
			{
				check(ActionVector[Idx] == 0.0f || ActionVector[Idx] == 1.0f);
				if (ActionVector[Idx])
				{
					InclusiveNames.Add(SchemaNames[Idx]);
				}
			}

			OutObjectElement = OutObject.CreateNamedDiscreteInclusive({ InclusiveNames }, SchemaElementTag);
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);
			
			// Create Sub-elements

			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Elements.Num());

			int32 SubElementOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetObjectFromVector(
					OutObject,
					SubElements[SchemaElementIdx],
					Schema,
					Parameters.Elements[SchemaElementIdx],
					ActionVector.Slice(SubElementOffset, SubElementSize));

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ActionVectorSize);

			OutObjectElement = OutObject.CreateAnd({ Parameters.ElementNames, SubElements }, SchemaElementTag);
			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

			// Find active element

			const int32 MaxSubElementSize = Private::GetMaxActionVectorSize(Schema, Parameters.Elements);

			int32 SchemaElementIndex = INDEX_NONE;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				check(ActionVector[MaxSubElementSize + SubElementIdx] == 0.0f || ActionVector[MaxSubElementSize + SubElementIdx] == 1.0f);
				if (ActionVector[MaxSubElementSize + SubElementIdx])
				{
					SchemaElementIndex = SubElementIdx;
					break;
				}
			}
			check(SchemaElementIndex != INDEX_NONE);

			// Create sub-element

			const int32 SubElementSize = Schema.GetActionVectorSize(Parameters.Elements[SchemaElementIndex]);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Elements[SchemaElementIndex],
				ActionVector.Slice(0, SubElementSize));

			OutObjectElement = OutObject.CreateOrExclusive({ Parameters.ElementNames[SchemaElementIndex], SubElement }, SchemaElementTag);
			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

			// Find total sub-element size

			const int32 TotalSubElementSize = Private::GetTotalActionVectorSize(Schema, Parameters.Elements);

			// Create sub-elements

			TArray<FName, TInlineAllocator<8>> SubElementNames;
			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElementNames.Reserve(Parameters.Elements.Num());
			SubElements.Reserve(Parameters.Elements.Num());

			int32 SubElementOffset = 0;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionVectorSize(Parameters.Elements[SubElementIdx]);

				check(ActionVector[TotalSubElementSize + SubElementIdx] == 0.0f || ActionVector[TotalSubElementSize + SubElementIdx] == 1.0f);
				if (ActionVector[TotalSubElementSize + SubElementIdx])
				{
					FObjectElement SubElement;
					GetObjectFromVector(
						OutObject,
						SubElement,
						Schema,
						Parameters.Elements[SubElementIdx],
						ActionVector.Slice(SubElementOffset, SubElementSize));

					SubElementNames.Add(Parameters.ElementNames[SubElementIdx]);
					SubElements.Add(SubElement);
				}

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset + Parameters.Elements.Num() == ActionVectorSize);

			OutObjectElement = OutObject.CreateOrInclusive({ SubElementNames, SubElements }, SchemaElementTag);
			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Num);

			// Create sub-elements

			const int32 SubElementSize = Schema.GetActionVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				GetObjectFromVector(
					OutObject,
					SubElements[ElementIdx],
					Schema,
					Parameters.Element,
					ActionVector.Slice(ElementIdx * SubElementSize, SubElementSize));
			}

			OutObjectElement = OutObject.CreateArray({ SubElements }, SchemaElementTag);
			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Element,
				ActionVector);

			OutObjectElement = OutObject.CreateEncoding({ SubElement }, SchemaElementTag);
			return;
		}

		default:
		{
			checkNoEntry();
			OutObjectElement = FObjectElement();
			return;
		}
		}
	}

	void SetVectorFromModifier(
		TLearningArrayView<1, float> OutActionModifierVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FModifier& Modifier,
		const FModifierElement ModifierElement)
	{
		check(Schema.IsValid(SchemaElement));
		check(Modifier.IsValid(ModifierElement));
		check(OutActionModifierVector.Num() == Schema.GetActionModifierVectorSize(SchemaElement));

		// Check that the types match

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const EType ModifierElementType = Modifier.GetType(ModifierElement);
		check(ModifierElementType == EType::Null || ModifierElementType == SchemaElementType);

		// Zero Action Modifier Vector and return early if we have a null Type

		Array::Zero(OutActionModifierVector);

		if (ModifierElementType == EType::Null)
		{
			return;
		}

		// Indicate we have a modifier by setting the first element in the vector to 1.0f

		OutActionModifierVector[0] = 1.0f;

		// Logic for each specific modifier type

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			// This should never be reached
			checkNoEntry();
			break;
		}

		case EType::Continuous:
		{
			// Check the input sizes match

			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);
			const int32 ValueNum = SchemaParameters.Num;

			const TArrayView<const bool> Masked = Modifier.GetContinuous(ModifierElement).Masked;
			const TArrayView<const float> MaskedValues = Modifier.GetContinuous(ModifierElement).MaskedValues;
			check(Masked.Num() == ValueNum);
			check(MaskedValues.Num() == ValueNum);
			check(Schema.GetActionModifierVectorSize(SchemaElement) == 1 + Masked.Num() + MaskedValues.Num());

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				OutActionModifierVector[1 + ValueIdx] = Masked[ValueIdx] ? 1.0f : 0.0f;
				OutActionModifierVector[1 + ValueNum + ValueIdx] = MaskedValues[ValueIdx];
			}

			return;
		}

		case EType::DiscreteExclusive:
		{
			const TArrayView<const int32> MaskIndices = Modifier.GetDiscreteExclusive(ModifierElement).MaskedIndices;
			check(Schema.GetDiscreteExclusive(SchemaElement).Num >= MaskIndices.Num());

			for (int32 MaskIndicesIdx = 0; MaskIndicesIdx < MaskIndices.Num(); MaskIndicesIdx++)
			{
				check(Schema.GetDiscreteExclusive(SchemaElement).Num > MaskIndices[MaskIndicesIdx] && MaskIndices[MaskIndicesIdx] >= 0);
				OutActionModifierVector[1 + MaskIndices[MaskIndicesIdx]] = 1.0f;
			}

			return;
		}

		case EType::DiscreteInclusive:
		{
			const TArrayView<const int32> MaskIndices = Modifier.GetDiscreteInclusive(ModifierElement).MaskedIndices;
			check(Schema.GetDiscreteInclusive(SchemaElement).Num >= MaskIndices.Num());

			for (int32 MaskIndicesIdx = 0; MaskIndicesIdx < MaskIndices.Num(); MaskIndicesIdx++)
			{
				check(Schema.GetDiscreteInclusive(SchemaElement).Num > MaskIndices[MaskIndicesIdx] && MaskIndices[MaskIndicesIdx] >= 0);
				OutActionModifierVector[1 + MaskIndices[MaskIndicesIdx]] = 1.0f;
			}

			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> MaskNames = Modifier.GetNamedDiscreteExclusive(ModifierElement).MaskedElementNames;
			check(Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num() >= MaskNames.Num());

			for (int32 MaskNameIdx = 0; MaskNameIdx < MaskNames.Num(); MaskNameIdx++)
			{
				const int32 MaskIdx = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Find(MaskNames[MaskNameIdx]);
				check(MaskIdx != INDEX_NONE);
				OutActionModifierVector[1 + MaskIdx] = 1.0f;
			}

			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> MaskNames = Modifier.GetNamedDiscreteInclusive(ModifierElement).MaskedElementNames;
			check(Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num() >= MaskNames.Num());

			for (int32 MaskNameIdx = 0; MaskNameIdx < MaskNames.Num(); MaskNameIdx++)
			{
				const int32 MaskIdx = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Find(MaskNames[MaskNameIdx]);
				check(MaskIdx != INDEX_NONE);
				OutActionModifierVector[1 + MaskIdx] = 1.0f;
			}

			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const FModifierAndParameters ModifierParameters = Modifier.GetAnd(ModifierElement);

			check(OutActionModifierVector.Num() == 1 + Private::GetTotalActionModifierVectorSize(Schema, SchemaParameters.Elements));

			// Set the Sub-elements

			int32 SubElementOffset = 1;

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				const int32 ModifierElementIndex = ModifierParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);
				if (ModifierElementIndex != INDEX_NONE)
				{
					SetVectorFromModifier(
						OutActionModifierVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						SchemaParameters.Elements[SchemaElementIdx],
						Modifier,
						ModifierParameters.Elements[ModifierElementIndex]);
				}

				SubElementOffset += SubElementSize;
			}

			check(SubElementOffset == OutActionModifierVector.Num());
			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const FModifierOrExclusiveParameters ModifierParameters = Modifier.GetOrExclusive(ModifierElement);

			check(OutActionModifierVector.Num() == 1 + SchemaParameters.Elements.Num() + Private::GetTotalActionModifierVectorSize(Schema, SchemaParameters.Elements));

			// Set the Mask

			for (int32 MaskElementIdx = 0; MaskElementIdx < ModifierParameters.MaskedElements.Num(); MaskElementIdx++)
			{
				const int32 SchemaMaskElementIdx = SchemaParameters.ElementNames.Find(ModifierParameters.MaskedElements[MaskElementIdx]);
				check(SchemaMaskElementIdx != INDEX_NONE);
				OutActionModifierVector[1 + SchemaMaskElementIdx] = 1.0f;
			}

			// Set the Sub-elements

			int32 SubElementOffset = 1 + SchemaParameters.Elements.Num();

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				const int32 ModifierElementIndex = ModifierParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);
				if (ModifierElementIndex != INDEX_NONE)
				{
					SetVectorFromModifier(
						OutActionModifierVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						SchemaParameters.Elements[SchemaElementIdx],
						Modifier,
						ModifierParameters.Elements[ModifierElementIndex]);
				}

				SubElementOffset += SubElementSize;
			}

			check(SubElementOffset == OutActionModifierVector.Num());
			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const FModifierOrInclusiveParameters ModifierParameters = Modifier.GetOrInclusive(ModifierElement);

			check(OutActionModifierVector.Num() == 1 + SchemaParameters.Elements.Num() + Private::GetTotalActionModifierVectorSize(Schema, SchemaParameters.Elements));

			// Set the Mask

			for (int32 MaskElementIdx = 0; MaskElementIdx < ModifierParameters.MaskedElements.Num(); MaskElementIdx++)
			{
				const int32 SchemaMaskElementIdx = SchemaParameters.ElementNames.Find(ModifierParameters.MaskedElements[MaskElementIdx]);
				check(SchemaMaskElementIdx != INDEX_NONE);
				OutActionModifierVector[1 + SchemaMaskElementIdx] = 1.0f;
			}

			// Set the Sub-elements

			int32 SubElementOffset = 1 + SchemaParameters.Elements.Num();

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				const int32 ModifierElementIndex = ModifierParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);
				if (ModifierElementIndex != INDEX_NONE)
				{
					SetVectorFromModifier(
						OutActionModifierVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						SchemaParameters.Elements[SchemaElementIdx],
						Modifier,
						ModifierParameters.Elements[ModifierElementIndex]);
				}

				SubElementOffset += SubElementSize;
			}

			check(SubElementOffset == OutActionModifierVector.Num());
			return;
		}

		case EType::Array:
		{
			// Check number of array elements is correct

			const FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const FModifierArrayParameters ModifierParameters = Modifier.GetArray(ModifierElement);
			check(SchemaParameters.Num == ModifierParameters.Elements.Num());

			// Update sub-elements

			const int32 SubElementSize = Schema.GetActionModifierVectorSize(SchemaParameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < SchemaParameters.Num; ElementIdx++)
			{
				SetVectorFromModifier(
					OutActionModifierVector.Slice(1 + ElementIdx * SubElementSize, SubElementSize),
					Schema,
					SchemaParameters.Element,
					Modifier,
					ModifierParameters.Elements[ElementIdx]);
			}

			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const FModifierEncodingParameters ModifierParameters = Modifier.GetEncoding(ModifierElement);

			const int32 SubElementSize = Schema.GetActionModifierVectorSize(SchemaParameters.Element);

			SetVectorFromModifier(
				OutActionModifierVector.Slice(1, SubElementSize),
				Schema,
				SchemaParameters.Element,
				Modifier,
				ModifierParameters.Element);

			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}

		}
	}

	void GetModifierFromVector(
		FModifier& OutModifier,
		FModifierElement& OutModifierElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ActionModifierVector)
	{
		check(Schema.IsValid(SchemaElement));

		// Get Type and Tag

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const FName SchemaElementTag = Schema.GetTag(SchemaElement);

		// Get Action Modifier Vector Size

		const int32 ActionModifierVectorSize = ActionModifierVector.Num();
		check(ActionModifierVectorSize == Schema.GetActionModifierVectorSize(SchemaElement));

		// We always have at least one element in the ActionModifierVector which says if the element is provided
		// if this first value is zero then it means nothing below is masked and we always just return the null element

		check(ActionModifierVectorSize > 0);

		if (ActionModifierVector[0] == 0.0f)
		{
			OutModifierElement = OutModifier.CreateNull(SchemaElementTag);
			return;
		}
		else
		{
			check(ActionModifierVector[0] == 1.0f);
		}

		// Logic for each specific element type

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			OutModifierElement = OutModifier.CreateNull(SchemaElementTag);
			return;
		}

		case EType::Continuous:
		{
			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);
			check(ActionModifierVectorSize == 1 + 2 * SchemaParameters.Num);

			const int32 ValueNum = SchemaParameters.Num;

			TLearningArray<1, bool, TInlineAllocator<32>> ActionMasked;
			TLearningArray<1, float, TInlineAllocator<32>> ActionMaskedValues;
			ActionMasked.SetNumUninitialized({ ValueNum });
			ActionMaskedValues.SetNumUninitialized({ ValueNum });

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				ActionMasked[ValueIdx] = ActionModifierVector[1 + ValueIdx] == 1.0f;
				ActionMaskedValues[ValueIdx] = ActionModifierVector[1 + ValueNum + ValueIdx];
			}

			OutModifierElement = OutModifier.CreateContinuous({ 
				MakeArrayView(ActionMasked.GetData(), ActionMasked.Num()),
				MakeArrayView(ActionMaskedValues.GetData(), ActionMaskedValues.Num()) }, SchemaElementTag);

			return;
		}

		case EType::DiscreteExclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteExclusive(SchemaElement).Num;
			check(ActionModifierVectorSize == 1 + ValueNum);

			// Find Indices
			TArray<int32, TInlineAllocator<8>> MaskedIndices;
			MaskedIndices.Reserve(ValueNum);
			for (int32 Idx = 0; Idx < ValueNum; Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedIndices.Add(Idx);
				}
			}

			OutModifierElement = OutModifier.CreateDiscreteExclusive({ MaskedIndices }, SchemaElementTag);
			return;
		}

		case EType::DiscreteInclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteInclusive(SchemaElement).Num;
			check(ActionModifierVectorSize == 1 + ValueNum);

			// Find Indices
			TArray<int32, TInlineAllocator<8>> MaskedIndices;
			MaskedIndices.Reserve(ValueNum);
			for (int32 Idx = 0; Idx < ValueNum; Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedIndices.Add(Idx);
				}
			}

			OutModifierElement = OutModifier.CreateDiscreteInclusive({ MaskedIndices }, SchemaElementTag);
			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> ElementNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			check(ActionModifierVectorSize == 1 + ElementNames.Num());

			// Find Names
			TArray<FName, TInlineAllocator<8>> MaskedNames;
			MaskedNames.Reserve(ElementNames.Num());
			for (int32 Idx = 0; Idx < ElementNames.Num(); Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedNames.Add(ElementNames[Idx]);
				}
			}

			OutModifierElement = OutModifier.CreateNamedDiscreteExclusive({ MaskedNames }, SchemaElementTag);
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> ElementNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			check(ActionModifierVectorSize == 1 + ElementNames.Num());

			// Find Names
			TArray<FName, TInlineAllocator<8>> MaskedNames;
			MaskedNames.Reserve(ElementNames.Num());
			for (int32 Idx = 0; Idx < ElementNames.Num(); Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedNames.Add(ElementNames[Idx]);
				}
			}

			OutModifierElement = OutModifier.CreateNamedDiscreteInclusive({ MaskedNames }, SchemaElementTag);
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			// Create Sub-elements

			TArray<FModifierElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Elements.Num());

			int32 SubElementOffset = 1;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetModifierFromVector(
					OutModifier,
					SubElements[SchemaElementIdx],
					Schema,
					Parameters.Elements[SchemaElementIdx],
					ActionModifierVector.Slice(SubElementOffset, SubElementSize));

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ActionModifierVectorSize);

			OutModifierElement = OutModifier.CreateAnd({ Parameters.ElementNames, SubElements }, SchemaElementTag);
			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			const int32 SubElementNum = Parameters.Elements.Num();

			// Extract Mask Elements

			TArray<FName, TInlineAllocator<8>> MaskedElements;
			MaskedElements.Reserve(SubElementNum);

			for (int32 Idx = 0; Idx < SubElementNum; Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedElements.Add(Parameters.ElementNames[Idx]);
				}
			}

			// Create Sub-elements

			TArray<FModifierElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(SubElementNum);

			int32 SubElementOffset = 1 + SubElementNum;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SubElementNum; SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetModifierFromVector(
					OutModifier,
					SubElements[SchemaElementIdx],
					Schema,
					Parameters.Elements[SchemaElementIdx],
					ActionModifierVector.Slice(SubElementOffset, SubElementSize));

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ActionModifierVectorSize);

			OutModifierElement = OutModifier.CreateOrExclusive({ Parameters.ElementNames, SubElements, MaskedElements }, SchemaElementTag);
			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			const int32 SubElementNum = Parameters.Elements.Num();

			// Extract Mask Elements

			TArray<FName, TInlineAllocator<8>> MaskedElements;
			MaskedElements.Reserve(SubElementNum);

			for (int32 Idx = 0; Idx < SubElementNum; Idx++)
			{
				check(ActionModifierVector[1 + Idx] == 0.0f || ActionModifierVector[1 + Idx] == 1.0f);
				if (ActionModifierVector[1 + Idx] == 1.0f)
				{
					MaskedElements.Add(Parameters.ElementNames[Idx]);
				}
			}

			// Create Sub-elements

			TArray<FModifierElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(SubElementNum);

			int32 SubElementOffset = 1 + SubElementNum;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SubElementNum; SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetActionModifierVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetModifierFromVector(
					OutModifier,
					SubElements[SchemaElementIdx],
					Schema,
					Parameters.Elements[SchemaElementIdx],
					ActionModifierVector.Slice(SubElementOffset, SubElementSize));

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ActionModifierVectorSize);

			OutModifierElement = OutModifier.CreateOrInclusive({ Parameters.ElementNames, SubElements, MaskedElements }, SchemaElementTag);
			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			TArray<FModifierElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Num);

			// Create sub-elements

			const int32 SubElementSize = Schema.GetActionModifierVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				GetModifierFromVector(
					OutModifier,
					SubElements[ElementIdx],
					Schema,
					Parameters.Element,
					ActionModifierVector.Slice(1 + ElementIdx * SubElementSize, SubElementSize));
			}

			OutModifierElement = OutModifier.CreateArray({ SubElements }, SchemaElementTag);
			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);
			const int32 SubElementSize = Schema.GetActionModifierVectorSize(Parameters.Element);

			FModifierElement SubElement;
			GetModifierFromVector(
				OutModifier,
				SubElement,
				Schema,
				Parameters.Element,
				ActionModifierVector.Slice(1, SubElementSize));

			OutModifierElement = OutModifier.CreateEncoding({ SubElement }, SchemaElementTag);
			return;
		}

		default:
		{
			checkNoEntry();
			OutModifierElement = FModifierElement();
			return;
		}
		}
	}

}

FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FSchemaElement& SchemaElement)
{
	Ar << SchemaElement.Generation;
	Ar << SchemaElement.Index;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FObjectElement& ObjectElement)
{
	Ar << ObjectElement.Generation;
	Ar << ObjectElement.Index;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FModifierElement& ModifierElement)
{
	Ar << ModifierElement.Generation;
	Ar << ModifierElement.Index;
	return Ar;
}

bool operator==(const UE::Learning::Action::FSchemaElement& Lhs, const UE::Learning::Action::FSchemaElement& Rhs)
{
	return Lhs.Generation == Rhs.Generation && Lhs.Index == Rhs.Index;
}

bool operator==(const UE::Learning::Action::FObjectElement& Lhs, const UE::Learning::Action::FObjectElement& Rhs)
{
	return Lhs.Generation == Rhs.Generation && Lhs.Index == Rhs.Index;
}

bool operator==(const UE::Learning::Action::FModifierElement& Lhs, const UE::Learning::Action::FModifierElement& Rhs)
{
	return Lhs.Generation == Rhs.Generation && Lhs.Index == Rhs.Index;
}

uint32 GetTypeHash(const UE::Learning::Action::FSchemaElement& SchemaElement)
{
	return GetTypeHash(TPair<int32, int32>{ SchemaElement.Generation, SchemaElement.Index });
}

uint32 GetTypeHash(const UE::Learning::Action::FObjectElement& ObjectElement)
{
	return GetTypeHash(TPair<int32, int32>{ ObjectElement.Generation, ObjectElement.Index });
}

uint32 GetTypeHash(const UE::Learning::Action::FModifierElement& ModifierElement)
{
	return GetTypeHash(TPair<int32, int32>{ ModifierElement.Generation, ModifierElement.Index });
}