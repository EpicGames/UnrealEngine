// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAdamOptimizer.h"

#include "LearningRandom.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	FAdamOptimizer::FAdamOptimizer(const uint32 InSeed, const FAdamOptimizerSettings& InSettings) : Seed(InSeed), Settings(InSettings) {}

	void FAdamOptimizer::Resize(const int32 SampleNum, const int32 DimensionsNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Resize);

		Iterations.SetNumUninitialized({ DimensionsNum });
		Estimate.SetNumUninitialized({ DimensionsNum });
		Gradient.SetNumUninitialized({ DimensionsNum });

		M0.SetNumUninitialized({ DimensionsNum });
		M1.SetNumUninitialized({ DimensionsNum });
		M1HatMax.SetNumUninitialized({ DimensionsNum });

		GaussianSamples.SetNumUninitialized({ SampleNum, DimensionsNum });
	}

	void FAdamOptimizer::Sample(TLearningArrayView<2, float> OutSamples)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Sample);

		const int32 SampleNum = OutSamples.Num<0>();
		const int32 DimNum = OutSamples.Num<1>();

		Random::SampleGaussianArray(
			GaussianSamples.Flatten(),
			Seed);

#if UE_LEARNING_ISPC
		ispc::LearningSampleAdamOptimizer(
			OutSamples.GetData(),
			GaussianSamples.GetData(),
			Estimate.GetData(),
			Settings.FiniteDifferenceStd,
			SampleNum,
			DimNum);
#else
		// Set first sample to estimate
		Array::Copy(OutSamples[0], Estimate);

		// Set finite difference samples
		for (int32 SampleIdx = 1; SampleIdx < SampleNum; SampleIdx++)
		{
			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				OutSamples[SampleIdx][DimIdx] = Estimate[DimIdx] + Settings.FiniteDifferenceStd * GaussianSamples[SampleIdx][DimIdx];
			}
		}
#endif

		Array::Check(OutSamples);
	}

	void FAdamOptimizer::Reset(TLearningArrayView<2, float> OutSamples, const TLearningArrayView<1, const float> InitialGuess)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Reset);

		Array::Zero(Iterations);
		Array::Copy(Estimate, InitialGuess);
		Array::Zero(Gradient);
		Array::Zero(M0);
		Array::Zero(M1);
		Array::Zero(M1HatMax);
		Array::Zero(GaussianSamples);

		Sample(OutSamples);
	}

	void FAdamOptimizer::Update(
		TLearningArrayView<2, float> InOutSamples,
		const TLearningArrayView<1, const float> Losses,
		const ELogSetting LogSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Update);

		const int32 SampleNum = InOutSamples.Num<0>();
		const int32 DimNum = InOutSamples.Num<1>();

		Array::Check(InOutSamples);
		Array::Check(Losses);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Update::EstimateGradient);

#if UE_LEARNING_ISPC
			ispc::LearningEstimateGradient(
				Gradient.GetData(),
				InOutSamples.GetData(),
				Losses.GetData(),
				SampleNum,
				DimNum);
#else
			Array::Zero(Gradient);

			for (int32 SampleIdx = 1; SampleIdx < SampleNum; SampleIdx++)
			{
				// Compute Sample length
				float Length = 0.0f;
				for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					Length += FMath::Square(InOutSamples[SampleIdx][DimIdx] - InOutSamples[0][DimIdx]);
				}
				Length = FMath::Max(FMath::Sqrt(Length), SMALL_NUMBER);

				const float LossGradient = (Losses[SampleIdx] - Losses[0]) / Length;

				// Estimate Gradient
				for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					const float ControlGradient = (InOutSamples[SampleIdx][DimIdx] - InOutSamples[0][DimIdx]) / Length;

					Gradient[DimIdx] += (DimNum * LossGradient * ControlGradient) / (SampleNum - 1);
				}
			}
#endif

			Array::Check(Gradient);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FAdamOptimizer::Update::UpdateEstimate);

#if UE_LEARNING_ISPC
			ispc::LearningUpdateAdamEstimate(
				Iterations.GetData(),
				Estimate.GetData(),
				M0.GetData(),
				M1.GetData(),
				M1HatMax.GetData(),
				Gradient.GetData(),
				Settings.LearningRate,
				Settings.Beta1,
				Settings.Beta2,
				DimNum);
#else
			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				// Increment Iterations
				Iterations[DimIdx]++;

				// Estimation of first and second moments
				M0[DimIdx] = Settings.Beta1 * M0[DimIdx] + (1.0f - Settings.Beta1) * Gradient[DimIdx];
				M1[DimIdx] = Settings.Beta2 * M1[DimIdx] + (1.0f - Settings.Beta2) * FMath::Square(Gradient[DimIdx]);

				// Correct bias in estimation from zero initialization
				const float M0Hat = M0[DimIdx] / (1.0f - FMath::Pow(Settings.Beta1, Iterations[DimIdx]));
				const float M1Hat = M1[DimIdx] / (1.0f - FMath::Pow(Settings.Beta2, Iterations[DimIdx]));

				// Compute maximum second moment
				M1HatMax[DimIdx] = FMath::Max(M1HatMax[DimIdx], M1Hat);

				// Update Estimate
				Estimate[DimIdx] -= (Settings.LearningRate * M0Hat) / (FMath::Sqrt(M1HatMax[DimIdx]) + UE_SMALL_NUMBER);
			}
#endif
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Adam Iteration %6i | Loss: %5.3f", (int32)Iterations[0], (float)Losses[0]);
		}

		Sample(InOutSamples);
	}
}