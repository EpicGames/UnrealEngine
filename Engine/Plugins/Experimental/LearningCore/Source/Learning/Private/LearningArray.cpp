// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningArray.h"

#include "Async/ParallelFor.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	bool FIndexSet::TryMakeSlice()
	{
		if (IsSlice())
		{
			return true;
		}

		const int32 IdxNum = Indices.Num<0>();

		if (IdxNum == 0)
		{
			SliceStart = 0;
			SliceNum = 0;
			Indices = TLearningArrayView<1, const int32>();
			return true;
		}

		if (IdxNum == 1)
		{
			SliceStart = Indices[0];
			SliceNum = 1;
			Indices = TLearningArrayView<1, const int32>();
			return true;
		}

		int32 MinIdx = INT32_MAX;
		int32 MaxIdx = INT32_MIN;
		for (int32 Idx = 0; Idx < IdxNum; Idx++)
		{
			MinIdx = FMath::Min(MinIdx, Indices[Idx]);
			MaxIdx = FMath::Max(MaxIdx, Indices[Idx]);
		}

		if (MaxIdx - MinIdx + 1 == IdxNum)
		{
			SliceStart = MinIdx;
			SliceNum = IdxNum;
			Indices = TLearningArrayView<1, const int32>();
			return true;
		}
		
		return false;
	}

	void SlicedParallelFor(const int32 Num, const int32 MinSliceElementNum, const TFunctionRef<void(int32 Start, int32 Length)> Body)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::SlicedParallelFor);

		check(Num >= 0);

		if (Num <= 0)
		{
			return;
		}

		check(MinSliceElementNum >= 1);

		const int32 IdealTaskNum = ParallelForImpl::GetNumberOfThreadTasks(Num, MinSliceElementNum, EParallelForFlags::None);
		const int32 IdealSliceLength = FMath::DivideAndRoundUp(Num, IdealTaskNum);
		const int32 ActualTaskNum = FMath::DivideAndRoundUp(Num, IdealSliceLength);

		if (ActualTaskNum == 1)
		{
			Body(0, Num);
		}
		else
		{
			ParallelFor(ActualTaskNum, [IdealSliceLength, Num, Body](int32 Index)
			{
				const int32 StartIndex = Index * IdealSliceLength;
				const int32 StopIndex = FMath::Min((Index + 1) * IdealSliceLength, Num);
				Body(StartIndex, StopIndex - StartIndex);
			});
		}
	}

	namespace Array
	{
		void FloatArrayZero32(float* Data, const int32 Num)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeZero(Data, Num);
#else
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				Data[Idx] = 0.0f;
			}
#endif
		}

		void FloatArraySet32(float* Data, const int32 Num, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSet(Data, Num, Value);
#else
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				Data[Idx] = Value;
			}
#endif
		}

		void FloatArrayCopy32(float* Dst, float* Src, const int32 Num)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeCopy(Dst, Src, Num);
#else
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				Dst[Idx] = Src[Idx];
			}
#endif
		}
	}
}
