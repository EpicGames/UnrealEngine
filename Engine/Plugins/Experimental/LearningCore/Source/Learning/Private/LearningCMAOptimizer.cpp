// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningCMAOptimizer.h"

#include "LearningEigen.h"
#include "LearningRandom.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	int32 FCMAOptimizer::DefaultSampleNum(const int32 DimensionNum)
	{
		return 4 + FMath::Floor(3 * FMath::Loge((float)DimensionNum));
	}

	FCMAOptimizer::FCMAOptimizer(const uint32 InSeed, const FCMAOptimizerSettings& InSettings) : Seed(InSeed), Settings(InSettings) {}

	void FCMAOptimizer::Resize(const int32 SampleNum, const int32 DimensionsNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Resize);

		PathSigma.SetNumUninitialized({ DimensionsNum });
		PathCovariance.SetNumUninitialized({ DimensionsNum });
		Mean.SetNumUninitialized({ DimensionsNum });
		Covariance.SetNumUninitialized({ DimensionsNum, DimensionsNum });
		OldMean.SetNumUninitialized({ DimensionsNum });
		LossRanking.SetNumUninitialized({ SampleNum });
		UpdateDirection.SetNumUninitialized({ DimensionsNum });
		CovarianceTransform.SetNumUninitialized({ DimensionsNum, DimensionsNum });
		CovarianceInverseSqrt.SetNumUninitialized({ DimensionsNum, DimensionsNum });
		GaussianSamples.SetNumUninitialized({ SampleNum, DimensionsNum });
		Weights.SetNumUninitialized({ SampleNum });
	}

	void FCMAOptimizer::Sample(TLearningArrayView<2, float> OutSamples)
	{
		Random::SampleGaussianArray(GaussianSamples.Flatten(), Seed);

#if UE_LEARNING_ISPC
		ispc::LearningTransformCMASamples(
			OutSamples.GetData(),
			GaussianSamples.GetData(),
			Mean.GetData(),
			CovarianceTransform.GetData(),
			OutSamples.Num<0>(),
			OutSamples.Num<1>(),
			Sigma);
#else
		OutEigenMatrix(OutSamples).noalias() =
			((InEigenColMatrix(CovarianceTransform) * (Sigma * InEigenMatrix(GaussianSamples).transpose())).colwise() + InEigenVector(Mean)).transpose();
#endif

		Array::Check(OutSamples);
	}

	void FCMAOptimizer::Reset(
		TLearningArrayView<2, float> OutSamples,
		const TLearningArrayView<1, const float> InitialGuess)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Reset);

		const int32 SampleNum = Weights.Num<0>();
		const int32 DimNum = InitialGuess.Num<0>();

		Iterations = 0;
		Sigma = Settings.InitialStepSize;

		Array::Zero(PathSigma);
		Array::Zero(PathCovariance);
		Array::Copy(Mean, InitialGuess);
		Array::Zero(Covariance);
		Array::Copy(OldMean, InitialGuess);
		Array::Zero(LossRanking);
		Array::Zero(UpdateDirection);
		Array::Zero(CovarianceTransform);
		Array::Zero(CovarianceInverseSqrt);
		Array::Zero(GaussianSamples);

		// Init Covariance to Identity

		for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
		{
			Covariance[DimIdx][DimIdx] = 1.0f;
			CovarianceTransform[DimIdx][DimIdx] = 1.0f;
			CovarianceInverseSqrt[DimIdx][DimIdx] = 1.0f;
		}

		// Pre-compute weighting

		Mu = FMath::Floor(SampleNum * Settings.SurvialRatio);

		Array::Zero(Weights);

		float TotalWeight = 0.0f;
		for (int32 WeightIdx = 0; WeightIdx < Mu; WeightIdx++)
		{
			float Weight = FMath::Loge((float)SampleNum / 2 + 0.5f) - FMath::Loge((float)WeightIdx + 1);
			Weights[WeightIdx] = Weight;
			TotalWeight += Weight;
		}

		// Normalize weights to sum to one

		for (int32 WeightIdx = 0; WeightIdx < Mu; WeightIdx++)
		{
			Weights[WeightIdx] /= TotalWeight;
		}

		// Compute Weight Variance MuEff

		float SquaredTotalWeight = 0.0f;
		for (int32 WeightIdx = 0; WeightIdx < Mu; WeightIdx++)
		{
			SquaredTotalWeight += FMath::Square(Weights[WeightIdx]);
		}

		MuEff = 1.0f / SquaredTotalWeight;

		Sample(OutSamples);
	}

	static inline void ArgSort(
		TLearningArrayView<1, int32> OutValueOrder,
		const TLearningArrayView<1, const float> Values)
	{
		for (int32 Idx = 0; Idx < Values.Num(); Idx++)
		{
			OutValueOrder[Idx] = Idx;
		}

		TArrayView<int32>(OutValueOrder.GetData(), OutValueOrder.Num()).Sort([Values](const int32& Lhs, const int32& Rhs)
		{
			return  Values[Lhs] < Values[Rhs];
		});
	}

	void FCMAOptimizer::Update(
		TLearningArrayView<2, float> InOutSamples,
		const TLearningArrayView<1, const float> Losses,
		const ELogSetting LogSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update);

		Array::Check(InOutSamples);
		Array::Check(Losses);

		const int32 DimNum = InOutSamples.Num<1>();
		const int32 SampleNum = InOutSamples.Num<0>();

		// Update Iteration Count

		Iterations++;

		// Compute Parameters

		const float CC = (4 + MuEff / DimNum) / (DimNum + 4 + 2 * MuEff / DimNum);
		const float CS = (MuEff + 2) / (DimNum + MuEff + 5);
		const float C1 = 2 / (FMath::Square(DimNum + 1.3f) + DimNum);
		const float Cmu = FMath::Min(
			1 - C1,
			2 * (MuEff - 2 + 1 / MuEff) / (FMath::Square(DimNum + 2) + MuEff));
		const float Damps = 2 * MuEff / SampleNum + 0.3f + CS;
		const float CN = CS / Damps;

		// Update Loss Ranking

		ArgSort(LossRanking, Losses);

		float AvgLoss = 0.0f;
		for (int32 LossIdx = 0; LossIdx < SampleNum; LossIdx++)
		{
			AvgLoss += Losses[LossIdx] / SampleNum;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "CMA Iteration %6i | Avg Loss: %5.3f | Best Loss: %5.3f | Sigma: %5.3f",
				Iterations, AvgLoss, (float)Losses[LossRanking[0]], Sigma);
		}

		// Update Mean as weighted sum of controls

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update::AdjustMean);

#if UE_LEARNING_ISPC
			ispc::LearningAdjustCMAMean(
				Mean.GetData(),
				OldMean.GetData(),
				Weights.GetData(),
				LossRanking.GetData(),
				InOutSamples.GetData(),
				Mu,
				DimNum);
#else
			Array::Copy(OldMean, Mean);
			Array::Zero(Mean);

			for (int32 LossIdx = 0; LossIdx < Mu; LossIdx++)
			{
				const float LossWeight = Weights[LossIdx];
				const int32 ControlIdx = LossRanking[LossIdx];

				for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					Mean[DimIdx] += LossWeight * InOutSamples[ControlIdx][DimIdx];
				}
			}
#endif
		}

		// Compute Update Direction

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update::ComputeUpdateDirection);

#if UE_LEARNING_ISPC
			ispc::LearningComputeCMAUpdateDirection(
				UpdateDirection.GetData(),
				Mean.GetData(),
				OldMean.GetData(),
				CovarianceInverseSqrt.GetData(),
				Mean.Num());
#else
			OutEigenVector(UpdateDirection).noalias() = InEigenColMatrix(CovarianceInverseSqrt) * (InEigenVector(Mean) - InEigenVector(OldMean));
#endif
		}

		// Update PathSigma and PathCovariance

		bool bHsig = false;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update::UpdatePaths);

			float SumSquarePs = 0.0f;

			float CSN = FMath::Sqrt(CS * (2 - CS) * MuEff) / Sigma;
			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				PathSigma[DimIdx] = (1 - CS) * PathSigma[DimIdx] + CSN * UpdateDirection[DimIdx];
			}

			float CCN = FMath::Sqrt(CC * (2 - CC) * MuEff) / Sigma;

			SumSquarePs = 0.0f;
			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				SumSquarePs += FMath::Square(PathSigma[DimIdx]);
			}

			bHsig = (SumSquarePs / DimNum
				/ (1 - FMath::Pow(1 - CS, 2 * Iterations))
				< 2 + 4.0f / (DimNum + 1));

			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				PathCovariance[DimIdx] = (1 - CC) * PathCovariance[DimIdx] + CCN * ((float)bHsig) * (Mean[DimIdx] - OldMean[DimIdx]);
			}
		}

		// Update Covariance and Sigma

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update::UpdateCovariance);

			// Original reference code here squares the bool... no idea why
			const float C1A = C1 * (1 - (1 - FMath::Square((float)bHsig)) * CC * (2 - CC));
			const float Scale = 1.0f - C1A - Cmu;

#if UE_LEARNING_ISPC
			ispc::LearningUpdateCMACovariance(
				Covariance.GetData(),
				PathCovariance.GetData(),
				Scale,
				C1,
				DimNum);
#else
			for (int32 DimX = 0; DimX < DimNum; DimX++)
			{
				for (int32 DimY = 0; DimY < DimNum; DimY++)
				{
					Covariance[DimX][DimY] = Scale * Covariance[DimX][DimY] + C1 * PathCovariance[DimX] * PathCovariance[DimY];
				}
			}
#endif

			float SumSquarePs = 0.0f;
			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				SumSquarePs += FMath::Square(PathSigma[DimIdx]);
			}

			Sigma *= FMath::Exp(FMath::Min(1.0f, CN * (SumSquarePs / DimNum - 1) / 2));
		}

		// Update Sampling Transform
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FCMAOptimizer::Update::SamplingTransform);

			Eigen::SelfAdjointEigenSolver<Eigen::MatrixXf> EigenSolver(InEigenColMatrix(Covariance));
			check(EigenSolver.info() == Eigen::Success);

			OutEigenColMatrix(CovarianceTransform).noalias() = EigenSolver.eigenvectors() * EigenSolver.eigenvalues().cwiseMax(0).cwiseSqrt().asDiagonal();
			OutEigenColMatrix(CovarianceInverseSqrt).noalias() = EigenSolver.operatorInverseSqrt();
		}

		Array::Check(CovarianceTransform);
		Array::Check(CovarianceInverseSqrt);

		Sample(InOutSamples);
	}
}

