// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningCompletion.h"

namespace UE::Learning
{
	void FResetInstanceBuffer::Reserve(const int32 InMaxInstanceNum)
	{
		ResetInstances.Reserve(InMaxInstanceNum);
	}

	void FResetInstanceBuffer::SetResetInstances(const FIndexSet Instances)
	{
		ResetInstances.Reset();
		for (const int32 InstanceIdx : Instances)
		{
			ResetInstances.Add(InstanceIdx);
		}

		ResetInstancesSet = ResetInstances;
		ResetInstancesSet.TryMakeSlice();
	}

	void FResetInstanceBuffer::SetResetInstancesFromCompletions(const TLearningArrayView<1, const ECompletionMode> Completions, const FIndexSet Instances)
	{
		ResetInstances.Reset();
		for (const int32 InstanceIdx : Instances)
		{
			if (Completions[InstanceIdx] != ECompletionMode::Running)
			{
				ResetInstances.Add(InstanceIdx);
			}
		}

		ResetInstancesSet = ResetInstances;
		ResetInstancesSet.TryMakeSlice();
	}

	const int32 FResetInstanceBuffer::GetResetInstanceNum() const { return ResetInstances.Num(); }

	const TArray<int32>& FResetInstanceBuffer::GetResetInstancesArray() const { return ResetInstances; }

	const FIndexSet FResetInstanceBuffer::GetResetInstances() const { return ResetInstancesSet; }

	namespace Completion
	{
		const TCHAR* CompletionModeString(const ECompletionMode Completion)
		{
			switch (Completion)
			{
			case ECompletionMode::Running: return TEXT("Running");
			case ECompletionMode::Truncated: return TEXT("Truncated");
			case ECompletionMode::Terminated: return TEXT("Terminated");
			default: checkNoEntry(); return TEXT("Unknown");
			}
		}

		ECompletionMode Or(const ECompletionMode Lhs, const ECompletionMode Rhs)
		{
			if ((Lhs == ECompletionMode::Running && Rhs != ECompletionMode::Running) ||
				(Lhs == ECompletionMode::Truncated && Rhs == ECompletionMode::Terminated))
			{
				return Rhs;
			}
			else
			{
				return Lhs;
			}
		}

		void EvaluateEndOfEpisodeCompletions(
			TLearningArrayView<1, ECompletionMode> InOutCompletions,
			const TLearningArrayView<1, const int32> EpisodeStepNums,
			const int32 EpisodeMaxStepNum,
			const FIndexSet Instances)
		{
			for (const int32 InstanceIdx : Instances)
			{
				InOutCompletions[InstanceIdx] = EpisodeStepNums[InstanceIdx] == EpisodeMaxStepNum ? ECompletionMode::Truncated : ECompletionMode::Running;
			}
		}
	}
}

