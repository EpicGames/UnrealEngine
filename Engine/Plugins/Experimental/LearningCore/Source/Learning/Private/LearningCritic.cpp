// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningCritic.h"

namespace UE::Learning
{
	FNeuralNetworkCritic::FNeuralNetworkCritic(
		const int32 InMaxInstanceNum,
		const int32 InObservationEncodedNum,
		const int32 InMemoryStateNum,
		const TSharedPtr<FNeuralNetwork>& InNeuralNetwork,
		const FNeuralNetworkInferenceSettings& InInferenceSettings)
		: ObservationEncodedNum(InObservationEncodedNum)
		, MemoryStateNum(InMemoryStateNum)
	{
		check(InNeuralNetwork->GetInputSize() == ObservationEncodedNum + MemoryStateNum);
		check(InNeuralNetwork->GetOutputSize() == 1);

		Input.SetNumUninitialized({ InMaxInstanceNum, ObservationEncodedNum + MemoryStateNum });
		Array::Zero(Input);

		NeuralNetworkFunction = MakeShared<FNeuralNetworkFunction>(InMaxInstanceNum, InNeuralNetwork, InInferenceSettings);
	}

	void FNeuralNetworkCritic::Evaluate(
		TLearningArrayView<1, float> OutputReturns,
		const TLearningArrayView<2, const float> InputObservationVectorsEncoded,
		const TLearningArrayView<2, const float> InputMemoryState,
		const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FNeuralNetworkCritic::Evaluate);

		Array::Check(InputObservationVectorsEncoded, Instances);
		Array::Check(InputMemoryState, Instances);

		// Copy in Observation and Memory State into network input

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(Input[InstanceIdx].Slice(0, ObservationEncodedNum), InputObservationVectorsEncoded[InstanceIdx]);
			Array::Copy(Input[InstanceIdx].Slice(ObservationEncodedNum, MemoryStateNum), InputMemoryState[InstanceIdx]);
		}

		// Evaluate Network

		NeuralNetworkFunction->Evaluate(
			TLearningArrayView<2, float>(OutputReturns.GetData(), { OutputReturns.Num(), 1 }),
			Input, 
			Instances);

		Array::Check(OutputReturns, Instances);
	}

	void FNeuralNetworkCritic::UpdateNeuralNetwork(const TSharedPtr<FNeuralNetwork>& NewNeuralNetwork)
	{
		NeuralNetworkFunction->UpdateNeuralNetwork(NewNeuralNetwork);
	}

	const TSharedPtr<FNeuralNetwork>& FNeuralNetworkCritic::GetNeuralNetwork() const
	{
		return NeuralNetworkFunction->GetNeuralNetwork();
	}
}
