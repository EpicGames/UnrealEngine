// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h" // for UE_LEARNING_DEVELOPMENT

#include "HAL/Platform.h"
#include "Misc/Build.h"

// Disable static analysis warnings
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif

// Enable static asserts
#ifdef EIGEN_NO_STATIC_ASSERT
#undef EIGEN_NO_STATIC_ASSERT
#endif

// Enables or disables various Eigen debugging related functionality. 
// In general there is not much harm in enabling this since it does
// not perform expensive checks however this can still be disabled 
// in editor mode to give some small improvement to performance
#define UE_LEARNING_EIGEN_DEBUG_ENABLE (!UE_BUILD_SHIPPING)
//#define UE_LEARNING_EIGEN_DEBUG_ENABLE 0

#if UE_LEARNING_EIGEN_DEBUG_ENABLE
	#ifndef EIGEN_MPL2_ONLY
	#define EIGEN_MPL2_ONLY
	#endif

	#ifdef EIGEN_NO_DEBUG
	#undef EIGEN_NO_DEBUG
	#endif

	#ifdef eigen_assert
	#undef eigen_assert
	#endif

	#define eigen_assert(x) check(x)
#else
	#ifdef EIGEN_MPL2_ONLY
	#undef EIGEN_MPL2_ONLY
	#endif

	#ifndef EIGEN_NO_DEBUG
	#define EIGEN_NO_DEBUG
	#endif

	#ifdef eigen_assert
	#undef eigen_assert
	#endif
#endif

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Core>
#include <Eigen/Eigenvalues>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

// Restore static analysis warnings
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

namespace Eigen
{
	// RowMajor matrix type we can use to Map our C-like array to Eigen
	using RowMajorMatrixXf = Matrix<float, Dynamic, Dynamic, RowMajor>;

	// RowMajor 2D array type we can use to Map our C-like array to Eigen
	using RowMajorArrayXXf = Array<float, Dynamic, Dynamic, RowMajor>;

	// Computation info enum to string mapping
	static FORCEINLINE const TCHAR* ComputationInfoString(ComputationInfo Info)
	{
		switch (Info)
		{
		case ComputationInfo::InvalidInput: return TEXT("Invalid Input");
		case ComputationInfo::NoConvergence: return TEXT("No Convergence");
		case ComputationInfo::NumericalIssue: return TEXT("Numerical Issues");
		case ComputationInfo::Success: return TEXT("Success");
		default: return TEXT("Unknown");
		}
	}
}

// For some reason these need to be macros rather than functions.
// I don't know exactly why but I get memory errors when I try to
// return `Eigen::Map` from a function.

#define InEigenVector(Array) Eigen::Map<const Eigen::VectorXf>(Array.GetData(), Eigen::Index(Array.Num()))
#define InEigenRowVector(Array) Eigen::Map<const Eigen::RowVectorXf>(Array.GetData(), Eigen::Index(Array.Num()))
#define InEigenArray1D(Array) Eigen::Map<const Eigen::ArrayXf>(Array.GetData(), Eigen::Index(Array.Num()))
#define InEigenMatrix(Array) Eigen::Map<const Eigen::RowMajorMatrixXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define InEigenColMatrix(Array) Eigen::Map<const Eigen::MatrixXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define InEigenArray2D(Array) Eigen::Map<const Eigen::RowMajorArrayXXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define InEigenColArray2D(Array) Eigen::Map<const Eigen::ArrayXXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))

#define OutEigenVector(Array) Eigen::Map<Eigen::VectorXf>(Array.GetData(), Eigen::Index(Array.Num()))
#define OutEigenArray1D(Array) Eigen::Map<Eigen::ArrayXf>(Array.GetData(), Eigen::Index(Array.Num()))
#define OutEigenMatrix(Array) Eigen::Map<Eigen::RowMajorMatrixXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define OutEigenColMatrix(Array) Eigen::Map<Eigen::MatrixXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define OutEigenArray2D(Array) Eigen::Map<Eigen::RowMajorArrayXXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
#define OutEigenColArray2D(Array) Eigen::Map<Eigen::ArrayXXf>(Array.GetData(), Eigen::Index(Array.Num<0>()), Eigen::Index(Array.Num<1>()))
