// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningFrameAttribute.h"
#include "LearningFrameSet.h"

#include "Math/Transform.h"

#include "LearningEigen.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	namespace FrameAttribute::Private
	{
		static inline float ApproxAcos(const float X)
		{
			const float Y = FMath::Abs(X);
			const float P = -0.1565827 * Y + 1.570796;
			const float Q = P * FMath::Sqrt(FMath::Max(1.0 - Y, 0.0));
			return X >= 0.0 ? Q : UE_PI - Q;
		}

		static inline float InertializationPartialIntegral(
			const float X,
			const float V,
			const float Time,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = FMath::Clamp(Time / Z, 0.0f, 1.0f);
			return
				X * ((Z / 2) * T * T * T * T - Z * T * T * T + Z * T) +
				V * (((Z * Z) / 4) * T * T * T * T - ((2 * Z * Z) / 3) * T * T * T + ((Z * Z) / 2) * T * T);
		}

		static inline float InertializationFullIntegral(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			return X * (Z / 2) + V * ((Z * Z) / 12);
		}

		static inline float InertializationPosition(
			const float X,
			const float V,
			const float Time,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = FMath::Clamp(Time / Z, 0.0f, 1.0f);
			return X * (2 * T * T * T - 3 * T * T + 1) + V * (Z * T * T * T - Z * 2 * T * T + Z * T);
		}

		static inline float InertializationVelocity(
			const float X,
			const float V,
			const float Time,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = FMath::Clamp(Time / Z, 0.0f, 1.0f);
			return X * ((6 / Z) * T * T - (6 / Z) * T) + V * (3 * T * T - 4 * T + 1);
		}

		static inline float InertializationPositionIntersectionTime(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			return (-Z * X) / (2 * X + Z * V);
		}

		static inline float InertializationVelocityIntersectionTime(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			return (Z * Z * V) / (6 * X + 3 * Z * V);
		}

		static inline float InertializationAccelerationIntersectionTime(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			return (3 * Z * X + 2 * Z * Z * V) / (6 * X + 3 * Z * V);
		}

		static inline float InertializationMaximumPositionMagnitude(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = InertializationVelocityIntersectionTime(X, V, Z);

			return T > 0.0f && T < Z ?
				FMath::Max(FMath::Abs(X), FMath::Abs(InertializationPosition(X, V, T, Z))) : FMath::Abs(X);
		}

		static inline float InertializationMaximumVelocityMagnitude(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = InertializationAccelerationIntersectionTime(X, V, Z);

			return T > 0.0f && T < Z ?
				FMath::Max(FMath::Abs(V), FMath::Abs(InertializationVelocity(X, V, T, Z))) : FMath::Abs(V);
		}

		static inline float InertializationIntegralMagnitude(
			const float X,
			const float V,
			const float BlendTime)
		{
			const float Z = FMath::Max(BlendTime, UE_SMALL_NUMBER);
			const float T = InertializationPositionIntersectionTime(X, V, Z);
			const float IntegralT = InertializationPartialIntegral(X, V, T, Z);
			const float IntegralZ = InertializationFullIntegral(X, V, Z);

			return T > 0.0f && T < Z ?
				FMath::Abs(IntegralZ - IntegralT) + FMath::Abs(IntegralT) :
				FMath::Abs(IntegralZ);
		}

		static inline bool FindMinimum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMinimumValue,
			const TLearningArrayView<2, const float> Values)
		{
#if UE_LEARNING_ISPC
			return ispc::LearningFrameAttributeFindMinimum(
				OutChannelIdx,
				OutFrameIdx,
				OutMinimumValue,
				Values.GetData(),
				Values.Num<0>(),
				Values.Num<1>());
#else
			OutChannelIdx = INDEX_NONE;
			OutFrameIdx = INDEX_NONE;
			OutMinimumValue = UE_MAX_FLT;

			const int32 ChannelNum = Values.Num<0>();
			const int32 FrameNum = Values.Num<1>();

			for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
			{
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					if (Values[ChannelIdx][FrameIdx] < OutMinimumValue)
					{
						OutChannelIdx = ChannelIdx;
						OutFrameIdx = FrameIdx;
						OutMinimumValue = Values[ChannelIdx][FrameIdx];
					}
				}
			}

			return OutChannelIdx != INDEX_NONE;
#endif
		}
		
		static inline bool FindMaximum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMaximumValue,
			const TLearningArrayView<2, const float> Values)
		{
#if UE_LEARNING_ISPC
			return ispc::LearningFrameAttributeFindMaximum(
				OutChannelIdx,
				OutFrameIdx,
				OutMaximumValue,
				Values.GetData(),
				Values.Num<0>(),
				Values.Num<1>());
#else
			OutChannelIdx = INDEX_NONE;
			OutFrameIdx = INDEX_NONE;
			OutMaximumValue = -UE_MAX_FLT;

			const int32 ChannelNum = Values.Num<0>();
			const int32 FrameNum = Values.Num<1>();

			for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
			{
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					if (Values[ChannelIdx][FrameIdx] > OutMaximumValue)
					{
						OutChannelIdx = ChannelIdx;
						OutFrameIdx = FrameIdx;
						OutMaximumValue = Values[ChannelIdx][FrameIdx];
					}
				}
			}

			return OutChannelIdx != INDEX_NONE;
#endif
		}

		static inline void Zero(TLearningArrayView<1, float> Out)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeZero(Out.GetData(), Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = 0.0f;
			}
#endif
		}

		static inline void Set(TLearningArrayView<1, float> Out, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSet(Out.GetData(), Out.Num(), Value);
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Value;
			}
#endif
		}

		static inline void Copy(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeCopy(Out.GetData(), In.GetData(), Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = In[ValueIdx];
			}
#endif
		}

		static inline bool Equal(const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningFrameAttributeEqual(Lhs.GetData(), Rhs.GetData(), Lhs.Num());
#else
			const int32 ValueNum = Lhs.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				if (Lhs[ValueIdx] != Rhs[ValueIdx]) { return false; }
			}
			return true;
#endif
		}

		static inline void Select(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Cond, const TLearningArrayView<1, const float> True, const TLearningArrayView<1, const float> False)
		{
			check(Out.Num() == Cond.Num());
			check(Out.Num() == True.Num());
			check(Out.Num() == False.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSelect(
				Out.GetData(),
				Cond.GetData(),
				True.GetData(),
				False.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Cond[ValueIdx] ? True[ValueIdx] : False[ValueIdx];
			}
#endif
		}

		static inline void Add(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num()); 

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeAdd(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] + Rhs[ValueIdx];
			}
#endif
		}

		static inline void Sub(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSub(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] - Rhs[ValueIdx];
			}
#endif
		}

		static inline void Mul(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeMul(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] * Rhs[ValueIdx];
			}
#endif
		}

		static inline void Div(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeDiv(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] / Rhs[ValueIdx];
			}
#endif
		}

		static inline void Max(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeMax(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Max(Lhs[ValueIdx], Rhs[ValueIdx]);
			}
#endif
		}

		static inline void Min(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeMin(
				Out.GetData(),
				Lhs.GetData(),
				Rhs.GetData(),
				Out.Num());
#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Min(Lhs[ValueIdx], Rhs[ValueIdx]);
			}
#endif
		}

		static inline void Dot(TLearningArrayView<1, float> InOut, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.Num() == Lhs.Num());
			check(InOut.Num() == Rhs.Num());
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] += Lhs[ValueIdx] * Rhs[ValueIdx];
			}
		}

		static inline void Cross(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ)
		{
			check(OutX.Num() == OutY.Num());
			check(OutX.Num() == OutZ.Num());
			check(OutX.Num() == LhsX.Num());
			check(OutX.Num() == LhsY.Num());
			check(OutX.Num() == LhsZ.Num());
			check(OutX.Num() == RhsX.Num());
			check(OutX.Num() == RhsY.Num());
			check(OutX.Num() == RhsZ.Num());

			const int32 ValueNum = OutX.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f Lhs = FVector3f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx]);
				const FVector3f Rhs = FVector3f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx]);
				const FVector3f Out = Lhs.Cross(Rhs);

				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
			}
		}

		static inline void Neg(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = -In[ValueIdx];
			}
		}

		static inline void Inv(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = 1.0f / In[ValueIdx];
			}
		}

		static inline void Abs(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Abs(In[ValueIdx]);
			}
		}

		static inline void Log(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Loge(In[ValueIdx]);
			}
		}

		static inline void Exp(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Exp(In[ValueIdx]);
			}
		}

		static inline void Sqrt(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Sqrt(In[ValueIdx]);
			}
		}

		static inline void Sin(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Sin(In[ValueIdx]);
			}
		}

		static inline void Cos(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Cos(In[ValueIdx]);
			}
		}

		static inline void Atan2(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> InY, const TLearningArrayView<1, const float> InX)
		{
			check(Out.Num() == InY.Num());
			check(Out.Num() == InX.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Atan2(InY[ValueIdx], InX[ValueIdx]);
			}
		}

		static inline void NegInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = -InOut[ValueIdx];
			}
		}

		static inline void RepeatFirstRangeEntryInplace(
			TLearningArrayView<1, float> InOut,
			const TLearningArrayView<1, const int32> RangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)
		{
			const int32 RangeNum = RangeOffsets.Num();

			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const int32 RangeOffset = RangeOffsets[RangeIdx];
				const int32 RangeLength = RangeLengths[RangeIdx];
				for (int32 ValueIdx = RangeOffset; ValueIdx < RangeOffset + RangeLength; ValueIdx++)
				{
					InOut[ValueIdx] = InOut[RangeOffset];
				}
			}
		}

		static inline void InvInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = 1.0f / InOut[ValueIdx];
			}
		}

		static inline void AbsInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = FMath::Abs(InOut[ValueIdx]);
			}
		}

		static inline void LogInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = FMath::Loge(InOut[ValueIdx]);
			}
		}

		static inline void ExpInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = FMath::Exp(InOut[ValueIdx]);
			}
		}

		static inline void SqrtInplace(TLearningArrayView<1, float> InOut)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSqrtInplace(
				InOut.GetData(),
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = FMath::Sqrt(InOut[ValueIdx]);
			}
#endif
		}

		static inline void ApproxAcosInplace(TLearningArrayView<1, float> InOut)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeApproxAcosInplace(
				InOut.GetData(),
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = ApproxAcos(InOut[ValueIdx]);
			}
#endif
		}

		static inline void NormalizeInplace(TLearningArrayView<2, float> InOut, const float Eps = UE_SMALL_NUMBER)
		{
			const int32 ChannelNum = InOut.Num<0>();
			const int32 FrameNum = InOut.Num<1>();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				float LengthSquared = 0.0f;
				for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
				{
					LengthSquared += InOut[ChannelIdx][FrameIdx] * InOut[ChannelIdx][FrameIdx];
				}

				const float Length = FMath::Sqrt(LengthSquared);

				for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
				{
					InOut[ChannelIdx][FrameIdx] = InOut[ChannelIdx][FrameIdx] / FMath::Max(Length, Eps);
				}
			}
		}

		static inline void LengthSquared(TLearningArrayView<1, float> InOut, const TLearningArrayView<1, const float> In)
		{
			check(InOut.Num() == In.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeLengthSquared(
				InOut.GetData(),
				In.GetData(),
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] += In[ValueIdx] * In[ValueIdx];
			}
#endif
		}

		static inline void Normalize(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In, const float Eps = UE_SMALL_NUMBER)
		{
			check(Out.Num<0>() == In.Num<0>());
			check(Out.Num<1>() == In.Num<1>());

			const int32 ChannelNum = In.Num<0>();
			const int32 FrameNum = In.Num<1>();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				float LengthSquared = 0.0f;
				for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
				{
					LengthSquared += In[ChannelIdx][FrameIdx] * In[ChannelIdx][FrameIdx];
				}

				const float Length = FMath::Sqrt(LengthSquared);

				for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
				{
					Out[ChannelIdx][FrameIdx] = In[ChannelIdx][FrameIdx] / FMath::Max(Length, UE_SMALL_NUMBER);
				}
			}
		}

		static inline void AddConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] + Rhs;
			}
		}

		static inline void SubConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] - Rhs;
			}
		}

		static inline void MulConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] * Rhs;
			}
		}

		static inline void DivConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] / Rhs;
			}
		}

		static inline void MaxConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Max(Lhs[ValueIdx], Rhs);
			}
		}

		static inline void MinConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Min(Lhs[ValueIdx], Rhs);
			}
		}

		static inline void ConstantSub(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs - Rhs[ValueIdx];
			}
		}

		static inline void ConstantDiv(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs / Rhs[ValueIdx];
			}
		}

		static inline void ConstantDot(TLearningArrayView<1, float> InOut, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeConstantDot(
				InOut.GetData(),
				Lhs,
				Rhs.GetData(),
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] += Lhs * Rhs[ValueIdx];
			}
#endif
		}

		static inline void ConstantDistSquared(TLearningArrayView<1, float> InOut, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.Num() == Rhs.Num());

#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeConstantDistSquared(
				InOut.GetData(),
				Lhs,
				Rhs.GetData(),
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] += FMath::Square(Lhs - Rhs[ValueIdx]);
			}
#endif
		}

		static inline void AddConstantInplace(TLearningArrayView<1, float> InOut, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeAddConstantInplace(
				InOut.GetData(),
				Value,
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] += Value;
			}
#endif
		}

		static inline void SubConstantInplace(TLearningArrayView<1, float> InOut, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeSubConstantInplace(
				InOut.GetData(),
				Value,
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] -= Value;
			}
#endif
		}

		static inline void MulConstantInplace(TLearningArrayView<1, float> InOut, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeMulConstantInplace(
				InOut.GetData(),
				Value,
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] *= Value;
			}
#endif
		}

		static inline void DivConstantInplace(TLearningArrayView<1, float> InOut, const float Value)
		{
#if UE_LEARNING_ISPC
			ispc::LearningFrameAttributeDivConstantInplace(
				InOut.GetData(),
				Value,
				InOut.Num());
#else
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] /= Value;
			}
#endif
		}

		static inline void LogicalAnd(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] && Rhs[ValueIdx];
			}
		}

		static inline void LogicalOr(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] || Rhs[ValueIdx];
			}
		}

		static inline void LogicalNot(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In)
		{
			check(Out.Num() == In.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = !In[ValueIdx];
			}
		}

		static inline void Gt(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] > Rhs[ValueIdx];
			}
		}

		static inline void Ge(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] >= Rhs[ValueIdx];
			}
		}

		static inline void Lt(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] < Rhs[ValueIdx];
			}
		}

		static inline void Le(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] <= Rhs[ValueIdx];
			}
		}

		static inline void Eq(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] == Rhs[ValueIdx];
			}
		}

		static inline void Neq(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Lhs.Num());
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] != Rhs[ValueIdx];
			}
		}

		static inline void GtConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] > Rhs;
			}
		}

		static inline void GeConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] >= Rhs;
			}
		}

		static inline void LtConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] < Rhs;
			}
		}

		static inline void LeConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] <= Rhs;
			}
		}

		static inline void EqConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] == Rhs;
			}
		}

		static inline void NeqConstant(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> Lhs, const float Rhs)
		{
			check(Out.Num() == Lhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs[ValueIdx] != Rhs;
			}
		}

		static inline void ConstantGt(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs > Rhs[ValueIdx];
			}
		}

		static inline void ConstantGe(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs >= Rhs[ValueIdx];
			}
		}

		static inline void ConstantLt(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs < Rhs[ValueIdx];
			}
		}

		static inline void ConstantLe(TLearningArrayView<1, float> Out, const float Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Out.Num() == Rhs.Num());
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = Lhs <= Rhs[ValueIdx];
			}
		}

		static inline void FilterWithKernel(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In, const TLearningArrayView<1, const float> Kernel)
		{
			check(Kernel.Num() % 2 == 1); // Kernel must be odd in length

#if UE_LEARNING_ISPC

			ispc::LearningFilterWithKernel(Out.GetData(), In.GetData(), Kernel.GetData(), In.Num(), Kernel.Num());

#else
			const int32 KernelNum = Kernel.Num();
			const int32 KernelHalfWidth = FMath::DivideAndRoundDown(KernelNum, 2);

			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				float Total = 0.0f;
				Out[ValueIdx] = 0.0f;

				for (int32 Offset = -KernelHalfWidth; Offset <= +KernelHalfWidth; Offset++)
				{
					if (ValueIdx + Offset >= 0 && ValueIdx + Offset < ValueNum)
					{
						const float Weight = Kernel[Offset + KernelHalfWidth];
						Out[ValueIdx] += Weight * In[ValueIdx + Offset];
						Total += Weight;
					}
				}

				Out[ValueIdx] = Out[ValueIdx] / Total;
			}
#endif
		}

		static inline void GaussianKernel(TLearningArrayView<1, float> Kernel, const float StdInFrames)
		{
			check(Kernel.Num() % 2 == 1);

			const int32 KernelSize = Kernel.Num();

			for (int32 KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
			{
				Kernel[KernelIdx] = FMath::Exp(-FMath::Square(((float)KernelIdx + 0.5f - ((float)KernelSize / 2.0f)) / FMath::Max(StdInFrames, UE_SMALL_NUMBER)));
			}
		}

		static inline void FilterMajorityVote(TLearningArrayView<1, float> Out, const TLearningArrayView<1, const float> In, const int32 FilterWidthInFrames)
		{
			check(Out.Num() == In.Num());
			check(FilterWidthInFrames >= 0);

			const int32 FilerHalfWidth = FMath::DivideAndRoundDown(FilterWidthInFrames, 2);

#if UE_LEARNING_ISPC

			ispc::LearningFilterMajorityVote(Out.GetData(), In.GetData(), In.Num(), FilerHalfWidth);

#else
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				int32 Total = 0;

				for (int32 Offset = -FilerHalfWidth; Offset <= +FilerHalfWidth; Offset++)
				{
					if (ValueIdx + Offset >= 0 && ValueIdx + Offset < ValueNum)
					{
						Total += In[ValueIdx + Offset] ? 1 : -1;
					}
				}

				Out[ValueIdx] = Total > 0;
			}
#endif
		}

		static inline void SavGolKernel(TLearningArrayView<1, float> Kernel, const int32 PolynomialDegree, bool bGaussianWindowed)
		{
			check(Kernel.Num() % 2 == 1);

			const int32 KernelSize = Kernel.Num();
			const int32 Order = PolynomialDegree + 1;

			TLearningArray<2, float> Lhs;
			Lhs.SetNumUninitialized({ KernelSize, Order });
			for (int32 KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
			{
				for (int32 OrderIdx = 0; OrderIdx < Order; OrderIdx++)
				{
					Lhs[KernelIdx][OrderIdx] = FMath::Pow((float)KernelIdx, (float)OrderIdx);
				}
			}

			if (bGaussianWindowed)
			{
				for (int32 KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
				{
					const float Weight = FMath::Exp(-FMath::Square(((float)KernelIdx + 0.5f - ((float)KernelSize / 2.0f)) / (1.5f * KernelSize)));

					for (int32 OrderIdx = 0; OrderIdx < Order; OrderIdx++)
					{
						Lhs[KernelIdx][OrderIdx] *= Weight;
					}
				}
			}

			TLearningArray<2, float> Rhs;
			Rhs.SetNumZeroed({ KernelSize, 1 });
			Rhs[KernelSize / 2][0] = 1.0f;

			TLearningArray<2, float> Coeff;
			Coeff.SetNumZeroed({ Order, 1 });

			OutEigenMatrix(Coeff).noalias() = InEigenMatrix(Lhs).bdcSvd(Eigen::ComputeThinU | Eigen::ComputeThinV).solve(InEigenMatrix(Rhs));

			Array::Zero(Kernel);

			for (int32 KernelIdx = 0; KernelIdx < KernelSize; KernelIdx++)
			{
				for (int32 OrderIdx = 0; OrderIdx < Order; OrderIdx++)
				{
					Kernel[KernelIdx] += Coeff[OrderIdx][0] * FMath::Pow((float)KernelIdx, (float)OrderIdx);
				}
			}
		}

		static inline void Mean(float& OutMean, const TLearningArrayView<1, const float> In)
		{
			const int32 Num = In.Num();

			OutMean = 0.0f;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				OutMean += (In[Idx] - OutMean) / (Idx + 1);
			}
		}

		static inline void MeanStd(float& OutMean, float& OutStd, const TLearningArrayView<1, const float> In)
		{
			const int32 Num = In.Num();

			OutMean = 0.0f;
			OutStd = 0.0f;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				OutStd += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(In[Idx] - OutMean);
				OutMean += (In[Idx] - OutMean) / (Idx + 1);
			}
			OutStd = FMath::Sqrt(OutStd);
		}

		static inline void AngularMean(float& OutMean, const TLearningArrayView<1, const float> In)
		{
			const int32 Num = In.Num();

			float MeanSin = 0.0f;
			float MeanCos = 0.0f;

			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				MeanSin += (FMath::Sin(In[Idx]) - MeanSin) / (Idx + 1);
				MeanCos += (FMath::Cos(In[Idx]) - MeanCos) / (Idx + 1);
			}

			OutMean = FMath::Atan2(MeanSin, MeanCos);
		}

		static inline void AngularMeanStd(float& OutMean, float& OutStd, const TLearningArrayView<1, const float> In)
		{
			AngularMean(OutMean, In);

			const int32 Num = In.Num();

			OutStd = 0.0f;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				OutStd += FMath::Square(FMath::FindDeltaAngleRadians(OutMean, In[Idx])) / Num;
			}
			OutStd = FMath::Sqrt(OutStd);
		}

		static inline void LogMeanStd(float& OutMean, float& OutLogStd, const TLearningArrayView<1, const float> In)
		{
			const int32 Num = In.Num();

			OutMean = 0.0f;
			OutLogStd = 0.0f;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				const float Value = FMath::Loge(FMath::Max(In[Idx], UE_SMALL_NUMBER));
				OutLogStd += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(Value - OutMean);
				OutMean += (Value - OutMean) / (Idx + 1);
			}
			OutMean = FMath::Exp(OutMean);
			OutLogStd = FMath::Sqrt(OutLogStd);
		}
		
		static inline void MinMax(float& OutMin, float& OutMax, const TLearningArrayView<1, const float> In)
		{
			const int32 Num = In.Num();

			OutMin = +UE_MAX_FLT;
			OutMax = -UE_MAX_FLT;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				OutMin = FMath::Min(OutMin, In[Idx]);
				OutMax = FMath::Max(OutMax, In[Idx]);
			}
		}

		static inline void LocationDistanceTraveled(
			float& OutDistance, 
			const TLearningArrayView<1, const float> InX,
			const TLearningArrayView<1, const float> InY, 
			const TLearningArrayView<1, const float> InZ)
		{
			check(InX.Num() == InY.Num());
			check(InX.Num() == InZ.Num());

			const int32 Num = InX.Num();

			OutDistance = 0.0f;
			for (int32 Idx = 0; Idx < Num - 1; Idx++)
			{
				OutDistance += FMath::Sqrt(
					FMath::Square(InX[Idx + 1] - InX[Idx + 0]) +
					FMath::Square(InY[Idx + 1] - InY[Idx + 0]) +
					FMath::Square(InZ[Idx + 1] - InZ[Idx + 0]));
			}
		}

		static inline void QuatAngleTraveled(
			float& OutAngle,
			const TLearningArrayView<1, const float> InX,
			const TLearningArrayView<1, const float> InY,
			const TLearningArrayView<1, const float> InZ,
			const TLearningArrayView<1, const float> InW)
		{
			check(InX.Num() == InY.Num());
			check(InX.Num() == InZ.Num());
			check(InX.Num() == InW.Num());

			const int32 Num = InX.Num();

			OutAngle = 0.0f;
			for (int32 Idx = 0; Idx < Num - 1; Idx++)
			{
				OutAngle += FQuat4f(
					InX[Idx + 1],
					InY[Idx + 1],
					InZ[Idx + 1],
					InW[Idx + 1]).AngularDistance(FQuat4f(
						InX[Idx + 0],
						InY[Idx + 0],
						InZ[Idx + 0],
						InW[Idx + 0]));
			}
		}

		static inline void QuatInv(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 4);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				const FQuat4f OutValue = InValue.Inverse();
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatAbs(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 4);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				const FQuat4f OutValue = InValue.GetShortestArcWith(FQuat4f::Identity);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatToRotationVector(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 3);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				const FVector3f OutValue = InValue.ToRotationVector();
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
			}
		}

		static inline void QuatFromRotationVector(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 4);
			check(In.Num<0>() == 3);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f InValue = FVector3f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx]);
				const FQuat4f OutValue = FQuat4f::MakeFromRotationVector(InValue);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		// See: https://theorangeduck.com/page/variations-muller
		static inline void QuatFromBasisDirections(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> InForwardX,
			const TLearningArrayView<1, const float> InForwardY,
			const TLearningArrayView<1, const float> InForwardZ,
			const TLearningArrayView<1, const float> InRightX,
			const TLearningArrayView<1, const float> InRightY,
			const TLearningArrayView<1, const float> InRightZ,
			const TLearningArrayView<1, const float> InUpX,
			const TLearningArrayView<1, const float> InUpY,
			const TLearningArrayView<1, const float> InUpZ,
			const int32 Iterations = 30,
			const float Eps = UE_SMALL_NUMBER)
		{
			const int32 ValueNum = OutX.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f C0 = FVector3f(InForwardX[ValueIdx], InForwardY[ValueIdx], InForwardZ[ValueIdx]);
				const FVector3f C1 = FVector3f(InRightX[ValueIdx], InRightY[ValueIdx], InRightZ[ValueIdx]);
				const FVector3f C2 = FVector3f(InUpX[ValueIdx], InUpY[ValueIdx], InUpZ[ValueIdx]);
				const float MLen = C0.Length() + C1.Length() + C2.Length() + Eps;

				FQuat4f Q = FQuat4f::Identity;

				for (int32 Iteration = 0; Iteration < Iterations; Iteration++)
				{
					const FVector3f R0 = Q.RotateVector(FVector3f::ForwardVector);
					const FVector3f R1 = Q.RotateVector(FVector3f::RightVector);
					const FVector3f R2 = Q.RotateVector(FVector3f::UpVector);

					const float W = MLen + R0.Dot(C0) + R1.Dot(C1) + R2.Dot(C2);
					const FVector3f V = R0.Cross(C0) + R1.Cross(C1) + R2.Cross(C2);

					Q = FQuat4f(V.X, V.Y, V.Z, W).GetNormalized() * Q;
				}

				OutX[ValueIdx] = Q.X;
				OutY[ValueIdx] = Q.Y;
				OutZ[ValueIdx] = Q.Z;
				OutW[ValueIdx] = Q.W;
			}
		}

		static inline void QuatFromPitchYawRoll(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> InPitch,
			const TLearningArrayView<1, const float> InYaw,
			const TLearningArrayView<1, const float> InRoll)
		{
			const int32 ValueNum = OutX.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f OutValue = FRotator3f(
					FMath::RadiansToDegrees(InPitch[ValueIdx]), 
					FMath::RadiansToDegrees(InYaw[ValueIdx]), 
					FMath::RadiansToDegrees(InRoll[ValueIdx])).Quaternion();

				OutX[ValueIdx] = OutValue.X;
				OutY[ValueIdx] = OutValue.Y;
				OutZ[ValueIdx] = OutValue.Z;
				OutW[ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatMul(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> LhsW,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ,
			const TLearningArrayView<1, const float> RhsW)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f Lhs = FQuat4f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx], LhsW[ValueIdx]);
				const FQuat4f Rhs = FQuat4f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx], RhsW[ValueIdx]);
				const FQuat4f Out = Lhs * Rhs;
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
				OutW[ValueIdx] = Out.W;
			}
		}

		static inline void QuatMulInv(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> LhsW,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ,
			const TLearningArrayView<1, const float> RhsW)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f Lhs = FQuat4f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx], LhsW[ValueIdx]);
				const FQuat4f Rhs = FQuat4f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx], RhsW[ValueIdx]);
				const FQuat4f Out = Lhs * Rhs.Inverse();
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
				OutW[ValueIdx] = Out.W;
			}
		}

		static inline void QuatInvMul(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> LhsW,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ,
			const TLearningArrayView<1, const float> RhsW)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f Lhs = FQuat4f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx], LhsW[ValueIdx]);
				const FQuat4f Rhs = FQuat4f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx], RhsW[ValueIdx]);
				const FQuat4f Out = Lhs.Inverse() * Rhs;
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
				OutW[ValueIdx] = Out.W;
			}
		}

		static inline void QuatRotate(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> LhsW,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f Lhs = FQuat4f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx], LhsW[ValueIdx]);
				const FVector3f Rhs = FVector3f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx]);
				const FVector3f Out = Lhs.RotateVector(Rhs);
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
			}
		}

		static inline void QuatUnrotate(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> LhsW,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f Lhs = FQuat4f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx], LhsW[ValueIdx]);
				const FVector3f Rhs = FVector3f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx]);
				const FVector3f Out = Lhs.UnrotateVector(Rhs);
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
			}
		}

		static inline void QuatBetween(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			TLearningArrayView<1, float> OutW,
			const TLearningArrayView<1, const float> LhsX,
			const TLearningArrayView<1, const float> LhsY,
			const TLearningArrayView<1, const float> LhsZ,
			const TLearningArrayView<1, const float> RhsX,
			const TLearningArrayView<1, const float> RhsY,
			const TLearningArrayView<1, const float> RhsZ)
		{
			const int32 ValueNum = OutX.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f Lhs = FVector3f(LhsX[ValueIdx], LhsY[ValueIdx], LhsZ[ValueIdx]);
				const FVector3f Rhs = FVector3f(RhsX[ValueIdx], RhsY[ValueIdx], RhsZ[ValueIdx]);
				const FQuat4f Out = FQuat4f::FindBetween(Lhs, Rhs);
				OutX[ValueIdx] = Out.X;
				OutY[ValueIdx] = Out.Y;
				OutZ[ValueIdx] = Out.Z;
				OutW[ValueIdx] = Out.W;
			}
		}

		static inline void QuatMulConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FQuat4f Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f LhsValue = FQuat4f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx], Lhs[3][ValueIdx]);
				const FQuat4f OutValue = LhsValue * Rhs;
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatInvMulConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FQuat4f Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f LhsValue = FQuat4f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx], Lhs[3][ValueIdx]);
				const FQuat4f OutValue = LhsValue.Inverse() * Rhs;
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatMulInvConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FQuat4f Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f LhsValue = FQuat4f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx], Lhs[3][ValueIdx]);
				const FQuat4f OutValue = LhsValue * Rhs.Inverse();
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatRotateConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FVector3f Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f LhsValue = FQuat4f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx], Lhs[3][ValueIdx]);
				const FVector3f OutValue = LhsValue.RotateVector(Rhs);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
			}
		}

		static inline void QuatUnrotateConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FVector3f Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f LhsValue = FQuat4f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx], Lhs[3][ValueIdx]);
				const FVector3f OutValue = LhsValue.RotateVector(Rhs);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
			}
		}

		static inline void QuatBetweenConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FVector3f Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Lhs.Num<0>() == 3);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f LhsValue = FVector3f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx]);
				const FQuat4f OutValue = FQuat4f::FindBetween(LhsValue, Rhs);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatApproxShortestAngleBetweenConstant(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> Lhs, const FQuat4f Rhs)
		{
			check(Out.Num<0>() == 1);
			check(Lhs.Num<0>() == 4);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const float DotProduct0 =
					Lhs[0][ValueIdx] * +Rhs.X +
					Lhs[1][ValueIdx] * +Rhs.Y +
					Lhs[2][ValueIdx] * +Rhs.Z +
					Lhs[3][ValueIdx] * +Rhs.W;

				const float DotProduct1 =
					Lhs[0][ValueIdx] * -Rhs.X +
					Lhs[1][ValueIdx] * -Rhs.Y +
					Lhs[2][ValueIdx] * -Rhs.Z +
					Lhs[3][ValueIdx] * -Rhs.W;

				Out[0][ValueIdx] = ApproxAcos(FMath::Max(DotProduct0, DotProduct1));
			}
		}

		static inline void QuatConstantMul(TLearningArrayView<2, float> Out, const FQuat4f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Rhs.Num<0>() == 4);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f RhsValue = FQuat4f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx], Rhs[3][ValueIdx]);
				const FQuat4f OutValue = Lhs * RhsValue;
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatConstantInvMul(TLearningArrayView<2, float> Out, const FQuat4f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Rhs.Num<0>() == 4);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f RhsValue = FQuat4f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx], Rhs[3][ValueIdx]);
				const FQuat4f OutValue = Lhs.Inverse() * RhsValue;
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatConstantMulInv(TLearningArrayView<2, float> Out, const FQuat4f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Rhs.Num<0>() == 4);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f RhsValue = FQuat4f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx], Rhs[3][ValueIdx]);
				const FQuat4f OutValue = Lhs * RhsValue.Inverse();
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatConstantRotate(TLearningArrayView<2, float> Out, const FQuat4f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f RhsValue = FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]);
				const FVector3f OutValue = Lhs.RotateVector(RhsValue);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
			}
		}

		static inline void QuatConstantUnrotate(TLearningArrayView<2, float> Out, const FQuat4f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f RhsValue = FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]);
				const FVector3f OutValue = Lhs.UnrotateVector(RhsValue);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
			}
		}

		static inline void QuatConstantBetween(TLearningArrayView<2, float> Out, const FVector3f Lhs, const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 4);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f RhsValue = FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]);
				const FQuat4f OutValue = FQuat4f::FindBetween(Lhs, RhsValue);
				Out[0][ValueIdx] = OutValue.X;
				Out[1][ValueIdx] = OutValue.Y;
				Out[2][ValueIdx] = OutValue.Z;
				Out[3][ValueIdx] = OutValue.W;
			}
		}

		static inline void QuatPitch(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 1);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				Out[0][ValueIdx] = FMath::DegreesToRadians(InValue.Rotator().Pitch);
			}
		}

		static inline void QuatYaw(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 1);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				Out[0][ValueIdx] = FMath::DegreesToRadians(InValue.Rotator().Yaw);
			}
		}

		static inline void QuatRoll(TLearningArrayView<2, float> Out, const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 1);
			check(In.Num<0>() == 4);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = Out.Num<1>();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FQuat4f InValue = FQuat4f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx], In[3][ValueIdx]);
				Out[0][ValueIdx] = FMath::DegreesToRadians(InValue.Rotator().Roll);
			}
		}

		static inline void WrapAngleInplace(TLearningArrayView<1, float> InOut)
		{
			const int32 ValueNum = InOut.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				InOut[ValueIdx] = FMath::Wrap(InOut[ValueIdx], -UE_PI, UE_PI);
			}
		}

		static inline void AngleToDirection3D(
			TLearningArrayView<1, float> OutX,
			TLearningArrayView<1, float> OutY,
			TLearningArrayView<1, float> OutZ,
			const TLearningArrayView<1, const float> In)
		{
			const int32 ValueNum = In.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				OutX[ValueIdx] = FMath::Cos(In[ValueIdx]);
				OutY[ValueIdx] = FMath::Sin(In[ValueIdx]);
				OutZ[ValueIdx] = 0.0f;
			}
		}

		static inline void Direction3DToAngle(
			TLearningArrayView<1, float> Out,
			const TLearningArrayView<1, const float> InX,
			const TLearningArrayView<1, const float> InY)
		{
			const int32 ValueNum = Out.Num();
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				Out[ValueIdx] = FMath::Atan2(InY[ValueIdx], InX[ValueIdx]);
			}
		}

		static inline void InertializationDistanceConstant(
			TLearningArrayView<1, float> OutValues,
			const TLearningArrayView<1, const float> Locations,
			const TLearningArrayView<1, const float> Velocities,
			const float Location,
			const float Velocity,
			const float BlendTime,
			const float LocWeight,
			const float VelWeight)
		{
			check(OutValues.Num() == Locations.Num());
			check(OutValues.Num() == Velocities.Num());

#if UE_LEARNING_ISPC
			ispc::LearningInertializationDistanceConstant(
				OutValues.GetData(),
				Locations.GetData(),
				Velocities.GetData(),
				OutValues.Num(),
				Location,
				Velocity,
				BlendTime,
				LocWeight,
				VelWeight);
#else
			const int32 ValueNum = OutValues.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const float Integral = InertializationIntegralMagnitude(Locations[ValueIdx] - Location, Velocities[ValueIdx] - Velocity, BlendTime);
				const float MaxVelocity = InertializationMaximumVelocityMagnitude(Locations[ValueIdx] - Location, Velocities[ValueIdx] - Velocity, BlendTime);

				OutValues[ValueIdx] = LocWeight * Integral + VelWeight * MaxVelocity;
			}
#endif
		}

		// We need these custom Vector4 functions because the defaults for the FVector4f class will treat it like a Vector3

		static inline float Vector4Length(const FVector4f X)
		{
			return FMath::Sqrt(X.X * X.X + X.Y * X.Y + X.Z * X.Z + X.W * X.W);
		}

		static inline FVector4f Vector4Normalize(const FVector4f X)
		{
			return X / Vector4Length(X);
		}

		static inline FVector4f DominantEigenVector(
			const FMatrix44f& A,
			const FVector4f V0)
		{
			// Initial Guess at Eigen Vector
			FVector4f V = V0;

			for (int32 Iteration = 0; Iteration < 20; Iteration++)
			{
				// Power Iteration
				const FVector4f Av = A.TransformFVector4(V);

				// Next Guess at Eigen Vector
				V = Vector4Normalize(Av);
			}

			return V;
		}

		static inline void QuatMeanStd(FQuat4f& OutMean, FVector3f& OutStd, const TLearningArrayView<2, const float> In)
		{
			check(In.Num<0>() == 4);

			const int32 Num = In.Num<1>();

			FMatrix44f Accum;
			FPlatformMemory::Memzero(&Accum, sizeof(FMatrix44f));

			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				const FQuat4f Q = FQuat4f(In[0][Idx], In[1][Idx], In[2][Idx], In[3][Idx]);

				// make Q^T * Q matrix and accumulate using online mean algorithm :

				Accum.M[0][0] += ((Q.X * Q.X) - Accum.M[0][0]) / (Idx + 1);
				Accum.M[0][1] += ((Q.X * Q.Y) - Accum.M[0][1]) / (Idx + 1);
				Accum.M[0][2] += ((Q.X * Q.Z) - Accum.M[0][2]) / (Idx + 1);
				Accum.M[0][3] += ((Q.X * Q.W) - Accum.M[0][3]) / (Idx + 1);

				Accum.M[1][0] += ((Q.Y * Q.X) - Accum.M[1][0]) / (Idx + 1);
				Accum.M[1][1] += ((Q.Y * Q.Y) - Accum.M[1][1]) / (Idx + 1);
				Accum.M[1][2] += ((Q.Y * Q.Z) - Accum.M[1][2]) / (Idx + 1);
				Accum.M[1][3] += ((Q.Y * Q.W) - Accum.M[1][3]) / (Idx + 1);

				Accum.M[2][0] += ((Q.Z * Q.X) - Accum.M[2][0]) / (Idx + 1);
				Accum.M[2][1] += ((Q.Z * Q.Y) - Accum.M[2][1]) / (Idx + 1);
				Accum.M[2][2] += ((Q.Z * Q.Z) - Accum.M[2][2]) / (Idx + 1);
				Accum.M[2][3] += ((Q.Z * Q.W) - Accum.M[2][3]) / (Idx + 1);

				Accum.M[3][0] += ((Q.W * Q.X) - Accum.M[3][0]) / (Idx + 1);
				Accum.M[3][1] += ((Q.W * Q.Y) - Accum.M[3][1]) / (Idx + 1);
				Accum.M[3][2] += ((Q.W * Q.Z) - Accum.M[3][2]) / (Idx + 1);
				Accum.M[3][3] += ((Q.W * Q.W) - Accum.M[3][3]) / (Idx + 1);
			}

			const FVector4f AverageQuat = DominantEigenVector(Accum, FVector4f(0.0f, 0.0f, 0.0f, 1.0f) );
			OutMean = FQuat4f(AverageQuat.X, AverageQuat.Y, AverageQuat.Z, AverageQuat.W);
			check(OutMean.IsNormalized());

			OutStd = FVector3f::ZeroVector;
			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				const FQuat4f Q = FQuat4f(In[0][Idx], In[1][Idx], In[2][Idx], In[3][Idx]);
				const FQuat4f Diff = (Q * OutMean.Inverse()).GetShortestArcWith(FQuat4f::Identity);
				OutStd += FMath::Square(Diff.ToRotationVector()) / Num;
			}
			OutStd.X = FMath::Sqrt(OutStd.X);
			OutStd.Y = FMath::Sqrt(OutStd.Y);
			OutStd.Z = FMath::Sqrt(OutStd.Z);
		}


		static inline void TransformInv(
			TLearningArrayView<2, float> Out,
			const TLearningArrayView<2, const float> In)
		{
			check(Out.Num<0>() == 10);
			check(In.Num<0>() == 10);
			check(Out.Num<1>() == In.Num<1>());

			const int32 ValueNum = In.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f T = FTransform3f(
					FQuat4f(In[3][ValueIdx], In[4][ValueIdx], In[5][ValueIdx], In[6][ValueIdx]),
					FVector3f(In[0][ValueIdx], In[1][ValueIdx], In[2][ValueIdx]),
					FVector3f(In[7][ValueIdx], In[8][ValueIdx], In[9][ValueIdx])).Inverse();

				Out[0][ValueIdx] = T.GetLocation().X;
				Out[1][ValueIdx] = T.GetLocation().Y;
				Out[2][ValueIdx] = T.GetLocation().Z;
				Out[3][ValueIdx] = T.GetRotation().X;
				Out[4][ValueIdx] = T.GetRotation().Y;
				Out[5][ValueIdx] = T.GetRotation().Z;
				Out[6][ValueIdx] = T.GetRotation().W;
				Out[7][ValueIdx] = T.GetScale3D().X;
				Out[8][ValueIdx] = T.GetScale3D().Y;
				Out[9][ValueIdx] = T.GetScale3D().Z;
			}
		}

		static inline void TransformApplyConstantLocation(
			TLearningArrayView<2, float> Out,
			const TLearningArrayView<2, const float> Lhs,
			const FVector3f& Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Lhs.Num<0>() == 10);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Lhs.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f V = FTransform3f(
					FQuat4f(Lhs[3][ValueIdx], Lhs[4][ValueIdx], Lhs[5][ValueIdx], Lhs[6][ValueIdx]),
					FVector3f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx]),
					FVector3f(Lhs[7][ValueIdx], Lhs[8][ValueIdx], Lhs[9][ValueIdx])).TransformPosition(Rhs);

				Out[0][ValueIdx] = V.X;
				Out[1][ValueIdx] = V.Y;
				Out[2][ValueIdx] = V.Z;
			}
		}

		static inline void TransformApplyConstantDirection(
			TLearningArrayView<2, float> Out,
			const TLearningArrayView<2, const float> Lhs,
			const FVector3f& Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Lhs.Num<0>() == 10);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Lhs.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f V = FTransform3f(
					FQuat4f(Lhs[3][ValueIdx], Lhs[4][ValueIdx], Lhs[5][ValueIdx], Lhs[6][ValueIdx]),
					FVector3f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx]),
					FVector3f(Lhs[7][ValueIdx], Lhs[8][ValueIdx], Lhs[9][ValueIdx])).TransformVectorNoScale(Rhs);

				Out[0][ValueIdx] = V.X;
				Out[1][ValueIdx] = V.Y;
				Out[2][ValueIdx] = V.Z;
			}
		}

		static inline void TransformConstantApply(
			TLearningArrayView<2, float> Out,
			const FTransform3f& Lhs,
			const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Rhs.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f V = Lhs.TransformPosition(FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]));

				Out[0][ValueIdx] = V.X;
				Out[1][ValueIdx] = V.Y;
				Out[2][ValueIdx] = V.Z;
			}
		}

		static inline void TransformConstantApplyNoTranslation(
			TLearningArrayView<2, float> Out,
			const FTransform3f& Lhs,
			const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Rhs.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f V = Lhs.TransformVector(FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]));

				Out[0][ValueIdx] = V.X;
				Out[1][ValueIdx] = V.Y;
				Out[2][ValueIdx] = V.Z;
			}
		}

		static inline void TransformConstantApplyNoTranslationScale(
			TLearningArrayView<2, float> Out,
			const FTransform3f& Lhs,
			const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 3);
			check(Rhs.Num<0>() == 3);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Rhs.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FVector3f V = Lhs.TransformVectorNoScale(FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]));

				Out[0][ValueIdx] = V.X;
				Out[1][ValueIdx] = V.Y;
				Out[2][ValueIdx] = V.Z;
			}
		}

		static inline void TransformMul(
			TLearningArrayView<1, float> OutPosX,
			TLearningArrayView<1, float> OutPosY,
			TLearningArrayView<1, float> OutPosZ,
			TLearningArrayView<1, float> OutRotX,
			TLearningArrayView<1, float> OutRotY,
			TLearningArrayView<1, float> OutRotZ,
			TLearningArrayView<1, float> OutRotW,
			TLearningArrayView<1, float> OutSclX,
			TLearningArrayView<1, float> OutSclY,
			TLearningArrayView<1, float> OutSclZ,
			const TLearningArrayView<1, const float> LhsPosX,
			const TLearningArrayView<1, const float> LhsPosY,
			const TLearningArrayView<1, const float> LhsPosZ,
			const TLearningArrayView<1, const float> LhsRotX,
			const TLearningArrayView<1, const float> LhsRotY,
			const TLearningArrayView<1, const float> LhsRotZ,
			const TLearningArrayView<1, const float> LhsRotW,
			const TLearningArrayView<1, const float> LhsSclX,
			const TLearningArrayView<1, const float> LhsSclY,
			const TLearningArrayView<1, const float> LhsSclZ,
			const TLearningArrayView<1, const float> RhsPosX,
			const TLearningArrayView<1, const float> RhsPosY,
			const TLearningArrayView<1, const float> RhsPosZ,
			const TLearningArrayView<1, const float> RhsRotX,
			const TLearningArrayView<1, const float> RhsRotY,
			const TLearningArrayView<1, const float> RhsRotZ,
			const TLearningArrayView<1, const float> RhsRotW,
			const TLearningArrayView<1, const float> RhsSclX,
			const TLearningArrayView<1, const float> RhsSclY,
			const TLearningArrayView<1, const float> RhsSclZ)
		{
			const int32 ValueNum = OutPosX.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f L = FTransform3f(
					FQuat4f(LhsRotX[ValueIdx], LhsRotY[ValueIdx], LhsRotZ[ValueIdx], LhsRotW[ValueIdx]),
					FVector3f(LhsPosX[ValueIdx], LhsPosY[ValueIdx], LhsPosZ[ValueIdx]),
					FVector3f(LhsSclX[ValueIdx], LhsSclY[ValueIdx], LhsSclZ[ValueIdx]));

				const FTransform3f R = FTransform3f(
					FQuat4f(RhsRotX[ValueIdx], RhsRotY[ValueIdx], RhsRotZ[ValueIdx], RhsRotW[ValueIdx]),
					FVector3f(RhsPosX[ValueIdx], RhsPosY[ValueIdx], RhsPosZ[ValueIdx]),
					FVector3f(RhsSclX[ValueIdx], RhsSclY[ValueIdx], RhsSclZ[ValueIdx]));

				const FTransform3f O = L * R;

				OutPosX[ValueIdx] = O.GetLocation().X;
				OutPosY[ValueIdx] = O.GetLocation().Y;
				OutPosZ[ValueIdx] = O.GetLocation().Z;
				OutRotX[ValueIdx] = O.GetRotation().X;
				OutRotY[ValueIdx] = O.GetRotation().Y;
				OutRotZ[ValueIdx] = O.GetRotation().Z;
				OutRotW[ValueIdx] = O.GetRotation().W;
				OutSclX[ValueIdx] = O.GetScale3D().X;
				OutSclY[ValueIdx] = O.GetScale3D().Y;
				OutSclZ[ValueIdx] = O.GetScale3D().Z;
			}
		}
		static inline void TransformDiv(
			TLearningArrayView<1, float> OutPosX,
			TLearningArrayView<1, float> OutPosY,
			TLearningArrayView<1, float> OutPosZ,
			TLearningArrayView<1, float> OutRotX,
			TLearningArrayView<1, float> OutRotY,
			TLearningArrayView<1, float> OutRotZ,
			TLearningArrayView<1, float> OutRotW,
			TLearningArrayView<1, float> OutSclX,
			TLearningArrayView<1, float> OutSclY,
			TLearningArrayView<1, float> OutSclZ,
			const TLearningArrayView<1, const float> LhsPosX,
			const TLearningArrayView<1, const float> LhsPosY,
			const TLearningArrayView<1, const float> LhsPosZ,
			const TLearningArrayView<1, const float> LhsRotX,
			const TLearningArrayView<1, const float> LhsRotY,
			const TLearningArrayView<1, const float> LhsRotZ,
			const TLearningArrayView<1, const float> LhsRotW,
			const TLearningArrayView<1, const float> LhsSclX,
			const TLearningArrayView<1, const float> LhsSclY,
			const TLearningArrayView<1, const float> LhsSclZ,
			const TLearningArrayView<1, const float> RhsPosX,
			const TLearningArrayView<1, const float> RhsPosY,
			const TLearningArrayView<1, const float> RhsPosZ,
			const TLearningArrayView<1, const float> RhsRotX,
			const TLearningArrayView<1, const float> RhsRotY,
			const TLearningArrayView<1, const float> RhsRotZ,
			const TLearningArrayView<1, const float> RhsRotW,
			const TLearningArrayView<1, const float> RhsSclX,
			const TLearningArrayView<1, const float> RhsSclY,
			const TLearningArrayView<1, const float> RhsSclZ)
		{
			const int32 ValueNum = OutPosX.Num();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f L = FTransform3f(
					FQuat4f(LhsRotX[ValueIdx], LhsRotY[ValueIdx], LhsRotZ[ValueIdx], LhsRotW[ValueIdx]),
					FVector3f(LhsPosX[ValueIdx], LhsPosY[ValueIdx], LhsPosZ[ValueIdx]),
					FVector3f(LhsSclX[ValueIdx], LhsSclY[ValueIdx], LhsSclZ[ValueIdx]));

				const FTransform3f R = FTransform3f(
					FQuat4f(RhsRotX[ValueIdx], RhsRotY[ValueIdx], RhsRotZ[ValueIdx], RhsRotW[ValueIdx]),
					FVector3f(RhsPosX[ValueIdx], RhsPosY[ValueIdx], RhsPosZ[ValueIdx]),
					FVector3f(RhsSclX[ValueIdx], RhsSclY[ValueIdx], RhsSclZ[ValueIdx]));

				const FTransform3f O = L * R.Inverse();

				OutPosX[ValueIdx] = O.GetLocation().X;
				OutPosY[ValueIdx] = O.GetLocation().Y;
				OutPosZ[ValueIdx] = O.GetLocation().Z;
				OutRotX[ValueIdx] = O.GetRotation().X;
				OutRotY[ValueIdx] = O.GetRotation().Y;
				OutRotZ[ValueIdx] = O.GetRotation().Z;
				OutRotW[ValueIdx] = O.GetRotation().W;
				OutSclX[ValueIdx] = O.GetScale3D().X;
				OutSclY[ValueIdx] = O.GetScale3D().Y;
				OutSclZ[ValueIdx] = O.GetScale3D().Z;
			}
		}

		static inline void TransformMulConstant(
			TLearningArrayView<2, float> Out,
			const TLearningArrayView<2, const float> Lhs,
			const FTransform3f Rhs)
		{
			check(Out.Num<0>() == 10);
			check(Lhs.Num<0>() == 10);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f L = FTransform3f(
					FQuat4f(Lhs[3][ValueIdx], Lhs[4][ValueIdx], Lhs[5][ValueIdx], Lhs[6][ValueIdx]),
					FVector3f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx]),
					FVector3f(Lhs[7][ValueIdx], Lhs[8][ValueIdx], Lhs[9][ValueIdx]));

				const FTransform3f O = L * Rhs;

				Out[0][ValueIdx] = O.GetLocation().X;
				Out[1][ValueIdx] = O.GetLocation().Y;
				Out[2][ValueIdx] = O.GetLocation().Z;
				Out[3][ValueIdx] = O.GetRotation().X;
				Out[4][ValueIdx] = O.GetRotation().Y;
				Out[5][ValueIdx] = O.GetRotation().Z;
				Out[6][ValueIdx] = O.GetRotation().W;
				Out[7][ValueIdx] = O.GetScale3D().X;
				Out[8][ValueIdx] = O.GetScale3D().Y;
				Out[9][ValueIdx] = O.GetScale3D().Z;
			}
		}

		static inline void TransformDivConstant(
			TLearningArrayView<2, float> Out,
			const TLearningArrayView<2, const float> Lhs,
			const FTransform3f Rhs)
		{
			check(Out.Num<0>() == 10);
			check(Lhs.Num<0>() == 10);
			check(Out.Num<1>() == Lhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f L = FTransform3f(
					FQuat4f(Lhs[3][ValueIdx], Lhs[4][ValueIdx], Lhs[5][ValueIdx], Lhs[6][ValueIdx]),
					FVector3f(Lhs[0][ValueIdx], Lhs[1][ValueIdx], Lhs[2][ValueIdx]),
					FVector3f(Lhs[7][ValueIdx], Lhs[8][ValueIdx], Lhs[9][ValueIdx]));

				const FTransform3f O = L * Rhs.Inverse();

				Out[0][ValueIdx] = O.GetLocation().X;
				Out[1][ValueIdx] = O.GetLocation().Y;
				Out[2][ValueIdx] = O.GetLocation().Z;
				Out[3][ValueIdx] = O.GetRotation().X;
				Out[4][ValueIdx] = O.GetRotation().Y;
				Out[5][ValueIdx] = O.GetRotation().Z;
				Out[6][ValueIdx] = O.GetRotation().W;
				Out[7][ValueIdx] = O.GetScale3D().X;
				Out[8][ValueIdx] = O.GetScale3D().Y;
				Out[9][ValueIdx] = O.GetScale3D().Z;
			}
		}

		static inline void TransformConstantMul(
			TLearningArrayView<2, float> Out,
			const FTransform3f Lhs,
			const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 10);
			check(Rhs.Num<0>() == 10);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f R = FTransform3f(
					FQuat4f(Rhs[3][ValueIdx], Rhs[4][ValueIdx], Rhs[5][ValueIdx], Rhs[6][ValueIdx]),
					FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]),
					FVector3f(Rhs[7][ValueIdx], Rhs[8][ValueIdx], Rhs[9][ValueIdx]));

				const FTransform3f O = Lhs * R;

				Out[0][ValueIdx] = O.GetLocation().X;
				Out[1][ValueIdx] = O.GetLocation().Y;
				Out[2][ValueIdx] = O.GetLocation().Z;
				Out[3][ValueIdx] = O.GetRotation().X;
				Out[4][ValueIdx] = O.GetRotation().Y;
				Out[5][ValueIdx] = O.GetRotation().Z;
				Out[6][ValueIdx] = O.GetRotation().W;
				Out[7][ValueIdx] = O.GetScale3D().X;
				Out[8][ValueIdx] = O.GetScale3D().Y;
				Out[9][ValueIdx] = O.GetScale3D().Z;
			}
		}

		static inline void TransformConstantDiv(
			TLearningArrayView<2, float> Out,
			const FTransform3f Lhs,
			const TLearningArrayView<2, const float> Rhs)
		{
			check(Out.Num<0>() == 10);
			check(Rhs.Num<0>() == 10);
			check(Out.Num<1>() == Rhs.Num<1>());

			const int32 ValueNum = Out.Num<1>();

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				const FTransform3f R = FTransform3f(
					FQuat4f(Rhs[3][ValueIdx], Rhs[4][ValueIdx], Rhs[5][ValueIdx], Rhs[6][ValueIdx]),
					FVector3f(Rhs[0][ValueIdx], Rhs[1][ValueIdx], Rhs[2][ValueIdx]),
					FVector3f(Rhs[7][ValueIdx], Rhs[8][ValueIdx], Rhs[9][ValueIdx]));

				const FTransform3f O = Lhs * R.Inverse();

				Out[0][ValueIdx] = O.GetLocation().X;
				Out[1][ValueIdx] = O.GetLocation().Y;
				Out[2][ValueIdx] = O.GetLocation().Z;
				Out[3][ValueIdx] = O.GetRotation().X;
				Out[4][ValueIdx] = O.GetRotation().Y;
				Out[5][ValueIdx] = O.GetRotation().Z;
				Out[6][ValueIdx] = O.GetRotation().W;
				Out[7][ValueIdx] = O.GetScale3D().X;
				Out[8][ValueIdx] = O.GetScale3D().Y;
				Out[9][ValueIdx] = O.GetScale3D().Z;
			}
		}

		static inline void TransformMake(
			TLearningArrayView<1, float> OutPosX,
			TLearningArrayView<1, float> OutPosY,
			TLearningArrayView<1, float> OutPosZ,
			TLearningArrayView<1, float> OutRotX,
			TLearningArrayView<1, float> OutRotY,
			TLearningArrayView<1, float> OutRotZ,
			TLearningArrayView<1, float> OutRotW,
			TLearningArrayView<1, float> OutSclX,
			TLearningArrayView<1, float> OutSclY,
			TLearningArrayView<1, float> OutSclZ,
			const TLearningArrayView<1, const float> InPosX,
			const TLearningArrayView<1, const float> InPosY,
			const TLearningArrayView<1, const float> InPosZ,
			const TLearningArrayView<1, const float> InRotX,
			const TLearningArrayView<1, const float> InRotY,
			const TLearningArrayView<1, const float> InRotZ,
			const TLearningArrayView<1, const float> InRotW)
		{
			Private::Copy(OutPosX, InPosX);
			Private::Copy(OutPosY, InPosY);
			Private::Copy(OutPosZ, InPosZ);

			Private::Copy(OutRotX, InRotX);
			Private::Copy(OutRotY, InRotY);
			Private::Copy(OutRotZ, InRotZ);
			Private::Copy(OutRotW, InRotW);

			Private::Set(OutSclX, 1.0f);
			Private::Set(OutSclY, 1.0f);
			Private::Set(OutSclZ, 1.0f);
		}

		static inline void TransformMake(
			TLearningArrayView<1, float> OutPosX,
			TLearningArrayView<1, float> OutPosY,
			TLearningArrayView<1, float> OutPosZ,
			TLearningArrayView<1, float> OutRotX,
			TLearningArrayView<1, float> OutRotY,
			TLearningArrayView<1, float> OutRotZ,
			TLearningArrayView<1, float> OutRotW,
			TLearningArrayView<1, float> OutSclX,
			TLearningArrayView<1, float> OutSclY,
			TLearningArrayView<1, float> OutSclZ,
			const TLearningArrayView<1, const float> InPosX,
			const TLearningArrayView<1, const float> InPosY,
			const TLearningArrayView<1, const float> InPosZ,
			const TLearningArrayView<1, const float> InRotX,
			const TLearningArrayView<1, const float> InRotY,
			const TLearningArrayView<1, const float> InRotZ,
			const TLearningArrayView<1, const float> InRotW,
			const TLearningArrayView<1, const float> InSclX,
			const TLearningArrayView<1, const float> InSclY,
			const TLearningArrayView<1, const float> InSclZ)
		{
			Private::Copy(OutPosX, InPosX);
			Private::Copy(OutPosY, InPosY);
			Private::Copy(OutPosZ, InPosZ);

			Private::Copy(OutRotX, InRotX);
			Private::Copy(OutRotY, InRotY);
			Private::Copy(OutRotZ, InRotZ);
			Private::Copy(OutRotW, InRotW);

			Private::Copy(OutSclX, InSclX);
			Private::Copy(OutSclY, InSclY);
			Private::Copy(OutSclZ, InSclZ);
		}

	}

	void FFrameAttribute::Check() const
	{
		FrameRangeSet.Check();
		check(AttributeData.Num<1>() == FrameRangeSet.GetTotalFrameNum());
	}

	void FFrameAttribute::Empty() { FrameRangeSet.Empty(); AttributeData.Empty(); }
	bool FFrameAttribute::IsEmpty() const { return FrameRangeSet.IsEmpty(); }

	const FFrameRangeSet& FFrameAttribute::GetFrameRangeSet() const { return FrameRangeSet; }
	int32 FFrameAttribute::GetTotalFrameNum() const { return AttributeData.Num<1>(); }
	int32 FFrameAttribute::GetTotalRangeNum() const { return FrameRangeSet.GetTotalRangeNum(); }
	int32 FFrameAttribute::GetChannelNum() const { return AttributeData.Num<0>(); }

	TLearningArrayView<2, const float> FFrameAttribute::GetAttributeData() const { return AttributeData; }

	TLearningArrayView<1, const float> FFrameAttribute::GetChannelAttributeData(const int32 ChannelIdx) const { return AttributeData[ChannelIdx]; }

	const float& FFrameAttribute::GetChannelAttributeDataAtFrame(const int32 ChannelIdx, const int32 RangeFrameIdx) const { return AttributeData[ChannelIdx][RangeFrameIdx]; }

	TLearningArrayView<1, const float> FFrameAttribute::GetChannelEntryRangeAttributeData(const int32 ChannelIdx, const int32 EntryIdx, const int32 RangeIdx) const
	{
		return AttributeData[ChannelIdx].Slice(FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx), FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx));
	}

	TLearningArrayView<1, const float> FFrameAttribute::GetChannelRangeAttributeData(const int32 ChannelIdx, const int32 RangeOffset, const int32 RangeLength) const
	{
		return AttributeData[ChannelIdx].Slice(RangeOffset, RangeLength);
	}

	TLearningArrayView<2, float> FFrameAttribute::GetAttributeData() { return AttributeData; }

	TLearningArrayView<1, float> FFrameAttribute::GetChannelAttributeData(const int32 ChannelIdx) { return AttributeData[ChannelIdx]; }

	float& FFrameAttribute::GetChannelAttributeDataAtFrame(const int32 ChannelIdx, const int32 RangeFrameIdx) { return AttributeData[ChannelIdx][RangeFrameIdx]; }

	TLearningArrayView<1, float> FFrameAttribute::GetChannelEntryRangeAttributeData(const int32 ChannelIdx, const int32 EntryIdx, const int32 RangeIdx)
	{
		return AttributeData[ChannelIdx].Slice(FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx), FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx));
	}

	TLearningArrayView<1, float> FFrameAttribute::GetChannelRangeAttributeData(const int32 ChannelIdx, const int32 RangeOffset, const int32 RangeLength)
	{
		return AttributeData[ChannelIdx].Slice(RangeOffset, RangeLength);
	}

	namespace FrameAttribute
	{
		bool Equal(const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			return FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet) && Private::Equal(Lhs.AttributeData.Flatten(), Rhs.AttributeData.Flatten());
		}

		void Intersection(FFrameAttribute& OutFrameAttribute, const FFrameAttribute& FrameAttribute, const FFrameRangeSet& FrameRangeSet)
		{
			if (FrameRangeSet::Equal(FrameAttribute.FrameRangeSet, FrameRangeSet))
			{
				OutFrameAttribute = FrameAttribute;
				return;
			}

			// Perform Intersection and get offsets
			TLearningArray<1, int32> LhsOffsets;
			TLearningArray<1, int32> RhsOffsets;
			LhsOffsets.SetNumUninitialized({ FrameAttribute.FrameRangeSet.GetTotalRangeNum() + FrameRangeSet.GetTotalRangeNum() });
			RhsOffsets.SetNumUninitialized({ FrameAttribute.FrameRangeSet.GetTotalRangeNum() + FrameRangeSet.GetTotalRangeNum() });

			const int32 OutTotalRangeNum = UE::Learning::FrameRangeSet::IntersectionWithOffsets(
				OutFrameAttribute.FrameRangeSet,
				LhsOffsets,
				RhsOffsets,
				FrameAttribute.FrameRangeSet,
				FrameRangeSet);

			// Resize back to correct size
			LhsOffsets.SetNumUninitialized({ OutTotalRangeNum });
			RhsOffsets.SetNumUninitialized({ OutTotalRangeNum });

			const int32 ChannelNum = FrameAttribute.GetChannelNum();
			const int32 TotalFrameNum = OutFrameAttribute.FrameRangeSet.GetTotalFrameNum();

			OutFrameAttribute.AttributeData.SetNumUninitialized({ ChannelNum, TotalFrameNum });

			for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
			{
				for (int32 RangeIdx = 0; RangeIdx < OutTotalRangeNum; RangeIdx++)
				{
					const int32 OutOffset = OutFrameAttribute.FrameRangeSet.GetAllRangeOffsets()[RangeIdx];
					const int32 LhsOffset = LhsOffsets[RangeIdx];
					const int32 Length = OutFrameAttribute.FrameRangeSet.GetAllRangeLengths()[RangeIdx];

					Private::Copy(
						OutFrameAttribute.GetChannelRangeAttributeData(ChannelIdx, OutOffset, Length),
						FrameAttribute.GetChannelRangeAttributeData(ChannelIdx, LhsOffset, Length));
				}
			}
		}

		void NonZeroFrameRangeSet(FFrameRangeSet& OutFrameRangeSet, const FFrameAttribute& FrameAttribute, const int32 ChannelIdx)
		{
			check(ChannelIdx >= 0 && ChannelIdx < FrameAttribute.GetChannelNum());

			const int32 EntryNum = FrameAttribute.FrameRangeSet.GetEntryNum();

			OutFrameRangeSet.Empty();

			TArray<int32> AddedRangeStarts;
			TArray<int32> AddedRangeLengths;

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameAttribute.FrameRangeSet.GetEntryRangeNum(EntryIdx);
				const int32 Sequence = FrameAttribute.FrameRangeSet.GetEntrySequence(EntryIdx);

				AddedRangeStarts.Reset();
				AddedRangeLengths.Reset();

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 FrameNum = FrameAttribute.FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
					const int32 StartFrame = FrameAttribute.FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);
					const int32 FrameOffset = FrameAttribute.FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);

					int32 StartFrameIndex = INDEX_NONE;

					for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
					{
						const float FrameValue = FrameAttribute.GetChannelAttributeDataAtFrame(ChannelIdx, FrameOffset + FrameIdx);

						if (StartFrameIndex == INDEX_NONE && FrameValue == 1.0f)
						{
							StartFrameIndex = FrameIdx;
						}
						else if (StartFrameIndex == INDEX_NONE && FrameValue == 0.0f) {}
						else if (StartFrameIndex != INDEX_NONE && FrameValue == 1.0f) {}
						else if (StartFrameIndex != INDEX_NONE && FrameValue == 0.0f)
						{
							check(FrameIdx - StartFrameIndex > 0);
							AddedRangeStarts.Add(StartFrame + StartFrameIndex);
							AddedRangeLengths.Add(FrameIdx - StartFrameIndex);
							StartFrameIndex = INDEX_NONE;
						}
					}

					if (StartFrameIndex != INDEX_NONE)
					{
						check(FrameNum - StartFrameIndex > 0);
						AddedRangeStarts.Add(StartFrame + StartFrameIndex);
						AddedRangeLengths.Add(FrameNum - StartFrameIndex);
					}
				}

				OutFrameRangeSet.AddEntry(Sequence, AddedRangeStarts, AddedRangeLengths);
			}
		}

		void Concat(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> In)
		{
			const int32 Num = In.Num();

			int32 TotalChannelNum = 0;
			for (int32 Idx = 0; Idx < Num; Idx++)
			{
				TotalChannelNum += In[Idx]->GetChannelNum();
			}

			NaryOp(Out, TotalChannelNum, In, [Num](
				FFrameAttribute& Out,
				const TArrayView<const ConstFrameAttributePtr> In,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					int32 ChannelOffset = 0;

					for (int32 Idx = 0; Idx < Num; Idx++)
					{
						const int32 ChannelNum = In[Idx]->GetChannelNum();

						for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
						{
							for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
							{
								Private::Copy(
									Out.GetChannelRangeAttributeData(ChannelOffset + ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
									In[Idx]->GetChannelRangeAttributeData(ChannelIdx, InRangeOffsets[Idx][RangeIdx], RangeLengths[RangeIdx]));
							}
						}

						ChannelOffset += ChannelNum;
					}

					check(ChannelOffset == Out.GetChannelNum());
				});
		}

		void ReduceOp(
			const FFrameAttribute& In,
			const ReduceOpFunction Op)
		{
			Op(In, In.FrameRangeSet.GetAllRangeOffsets(), In.FrameRangeSet.GetAllRangeLengths());
		}

		void NullaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameRangeSet& FrameRangeSet,
			const NullaryOpFunction Op)
		{
			Out.FrameRangeSet = FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ OutChannelNum, FrameRangeSet.GetTotalFrameNum() });

			Op(Out, Out.FrameRangeSet.GetAllRangeOffsets(), Out.FrameRangeSet.GetAllRangeLengths());
		}

		void UnaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameAttribute& In,
			const UnaryOpFunction Op)
		{
			Out.FrameRangeSet = In.FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ OutChannelNum, In.FrameRangeSet.GetTotalFrameNum() });

			Op(Out, In, Out.FrameRangeSet.GetAllRangeOffsets(), Out.FrameRangeSet.GetAllRangeLengths());
		}

		void InplaceOp(
			FFrameAttribute& InOut,
			const InplaceOpFunction Op)
		{
			Op(InOut, InOut.FrameRangeSet.GetAllRangeOffsets(), InOut.FrameRangeSet.GetAllRangeLengths());
		}

		void BinaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameAttribute& Lhs,
			const FFrameAttribute& Rhs,
			const BinaryOpFunction Op)
		{
			// Fast Path for when FrameRangeSets are Equal

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ OutChannelNum, Lhs.GetTotalFrameNum() });

				Op(Out, Lhs, Rhs, 
					Out.FrameRangeSet.GetAllRangeOffsets(),
					Lhs.FrameRangeSet.GetAllRangeOffsets(),
					Rhs.FrameRangeSet.GetAllRangeOffsets(),
					Out.FrameRangeSet.GetAllRangeLengths());

				return;
			}

			// Slow Path for when FrameRangeSets are not equal and we need to compute the intersection

			TLearningArray<1, int32> LhsRangeOffsets;
			TLearningArray<1, int32> RhsRangeOffsets;
			LhsRangeOffsets.SetNumUninitialized({ Lhs.FrameRangeSet.GetTotalRangeNum() + Rhs.FrameRangeSet.GetTotalRangeNum() });
			RhsRangeOffsets.SetNumUninitialized({ Lhs.FrameRangeSet.GetTotalRangeNum() + Rhs.FrameRangeSet.GetTotalRangeNum() });

			const int32 OutTotalRangeNum = UE::Learning::FrameRangeSet::IntersectionWithOffsets(
				Out.FrameRangeSet,
				LhsRangeOffsets,
				RhsRangeOffsets,
				Lhs.FrameRangeSet,
				Rhs.FrameRangeSet);

			// Resize back to correct size
			LhsRangeOffsets.SetNumUninitialized({ OutTotalRangeNum });
			RhsRangeOffsets.SetNumUninitialized({ OutTotalRangeNum });

			// Allocate Attribute Data
			Out.AttributeData.SetNumUninitialized({ OutChannelNum, Out.FrameRangeSet.GetTotalFrameNum() });

			Op(Out, Lhs, Rhs,
				Out.FrameRangeSet.GetAllRangeOffsets(),
				LhsRangeOffsets,
				RhsRangeOffsets,
				Out.FrameRangeSet.GetAllRangeLengths());
		}

		void NaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const TArrayView<const ConstFrameAttributePtr> Inputs,
			const NaryOpFunction Op)
		{
			if (Inputs.Num() == 0)
			{
				Out.Empty();
				Op(Out, {}, {}, {}, {});
				return;
			}

			// Check All Equal

			const int32 InputNum = Inputs.Num();

			bool bAllEqual = true;
			for (int32 InputIdx = 1; InputIdx < InputNum; InputIdx++)
			{
				bAllEqual &= FrameRangeSet::Equal(Inputs[0]->FrameRangeSet, Inputs[InputIdx]->FrameRangeSet);
			}

			if (bAllEqual)
			{
				Out.FrameRangeSet = Inputs[0]->FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ OutChannelNum, Inputs[0]->FrameRangeSet.GetTotalFrameNum() });

				TArray<TLearningArrayView<1, const int32>, TInlineAllocator<8>> InputRangeOffsetsViews;
				InputRangeOffsetsViews.SetNumUninitialized(InputNum);
				for (int32 InputIdx = 0; InputIdx < InputNum; InputIdx++)
				{
					InputRangeOffsetsViews[InputIdx] = Inputs[InputIdx]->FrameRangeSet.GetAllRangeOffsets();
				}

				Op(Out, Inputs,
					Out.FrameRangeSet.GetAllRangeOffsets(),
					InputRangeOffsetsViews,
					Out.FrameRangeSet.GetAllRangeLengths());
			}
			else
			{
				// Currently we don't support the case where things are not equal.

				checkNoEntry();
				Out.Empty();
				return;
			}
		}

		bool FindMinimum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMinimumValue,
			const FFrameAttribute& In)
		{
			return Private::FindMinimum(OutChannelIdx, OutFrameIdx, OutMinimumValue, In.GetAttributeData());
		}

		bool FindMaximum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMaximumValue,
			const FFrameAttribute& In)
		{
			return Private::FindMaximum(OutChannelIdx, OutFrameIdx, OutMaximumValue, In.GetAttributeData());
		}

		void Select(FFrameAttribute& Out, const FFrameAttribute& Cond, const FFrameAttribute& True, const FFrameAttribute& False)
		{
			check(Cond.GetChannelNum() == 1);
			check(True.GetChannelNum() == False.GetChannelNum());

			const int32 ChannelNum = True.GetChannelNum();

			NaryOp(Out, ChannelNum, {
				(ConstFrameAttributePtr)&Cond,
				(ConstFrameAttributePtr)&True,
				(ConstFrameAttributePtr)&False }, [ChannelNum](
					FFrameAttribute& Out,
					const TArrayView<const ConstFrameAttributePtr> In,
					const TLearningArrayView<1, const int32> OutRangeOffsets,
					const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
					const TLearningArrayView<1, const int32> RangeLengths) {

						for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
						{
							const int32 RangeNum = RangeLengths.Num();

							for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
							{
								Private::Select(
									Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
									In[0]->GetChannelRangeAttributeData(0, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
									In[1]->GetChannelRangeAttributeData(ChannelIdx, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
									In[2]->GetChannelRangeAttributeData(ChannelIdx, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]));
							}
						}
				});
		}

		void Zeros(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const int32 ChannelNum)
		{
			Out.FrameRangeSet = FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ ChannelNum, FrameRangeSet.GetTotalFrameNum() });
			Private::Zero(Out.AttributeData.Flatten());
		}

		void Ones(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const int32 ChannelNum)
		{
			Out.FrameRangeSet = FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ ChannelNum, FrameRangeSet.GetTotalFrameNum() });
			Private::Set(Out.AttributeData.Flatten(), 1.0f);
		}

		void Fill(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const TLearningArrayView<1, const float> Values)
		{
			const int32 ChannelNum = Values.Num();

			Out.FrameRangeSet = FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ ChannelNum, FrameRangeSet.GetTotalFrameNum() });
			for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
			{
				Private::Set(Out.AttributeData[ChannelIdx], Values[ChannelIdx]);
			}
		}

		void Add(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Add(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Add(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Sub(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Sub(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Sub(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Mul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Mul(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Mul(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Div(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Div(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Div(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}


		void Max(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Max(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Max(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Min(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			if (FrameRangeSet::Equal(Lhs.FrameRangeSet, Rhs.FrameRangeSet))
			{
				Out.FrameRangeSet = Lhs.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ Lhs.GetChannelNum(), Lhs.GetTotalFrameNum() });

				Private::Min(
					Out.GetAttributeData().Flatten(),
					Lhs.GetAttributeData().Flatten(),
					Rhs.GetAttributeData().Flatten());

				return;
			}

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Min(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Dot(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, 1, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::Zero(Out.GetAttributeData().Flatten());

					const int32 ChannelNum = Lhs.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Dot(
								Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Cross(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 3);
			check(Rhs.GetChannelNum() == 3);

			BinaryOp(Out, 3, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::Cross(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void Copy(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::Copy(Out.AttributeData.Flatten(), In.AttributeData.Flatten());
				});
		}

		void Repeat(FFrameAttribute& Out, const int32 ChannelNum, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 1);
			check(ChannelNum > 0);

			UnaryOp(Out, ChannelNum, In, [ChannelNum](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Copy(Out.GetChannelAttributeData(ChannelIdx), In.GetChannelAttributeData(0));
					}
				});
		}

		void Neg(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Neg(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Inv(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Inv(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Abs(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Abs(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Log(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Log(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Exp(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Exp(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Sqrt(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Sqrt(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Sin(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Sin(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Cos(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Cos(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Atan2(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 1);
			check(Rhs.GetChannelNum() == 1);

			BinaryOp(Out, 1, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::Atan2(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void NegInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::NegInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void RepeatFirstRangeEntryInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::RepeatFirstRangeEntryInplace(
							InOut.GetChannelAttributeData(ChannelIdx),
							RangeOffsets,
							RangeLengths);
					}
				});
		}

		void InvInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::InvInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void AbsInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::AbsInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void LogInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LogInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ExpInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ExpInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void SqrtInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::SqrtInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ApproxAcosInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ApproxAcosInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void NormalizeInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::NormalizeInplace(InOut.GetAttributeData());

				});
		}

		void Length(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, 1, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					Private::Zero(Out.GetAttributeData().Flatten());

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LengthSquared(
							Out.GetChannelAttributeData(0),
							In.GetChannelAttributeData(ChannelIdx));
					}

					Private::SqrtInplace(Out.GetChannelAttributeData(0));
				});
		}

		void Normalize(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::Normalize(Out.GetAttributeData(), In.GetAttributeData());

				});
		}

		void Index(FFrameAttribute& Out, const FFrameAttribute& In, const int32 ChannelIdx)
		{
			check(ChannelIdx >= 0);
			check(ChannelIdx < In.GetChannelNum());

			UnaryOp(Out, 1, In, [ChannelIdx](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::Copy(Out.GetChannelAttributeData(0), In.GetChannelAttributeData(ChannelIdx));

				});
		}

		void Slice(FFrameAttribute& Out, const FFrameAttribute& In, const int32 StartChannelIdx, const int32 ChannelNum)
		{
			check(StartChannelIdx >= 0);
			check(StartChannelIdx + ChannelNum <= In.GetChannelNum());

			UnaryOp(Out, ChannelNum, In, [StartChannelIdx, ChannelNum](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					check(Out.GetChannelNum() == ChannelNum);
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Copy(Out.GetChannelAttributeData(ChannelIdx), In.GetChannelAttributeData(ChannelIdx + StartChannelIdx));
					}

				});
		}

		void AddConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::AddConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void SubConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::SubConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void MulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MulConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void DivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::DivConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void MaxConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MaxConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void MinConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MinConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void ConstantAdd(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return AddConstant(Out, Rhs, Lhs);
		}

		void ConstantSub(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantSub(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantMul(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return MulConstant(Out, Rhs, Lhs);
		}

		void ConstantDiv(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantDiv(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantMax(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return MaxConstant(Out, Rhs, Lhs);
		}

		void ConstantMin(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return MinConstant(Out, Rhs, Lhs);
		}

		void ConstantDot(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, 1, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					Private::Zero(Out.GetAttributeData().Flatten());

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantDot(
							Out.GetChannelAttributeData(0),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantSquaredDist(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, 1, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					Private::Zero(Out.GetAttributeData().Flatten());

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantDistSquared(
							Out.GetChannelAttributeData(0),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantDist(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, 1, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					Private::Zero(Out.GetAttributeData().Flatten());

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantDistSquared(
							Out.GetChannelAttributeData(0),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}

					Private::SqrtInplace(Out.GetChannelAttributeData(0));
				});
		}

		void AddConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.GetChannelNum() == Rhs.Num());

			InplaceOp(InOut, [Rhs](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::AddConstantInplace(
							InOut.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void SubConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.GetChannelNum() == Rhs.Num());

			InplaceOp(InOut, [Rhs](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::SubConstantInplace(
							InOut.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void MulConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.GetChannelNum() == Rhs.Num());

			InplaceOp(InOut, [Rhs](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MulConstantInplace(
							InOut.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void DivConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs)
		{
			check(InOut.GetChannelNum() == Rhs.Num());

			InplaceOp(InOut, [Rhs](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::DivConstantInplace(
							InOut.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void Sum(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> Inputs)
		{
			const int32 InputNum = Inputs.Num();

			if (InputNum == 0)
			{
				Out.Empty();
			}
			else if (InputNum == 1)
			{
				Out = *Inputs[0];
			}
			else 
			{
				FFrameAttribute Tmp = *Inputs[0];

				for (int32 InputIdx = 1; InputIdx < InputNum; InputIdx++)
				{
					Add(Out, Tmp, *Inputs[InputIdx]);
					Tmp = Out;
				}
			}
		}

		void Prod(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> Inputs)
		{
			const int32 InputNum = Inputs.Num();

			if (InputNum == 0)
			{
				Out.Empty();
			}
			else if (InputNum == 1)
			{
				Out = *Inputs[0];
			}
			else
			{
				FFrameAttribute Tmp = *Inputs[0];

				for (int32 InputIdx = 1; InputIdx < InputNum; InputIdx++)
				{
					Mul(Out, Tmp, *Inputs[InputIdx]);
					Tmp = Out;
				}
			}
		}

		void LogicalAnd(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::LogicalAnd(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void LogicalOr(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::LogicalOr(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void LogicalNot(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			UnaryOp(Out, In.GetChannelNum(), In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LogicalNot(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void Gt(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Gt(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Ge(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Ge(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Lt(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Lt(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Le(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Le(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Eq(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Eq(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void Neq(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.GetChannelNum());

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::Neq(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
						}
					}
				});
		}

		void GtConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::GtConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void GeConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::GeConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void LtConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LtConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void LeConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LeConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void EqConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::EqConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void NeqConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs)
		{
			check(Lhs.GetChannelNum() == Rhs.Num());

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::NeqConstant(
							Out.GetChannelAttributeData(ChannelIdx),
							In.GetChannelAttributeData(ChannelIdx),
							Rhs[ChannelIdx]);
					}
				});
		}

		void ConstantGt(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantGt(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantGe(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantGe(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantLt(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantLt(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantLe(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.Num() == Rhs.GetChannelNum());

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::ConstantLe(
							Out.GetChannelAttributeData(ChannelIdx),
							Lhs[ChannelIdx],
							In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void ConstantEq(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return EqConstant(Out, Rhs, Lhs);
		}

		void ConstantNeq(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs)
		{
			// Since operation is symmetric we can just swap arguments
			return NeqConstant(Out, Rhs, Lhs);
		}

		void FilterGaussian(FFrameAttribute& Out, const FFrameAttribute& In, const float StdInFrames)
		{
			check(StdInFrames >= 0.0f);

			// Create odd sized kernel that is 3x the std
			TArray<float, TInlineAllocator<128>> Kernel;
			Kernel.SetNumUninitialized(FMath::RoundToInt(StdInFrames * 3.0f) * 2 + 1);
			Private::GaussianKernel(Kernel, StdInFrames);

			UnaryOp(Out, In.GetChannelNum(), In, [&Kernel](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::FilterWithKernel(
								Out.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								In.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Kernel);
						}
					}
				});
		}

		void FilterMajorityVote(FFrameAttribute& Out, const FFrameAttribute& In, const int32 FilterWidthFrames)
		{
			check(FilterWidthFrames >= 0);

			UnaryOp(Out, In.GetChannelNum(), In, [FilterWidthFrames](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::FilterMajorityVote(
								Out.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								In.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								FilterWidthFrames);
						}
					}
				});
		}

		void FilterSavGol(FFrameAttribute& Out, const FFrameAttribute& In, const int32 FilterWidthFrames, const int32 PolynomialDegree, const bool bGaussianWindowed)
		{
			check(FilterWidthFrames >= 0);
			check(PolynomialDegree >= 0);

			// Create odd sized kernel
			TArray<float, TInlineAllocator<128>> Kernel;
			Kernel.SetNumUninitialized((FilterWidthFrames / 2) * 2 + 1);
			Private::SavGolKernel(Kernel, PolynomialDegree, bGaussianWindowed);

			UnaryOp(Out, In.GetChannelNum(), In, [&Kernel](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();
					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::FilterWithKernel(
								Out.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								In.GetChannelRangeAttributeData(ChannelIdx, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Kernel);
						}
					}
				});
		}

		void Mean(TLearningArrayView<1, float> OutMean, const FFrameAttribute& In)
		{
			check(OutMean.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMean](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::Mean(OutMean[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void MeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutStd, const FFrameAttribute& In)
		{
			check(OutMean.Num() == In.GetChannelNum());
			check(OutStd.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMean, &OutStd](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MeanStd(OutMean[ChannelIdx], OutStd[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void AngularMean(TLearningArrayView<1, float> OutMean, const FFrameAttribute& In)
		{
			check(OutMean.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMean](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::AngularMean(OutMean[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void AngularMeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutStd, const FFrameAttribute& In)
		{
			check(OutMean.Num() == In.GetChannelNum());
			check(OutStd.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMean, &OutStd](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::AngularMeanStd(OutMean[ChannelIdx], OutStd[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void LogMeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutLogStd, const FFrameAttribute& In)
		{
			check(OutMean.Num() == In.GetChannelNum());
			check(OutLogStd.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMean, &OutLogStd](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::LogMeanStd(OutMean[ChannelIdx], OutLogStd[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void MinMax(TLearningArrayView<1, float> OutMin, TLearningArrayView<1, float> OutMax, const FFrameAttribute& In)
		{
			check(OutMin.Num() == In.GetChannelNum());
			check(OutMax.Num() == In.GetChannelNum());

			ReduceOp(In, [&OutMin, &OutMax](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = In.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::MinMax(OutMin[ChannelIdx], OutMax[ChannelIdx], In.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void QuatMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 4);
			check(Rhs.GetChannelNum() == 4);

			BinaryOp(Out, 4, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatMul(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatDiv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			QuatMulInv(Out, Lhs, Rhs);
		}

		void QuatInv(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 4, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatInv(Out.GetAttributeData(), In.GetAttributeData());
				});
		}

		void QuatAbs(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 4, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatAbs(Out.GetAttributeData(), In.GetAttributeData());
				});
		}

		void QuatToRotationVector(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 3, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {
				
					Private::QuatToRotationVector(Out.GetAttributeData(), In.GetAttributeData());
				});
		}

		void QuatFromRotationVector(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 3);

			UnaryOp(Out, 4, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {
				
					Private::QuatFromRotationVector(Out.GetAttributeData(),	In.GetAttributeData());
				});
		}

		void QuatFromPitchYawRoll(FFrameAttribute& Out, const FFrameAttribute& Pitch, const FFrameAttribute& Yaw, const FFrameAttribute& Roll)
		{
			check(Pitch.GetChannelNum() == 1);
			check(Yaw.GetChannelNum() == 1);
			check(Roll.GetChannelNum() == 1);

			NaryOp(Out, 4, {
				(ConstFrameAttributePtr)&Pitch,
				(ConstFrameAttributePtr)&Yaw,
				(ConstFrameAttributePtr)&Roll }, [](
					FFrameAttribute& Out,
					const TArrayView<const ConstFrameAttributePtr> In,
					const TLearningArrayView<1, const int32> OutRangeOffsets,
					const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
					const TLearningArrayView<1, const int32> RangeLengths) {

						const int32 RangeNum = RangeLengths.Num();

						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::QuatFromPitchYawRoll(
								Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

								In[0]->GetChannelRangeAttributeData(0, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(0, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[2]->GetChannelRangeAttributeData(0, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]));
						}
				});
		}

		void QuatFromBasisDirections(FFrameAttribute& Out, const FFrameAttribute& Forward, const FFrameAttribute& Right, const FFrameAttribute& Up)
		{
			check(Forward.GetChannelNum() == 3);
			check(Right.GetChannelNum() == 3);
			check(Up.GetChannelNum() == 3);

			NaryOp(Out, 4, {
				(ConstFrameAttributePtr)&Forward,
				(ConstFrameAttributePtr)&Right,
				(ConstFrameAttributePtr)&Up }, [](
					FFrameAttribute& Out,
					const TArrayView<const ConstFrameAttributePtr> In,
					const TLearningArrayView<1, const int32> OutRangeOffsets,
					const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
					const TLearningArrayView<1, const int32> RangeLengths) {

						const int32 RangeNum = RangeLengths.Num();

						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::QuatFromBasisDirections(
								Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

								In[0]->GetChannelRangeAttributeData(0, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
								In[0]->GetChannelRangeAttributeData(1, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
								In[0]->GetChannelRangeAttributeData(2, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),

								In[1]->GetChannelRangeAttributeData(0, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(1, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(2, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),

								In[2]->GetChannelRangeAttributeData(0, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]),
								In[2]->GetChannelRangeAttributeData(1, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]),
								In[2]->GetChannelRangeAttributeData(2, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]));
						}
				});
		}

		void QuatInvMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 4);
			check(Rhs.GetChannelNum() == 4);

			BinaryOp(Out, 4, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatInvMul(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatMulInv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 4);
			check(Rhs.GetChannelNum() == 4);

			BinaryOp(Out, 4, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatMulInv(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatRotate(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 4);
			check(Rhs.GetChannelNum() == 3);

			BinaryOp(Out, 3, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatRotate(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatUnrotate(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 4);
			check(Rhs.GetChannelNum() == 3);

			BinaryOp(Out, 3, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatUnrotate(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatBetween(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 3);
			check(Rhs.GetChannelNum() == 3);

			BinaryOp(Out, 4, Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatBetween(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatMulConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatDivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs)
		{
			QuatMulInvConstant(Out, Lhs, Rhs);
		}

		void QuatInvMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatInvMulConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatMulInvConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatMulInvConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatRotateConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 3, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatRotateConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatUnrotateConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 3, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatUnrotateConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatBetweenConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs)
		{
			check(Lhs.GetChannelNum() == 3);

			UnaryOp(Out, 4, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatBetweenConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatApproxShortestAngleBetweenConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs)
		{
			check(Lhs.GetChannelNum() == 4);

			UnaryOp(Out, 1, Lhs, [Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatApproxShortestAngleBetweenConstant(
						Out.GetAttributeData(),
						Lhs.GetAttributeData(),
						Rhs);
				});
		}

		void QuatConstantMul(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantMul(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatConstantDiv(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			QuatConstantMulInv(Out, Lhs, Rhs);
		}

		void QuatConstantInvMul(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantInvMul(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatConstantMulInv(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 4);

			UnaryOp(Out, 4, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantMulInv(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatConstantRotate(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 3);

			UnaryOp(Out, 3, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantRotate(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatConstantUnrotate(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 3);

			UnaryOp(Out, 3, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantUnrotate(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatConstantBetween(FFrameAttribute& Out, const FVector3f Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 3);

			UnaryOp(Out, 4, Rhs, [Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatConstantBetween(
						Out.GetAttributeData(),
						Lhs,
						Rhs.GetAttributeData());
				});
		}

		void QuatMeanStd(FQuat4f& OutMean, FVector3f& OutStd, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			ReduceOp(In, [&OutMean, &OutStd](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatMeanStd(OutMean, OutStd, In.GetAttributeData());
				});
		}

		void QuatPitch(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 1, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatPitch(Out.GetAttributeData(), In.GetAttributeData());
				});
		}

		void QuatYaw(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 1, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatYaw(Out.GetAttributeData(), In.GetAttributeData());
				});
		}

		void QuatRoll(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);

			UnaryOp(Out, 1, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::QuatRoll(Out.GetAttributeData(), In.GetAttributeData());
				});
		}


		void WrapAngleInplace(FFrameAttribute& InOut)
		{
			InplaceOp(InOut, [](
				FFrameAttribute& InOut,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = InOut.GetChannelNum();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						Private::WrapAngleInplace(
							InOut.GetChannelAttributeData(ChannelIdx));
					}
				});
		}

		void AngleToDirection3D(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 1);

			UnaryOp(Out, 3, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

						Private::AngleToDirection3D(
							Out.GetChannelAttributeData(0),
							Out.GetChannelAttributeData(1),
							Out.GetChannelAttributeData(2),
							In.GetChannelAttributeData(0));
				});
		}

		void Direction3DToAngle(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 3);

			UnaryOp(Out, 1, In, [](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::Direction3DToAngle(
						Out.GetChannelAttributeData(0),
						In.GetChannelAttributeData(0),
						In.GetChannelAttributeData(1));
				});
		}

		void TransformMake(FFrameAttribute& Out, const FFrameAttribute& Location, const FFrameAttribute& Rotation, const FFrameAttribute& Scale)
		{
			check(Location.GetChannelNum() == 3);
			check(Rotation.GetChannelNum() == 4);
			check(Scale.GetChannelNum() == 3);

			NaryOp(Out, 10, {
				(ConstFrameAttributePtr)&Location,
				(ConstFrameAttributePtr)&Rotation,
				(ConstFrameAttributePtr)&Scale }, [](
					FFrameAttribute& Out,
					const TArrayView<const ConstFrameAttributePtr> In,
					const TLearningArrayView<1, const int32> OutRangeOffsets,
					const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
					const TLearningArrayView<1, const int32> RangeLengths) {

						const int32 RangeNum = RangeLengths.Num();

						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::TransformMake(
								Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(4, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(5, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(6, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(7, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(8, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Out.GetChannelRangeAttributeData(9, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

								In[0]->GetChannelRangeAttributeData(0, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
								In[0]->GetChannelRangeAttributeData(1, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),
								In[0]->GetChannelRangeAttributeData(2, InRangeOffsets[0][RangeIdx], RangeLengths[RangeIdx]),

								In[1]->GetChannelRangeAttributeData(0, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(1, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(2, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								In[1]->GetChannelRangeAttributeData(3, InRangeOffsets[1][RangeIdx], RangeLengths[RangeIdx]),
								
								In[2]->GetChannelRangeAttributeData(0, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]),
								In[2]->GetChannelRangeAttributeData(1, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]),
								In[2]->GetChannelRangeAttributeData(2, InRangeOffsets[2][RangeIdx], RangeLengths[RangeIdx]));
						}
				});
		}

		void TransformMake(FFrameAttribute& Out, const FFrameAttribute& Location, const FFrameAttribute& Rotation)
		{
			check(Location.GetChannelNum() == 3);
			check(Rotation.GetChannelNum() == 4);

			BinaryOp(Out, 10, Location, Rotation, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::TransformMake(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(4, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(5, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(6, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(7, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(8, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(9, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void TransformMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);
			check(Rhs.GetChannelNum() == 10);

			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::TransformMul(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(4, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(5, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(6, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(7, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(8, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(9, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(4, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(5, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(6, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(7, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(8, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(9, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(4, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(5, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(6, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(7, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(8, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(9, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void TransformDiv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);
			check(Rhs.GetChannelNum() == 10);


			BinaryOp(Out, Lhs.GetChannelNum(), Lhs, Rhs, [](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					const int32 RangeNum = RangeLengths.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::TransformDiv(
							Out.GetChannelRangeAttributeData(0, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(1, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(2, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(3, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(4, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(5, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(6, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(7, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(8, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Out.GetChannelRangeAttributeData(9, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Lhs.GetChannelRangeAttributeData(0, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(1, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(2, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(3, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(4, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(5, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(6, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(7, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(8, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Lhs.GetChannelRangeAttributeData(9, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),

							Rhs.GetChannelRangeAttributeData(0, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(1, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(2, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(3, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(4, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(5, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(6, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(7, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(8, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							Rhs.GetChannelRangeAttributeData(9, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void TransformMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FTransform3f& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [&Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					Private::TransformMulConstant(
						Out.GetAttributeData(),
						In.GetAttributeData(),
						Rhs);
				});
		}

		void TransformDivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FTransform3f& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);

			UnaryOp(Out, Lhs.GetChannelNum(), Lhs, [&Rhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					Private::TransformDivConstant(
						Out.GetAttributeData(),
						In.GetAttributeData(),
						Rhs);
				});
		}

		void TransformConstantMul(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 10);

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [&Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					Private::TransformConstantMul(
						Out.GetAttributeData(),
						Lhs,
						In.GetAttributeData());
				});
		}

		void TransformConstantDiv(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 10);

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [&Lhs](
				FFrameAttribute& Out,
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths)
				{
					Private::TransformConstantDiv(
						Out.GetAttributeData(),
						Lhs,
						In.GetAttributeData());
				});
		}

		void TransformInv(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 10);

			UnaryOp(Out, In.GetChannelNum(), In, [](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformInv(
						Out.GetAttributeData(),
						In.GetAttributeData());
				});
		}

		void TransformLocation(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 10);
			Slice(Out, In, 0, 3);
		}

		void TransformRotation(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 10);
			Slice(Out, In, 3, 4);
		}

		void TransformScale(FFrameAttribute& Out, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 10);
			Slice(Out, In, 7, 3);
		}

		void TransformApplyConstantLocation(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);

			UnaryOp(Out, 3, Lhs, [Rhs](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformApplyConstantLocation(
						Out.GetAttributeData(),
						In.GetAttributeData(),
						Rhs);
				});
		}

		void TransformApplyConstantDirection(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f& Rhs)
		{
			check(Lhs.GetChannelNum() == 10);

			UnaryOp(Out, 3, Lhs, [Rhs](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformApplyConstantDirection(
						Out.GetAttributeData(),
						In.GetAttributeData(),
						Rhs);
				});
		}

		void TransformConstantApply(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs)
		{
			check(Rhs.GetChannelNum() == 3);

			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformConstantApply(
						Out.GetAttributeData(),
						Lhs,
						In.GetAttributeData());
				});
		}

		void TransformConstantApplyNoTranslation(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs)
		{
			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformConstantApplyNoTranslation(
						Out.GetAttributeData(),
						Lhs,
						In.GetAttributeData());
				});
		}

		void TransformConstantApplyNoTranslationScale(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs)
		{
			UnaryOp(Out, Rhs.GetChannelNum(), Rhs, [Lhs](
				UE::Learning::FFrameAttribute& Out,
				const UE::Learning::FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					Private::TransformConstantApplyNoTranslationScale(
						Out.GetAttributeData(),
						Lhs,
						In.GetAttributeData());
				});
		}

		void InertializationDistanceConstant(
			FFrameAttribute& Out,
			const FFrameAttribute& LhsPos,
			const FFrameAttribute& LhsVel,
			const TArrayView<const float> RhsLoc,
			const TArrayView<const float> RhsVel,
			const float BlendTime,
			const float LocWeight,
			const float VelWeight)
		{
			check(LhsPos.GetChannelNum() == LhsVel.GetChannelNum());
			check(LhsPos.GetChannelNum() == RhsLoc.Num());
			check(LhsPos.GetChannelNum() == RhsVel.Num());

			const int32 ChannelNum = LhsPos.GetChannelNum();

			if (FrameRangeSet::Equal(LhsPos.FrameRangeSet, LhsVel.FrameRangeSet))
			{
				Out.FrameRangeSet = LhsPos.FrameRangeSet;
				Out.AttributeData.SetNumUninitialized({ ChannelNum, LhsPos.GetTotalFrameNum() });

				for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
				{
					Private::InertializationDistanceConstant(
						Out.GetChannelAttributeData(ChannelIdx),
						LhsPos.GetChannelAttributeData(ChannelIdx),
						LhsVel.GetChannelAttributeData(ChannelIdx),
						RhsLoc[ChannelIdx],
						RhsVel[ChannelIdx],
						BlendTime,
						LocWeight,
						VelWeight);
				}

				return;
			}

			BinaryOp(Out, ChannelNum, LhsPos, LhsVel, [RhsLoc, RhsVel, BlendTime, LocWeight, VelWeight](
				FFrameAttribute& Out,
				const FFrameAttribute& Lhs,
				const FFrameAttribute& Rhs,
				const TLearningArrayView<1, const int32> OutRangeOffsets,
				const TLearningArrayView<1, const int32> LhsRangeOffsets,
				const TLearningArrayView<1, const int32> RhsRangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 ChannelNum = Out.GetChannelNum();
					const int32 RangeNum = RangeLengths.Num();

					for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
					{
						for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
						{
							Private::InertializationDistanceConstant(
								Out.GetChannelRangeAttributeData(ChannelIdx, OutRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Lhs.GetChannelRangeAttributeData(ChannelIdx, LhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								Rhs.GetChannelRangeAttributeData(ChannelIdx, RhsRangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
								RhsLoc[ChannelIdx],
								RhsVel[ChannelIdx],
								BlendTime,
								LocWeight,
								VelWeight);
						}
					}
				});
		}

		void LocationDistanceTraveled(TArrayView<float> OutDistancesTraveled, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 3);
			check(OutDistancesTraveled.Num() == In.FrameRangeSet.GetTotalRangeNum());

			ReduceOp(In, [&OutDistancesTraveled](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeOffsets.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::LocationDistanceTraveled(
							OutDistancesTraveled[RangeIdx],
							In.GetChannelRangeAttributeData(0, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							In.GetChannelRangeAttributeData(1, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							In.GetChannelRangeAttributeData(2, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}

		void QuatAngleTraveled(TArrayView<float> OutAnglesTraveled, const FFrameAttribute& In)
		{
			check(In.GetChannelNum() == 4);
			check(OutAnglesTraveled.Num() == In.FrameRangeSet.GetTotalRangeNum());

			ReduceOp(In, [&OutAnglesTraveled](
				const FFrameAttribute& In,
				const TLearningArrayView<1, const int32> RangeOffsets,
				const TLearningArrayView<1, const int32> RangeLengths) {

					const int32 RangeNum = RangeOffsets.Num();

					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Private::QuatAngleTraveled(
							OutAnglesTraveled[RangeIdx],
							In.GetChannelRangeAttributeData(0, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							In.GetChannelRangeAttributeData(1, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							In.GetChannelRangeAttributeData(2, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]),
							In.GetChannelRangeAttributeData(3, RangeOffsets[RangeIdx], RangeLengths[RangeIdx]));
					}
				});
		}


		void ExtractPhase(
			FFrameAttribute& Out,
			const FFrameRangeSet& FrameRangeSet,
			const FFrameSet& ZeroPhaseFrames,
			const FFrameSet& HalfPhaseFrames,
			const EPhaseExtrapolationMode Extrapolation)
		{
			Out.FrameRangeSet = FrameRangeSet;
			Out.AttributeData.SetNumUninitialized({ 1, FrameRangeSet.GetTotalFrameNum() });

			FFrameSet LhsFrames; FrameRangeSet::Intersection(LhsFrames, ZeroPhaseFrames, FrameRangeSet);
			FFrameSet RhsFrames; FrameRangeSet::Intersection(RhsFrames, HalfPhaseFrames, FrameRangeSet);

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 SequenceIdx = FrameRangeSet.GetEntrySequence(EntryIdx);
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				const int32 LhsEntryIdx = LhsFrames.FindSequenceEntry(SequenceIdx);
				const int32 RhsEntryIdx = RhsFrames.FindSequenceEntry(SequenceIdx);

				// No Rhs frames for this sequence
				if (RhsEntryIdx == INDEX_NONE)
				{
					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx), 0.0f);
					}
					continue;
				}

				// No Lhs frames for this sequence
				if (LhsEntryIdx == INDEX_NONE)
				{
					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx), UE_PI);
					}
					continue;
				}

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 RangeStart = FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);
					const int32 RangeStop = FrameRangeSet.GetEntryRangeStop(EntryIdx, RangeIdx);
					const int32 RangeLength = FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);

					int32 LhsStart = 0, RhsStart = 0;
					int32 LhsFrameNum = 0, RhsFrameNum = 0;
					LhsFrames.FindFramesInRange(LhsStart, LhsFrameNum, LhsEntryIdx, RangeStart, RangeLength);
					RhsFrames.FindFramesInRange(RhsStart, RhsFrameNum, RhsEntryIdx, RangeStart, RangeLength);

					int32 LhsIdx = LhsStart;
					int32 RhsIdx = RhsStart;

					// Check against no frames in range

					if (RhsFrameNum == 0)
					{
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx), 0.0f);
						continue;
					}

					if (LhsFrameNum == 0)
					{
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx), UE_PI);
						continue;
					}

					int32 LhsFrame = LhsFrames.GetEntryFrame(LhsEntryIdx, LhsIdx);
					int32 RhsFrame = RhsFrames.GetEntryFrame(RhsEntryIdx, RhsIdx);
					int32 LastFrame = RangeStart;
					int32 InitialFrame = INDEX_NONE;
					bool bLhsActive = false;

					// Set initial phase value based on first entry
					if (LhsFrame < RhsFrame)
					{
						InitialFrame = LhsFrame;
						LastFrame = LhsFrame;
						bLhsActive = true;
						LhsIdx++;
					}
					else
					{
						InitialFrame = RhsFrame;
						LastFrame = RhsFrame;
						bLhsActive = false;
						RhsIdx++;
					}

					// While we still have both left and right events
					while (LhsIdx < LhsStart + LhsFrameNum && RhsIdx < RhsStart + RhsFrameNum)
					{
						LhsFrame = LhsFrames.GetEntryFrame(LhsEntryIdx, LhsIdx);
						RhsFrame = RhsFrames.GetEntryFrame(RhsEntryIdx, RhsIdx);

						if (LhsFrame < RhsFrame)
						{
							if (!bLhsActive)
							{
								for (int32 Idx = 0; Idx < LhsFrame - LastFrame + 1; Idx++)
								{
									Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart + Idx] =
										FMath::Wrap(FMath::Lerp(UE_PI, UE_TWO_PI, (float)(Idx) / FMath::Max(LhsFrame - LastFrame, UE_SMALL_NUMBER)), -UE_PI, UE_PI);
								}
								LastFrame = LhsFrame;
								bLhsActive = true;
							}
							LhsIdx++;
						}
						else
						{
							if (bLhsActive)
							{
								for (int32 Idx = 0; Idx < RhsFrame - LastFrame + 1; Idx++)
								{
									Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart + Idx] =
										FMath::Lerp(0.0f, UE_PI, (float)(Idx) / FMath::Max(RhsFrame - LastFrame, UE_SMALL_NUMBER));
								}
								LastFrame = RhsFrame;
								bLhsActive = false;
							}
							RhsIdx++;
						}
					}

					// Remaining Left Events
					while (LhsIdx < LhsStart + LhsFrameNum)
					{
						LhsFrame = LhsFrames.GetEntryFrame(LhsEntryIdx, LhsIdx);

						if (!bLhsActive)
						{
							for (int32 Idx = 0; Idx < LhsFrame - LastFrame + 1; Idx++)
							{
								Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart + Idx] =
									FMath::Wrap(FMath::Lerp(UE_PI, UE_TWO_PI, (float)(Idx) / FMath::Max(LhsFrame - LastFrame, UE_SMALL_NUMBER)), -UE_PI, UE_PI);
							}
							LastFrame = LhsFrame;
							bLhsActive = true;
						}
						LhsIdx++;
					}

					// Remaining Right Events
					while (RhsIdx < RhsStart + RhsFrameNum)
					{
						RhsFrame = RhsFrames.GetEntryFrame(RhsEntryIdx, RhsIdx);

						if (bLhsActive)
						{
							for (int32 Idx = 0; Idx < RhsFrame - LastFrame + 1; Idx++)
							{
								Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart + Idx] =
									FMath::Lerp(0.0f, UE_PI, (float)(Idx) / FMath::Max(RhsFrame - LastFrame, UE_SMALL_NUMBER));
							}
							LastFrame = RhsFrame;
							bLhsActive = false;
						}
						RhsIdx++;
					}

					// Fill Ends

					if (Extrapolation == EPhaseExtrapolationMode::Repeat)
					{
						const float InitialValue = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[InitialFrame - RangeStart];
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx).Slice(0, InitialFrame - RangeStart), InitialValue);

						const float FinalValue = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart];
						Array::Set(Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx).Slice(LastFrame - RangeStart, RangeStop - LastFrame), FinalValue);
					}
					else if (Extrapolation == EPhaseExtrapolationMode::Extrapolate)
					{
						const float InitialValue0 = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[InitialFrame + 0 - RangeStart];
						const float InitialValue1 = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[InitialFrame + 1 - RangeStart];
						for (int32 Idx = 0; Idx < InitialFrame - RangeStart; Idx++)
						{
							Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[Idx] = FMath::Wrap(
								InitialValue0 - (InitialFrame - RangeStart - Idx) * (InitialValue1 - InitialValue0), -UE_PI, UE_PI);
						}
						
						const float FinalValue0 = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - 1 - RangeStart];
						const float FinalValue1 = Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - 0 - RangeStart];
						for (int32 Idx = 0; Idx < RangeStop - LastFrame; Idx++)
						{
							Out.GetChannelEntryRangeAttributeData(0, EntryIdx, RangeIdx)[LastFrame - RangeStart + Idx] = FMath::Wrap(
								FinalValue0 + (Idx + 1) * (FinalValue1 - FinalValue0), -UE_PI, UE_PI);
						}
					}
					else
					{
						checkNoEntry();
					}
				}
			}
		}

	}
}
