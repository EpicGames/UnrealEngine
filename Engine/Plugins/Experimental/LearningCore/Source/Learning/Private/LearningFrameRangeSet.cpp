// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningFrameRangeSet.h"
#include "LearningFrameSet.h"

#include "Async/ParallelFor.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	namespace FrameRangeSet::Private
	{
		static inline bool ContainsSorted(const TLearningArrayView<1, const int32> Elements, const int32 Element)
		{
			const int32 Num = Elements.Num();

#if UE_LEARNING_ISPC
			return ispc::LearningContainsSorted(Elements.GetData(), Element, Num);
#else
			if (Num <= 512)
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					if (Elements[Idx] == Element) { return true; }
				}
				return false;
			}

			int32 Low = 0;
			int32 High = Num - 1;

			while (Low <= High)
			{
				const int32 Mid = Low + (High - Low) / 2;

				if (Elements[Mid] == Element)
				{
					return true;
				}
				else if (Elements[Mid] < Element)
				{
					Low = Mid + 1;
				}
				else
				{
					High = Mid - 1;
				}
			}

			return false;
#endif
		}

		static inline int32 FindSorted(const TLearningArrayView<1, const int32> Elements, const int32 Element)
		{
			const int32 Num = Elements.Num();

#if UE_LEARNING_ISPC
			return ispc::LearningFindSorted(Elements.GetData(), Element, Num);
#else
			if (Num <= 512)
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					if (Elements[Idx] == Element) { return Idx; }
				}
				return INDEX_NONE;
			}

			int32 Low = 0;
			int32 High = Num - 1;

			while (Low <= High)
			{
				const int32 Mid = Low + (High - Low) / 2;

				if (Elements[Mid] == Element)
				{
					return Mid;
				}
				else if (Elements[Mid] < Element)
				{
					Low = Mid + 1;
				}
				else
				{
					High = Mid - 1;
				}
			}

			return INDEX_NONE;
#endif
		}

		static inline void FramesCheck(const TLearningArrayView<1, const int32> Frames)
		{
			const int32 FrameNum = Frames.Num();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum - 1; FrameIdx++)
			{
				check(Frames[FrameIdx + 0] < Frames[FrameIdx + 1]);
			}
		}

		static inline void RangesCheck(
			const TLearningArrayView<1, const int32> Starts, 
			const TLearningArrayView<1, const int32> Lengths)
		{
			check(Starts.Num() == Lengths.Num());

			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum - 1; RangeIdx++)
			{
				check(Lengths[RangeIdx + 0] > 0);
				check(Lengths[RangeIdx + 1] > 0);
				check(Starts[RangeIdx + 0] + Lengths[RangeIdx + 0] <= Starts[RangeIdx + 1]);
			}
		}

		static inline void OffsetsCheck(
			const TLearningArrayView<1, const int32> Offsets,
			const int32 MinimumOffset,
			const int32 MaximumOffset)
		{
			const int32 FrameNum = Offsets.Num();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum - 1; FrameIdx++)
			{
				check(Offsets[FrameIdx + 0] >= MinimumOffset);
				check(Offsets[FrameIdx + 1] >= MinimumOffset);
				check(Offsets[FrameIdx + 0] < MaximumOffset);
				check(Offsets[FrameIdx + 1] < MaximumOffset);
				check(Offsets[FrameIdx + 0] < Offsets[FrameIdx + 1]);
			}
		}

		static inline void ComputeRangeOffsets(
			TLearningArrayView<1, int32> Offsets,
			const TLearningArrayView<1, const int32> Lengths,
			const int32 InitialOffset = 0)
		{
			check(Offsets.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningComputeRangeOffsets(
				Offsets.GetData(),
				Lengths.GetData(),
				Lengths.Num(),
				InitialOffset);
#else
			const int32 RangeNum = Lengths.Num();
			int32 Offset = InitialOffset;
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				Offsets[RangeIdx] = Offset;
				Offset += Lengths[RangeIdx];
			}
#endif
		}

		static inline void FramesCopy(
			const TLearningArrayView<1, int32> DstFrames,
			const TLearningArrayView<1, const int32> SrcFrames)
		{
			check(DstFrames.Num() == SrcFrames.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningFramesCopy(
				DstFrames.GetData(),
				SrcFrames.GetData(),
				DstFrames.Num());
#else

			const int32 FrameNum = DstFrames.Num();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				DstFrames[FrameIdx] = SrcFrames[FrameIdx];
			}
#endif
		}

		static inline void RangesCopy(
			const TLearningArrayView<1, int32> DstStarts,
			const TLearningArrayView<1, int32> DstLengths,
			const TLearningArrayView<1, const int32> SrcStarts,
			const TLearningArrayView<1, const int32> SrcLengths)
		{
			check(DstStarts.Num() == DstLengths.Num());
			check(SrcStarts.Num() == SrcLengths.Num());
			check(DstStarts.Num() == SrcStarts.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesCopy(
				DstStarts.GetData(),
				DstLengths.GetData(),
				SrcStarts.GetData(),
				SrcLengths.GetData(),
				DstStarts.Num());
#else

			const int32 RangeNum = DstStarts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				DstStarts[RangeIdx] = SrcStarts[RangeIdx];
				DstLengths[RangeIdx] = SrcLengths[RangeIdx];
			}
#endif
		}

		static inline void RangesCopyFrames(
			const TLearningArrayView<1, int32> DstStarts,
			const TLearningArrayView<1, int32> DstLengths,
			const TLearningArrayView<1, const int32> SrcFrames)
		{
			check(DstStarts.Num() == DstLengths.Num());
			check(DstStarts.Num() == SrcFrames.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesCopyFrames(
				DstStarts.GetData(),
				DstLengths.GetData(),
				SrcFrames.GetData(),
				DstStarts.Num());
#else
			const int32 RangeNum = DstStarts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				DstStarts[RangeIdx] = SrcFrames[RangeIdx];
				DstLengths[RangeIdx] = 1;
			}
#endif
		}

		static inline bool RangesEqual(
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			check(LhsStarts.Num() == LhsLengths.Num());
			check(RhsStarts.Num() == RhsLengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesEqual(
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			if (LhsStarts.Num() != RhsStarts.Num()) { return false; }

			const int32 RangeNum = LhsStarts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				if (LhsStarts[RangeIdx] != RhsStarts[RangeIdx] || LhsLengths[RangeIdx] != RhsLengths[RangeIdx])
				{
					return false;
				}
			}

			return true;
#endif
		}

		static inline bool RangesContains(
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const int32 Frame)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesContains(
				Starts.GetData(),
				Lengths.GetData(),
				Frame,
				Starts.Num());
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				if (Frame >= Starts[RangeIdx] && Frame < Starts[RangeIdx] + Lengths[RangeIdx])
				{
					return true;
				}
			}
			return false;
#endif
		}

		static inline bool RangesIntersectsRange(
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const int32 Start,
			const int32 Length)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesIntersectsRange(
				Starts.GetData(),
				Lengths.GetData(),
				Start,
				Length,
				Starts.Num());
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				if (Starts[RangeIdx] < Start + Length && Start < Starts[RangeIdx] + Lengths[RangeIdx])
				{
					return true;
				}
			}
			return false;
#endif
		}

		static inline bool RangesContainsRange(
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const int32 Start,
			const int32 Length)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesContainsRange(
				Starts.GetData(),
				Lengths.GetData(),
				Start,
				Length,
				Starts.Num());
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				if (Starts[RangeIdx] <= Start && Start + Length <= Starts[RangeIdx] + Lengths[RangeIdx])
				{
					return true;
				}
			}
			return false;
#endif
		}

		static inline bool RangesContainsTime(
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const float Time,
			const float FrameDeltaTime,
			const float Tolerance = UE_KINDA_SMALL_NUMBER)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesContainsTime(
				Starts.GetData(),
				Lengths.GetData(),
				Starts.Num(),
				Time,
				FrameDeltaTime,
				Tolerance);
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const float StartTime = Starts[RangeIdx] * FrameDeltaTime;
				const float EndTime = (Starts[RangeIdx] + Lengths[RangeIdx] - 1) * FrameDeltaTime;

				if (Time >= StartTime - Tolerance && Time < EndTime + Tolerance)
				{
					return true;
				}
			}
			return false;
#endif
		}

		static inline bool RangesFind(
			int32& OutRangeIdx,
			int32& OutRangeFrame,
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const int32 Frame)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesFind(
				OutRangeIdx,
				OutRangeFrame,
				Starts.GetData(),
				Lengths.GetData(),
				Frame,
				Starts.Num());
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				if (Frame >= Starts[RangeIdx] && Frame < Starts[RangeIdx] + Lengths[RangeIdx])
				{
					OutRangeIdx = RangeIdx;
					OutRangeFrame = Frame - Starts[RangeIdx];
					return true;
				}
			}

			OutRangeIdx = INDEX_NONE;
			OutRangeFrame = INDEX_NONE;
			return false;
#endif
		}

		static inline bool RangesFindTime(
			int32& OutRangeIdx,
			float& OutRangeTime,
			const TLearningArrayView<1, const int32> Starts,
			const TLearningArrayView<1, const int32> Lengths,
			const float Time,
			const float FrameDeltaTime,
			const float Tolerance = UE_KINDA_SMALL_NUMBER)
		{
			check(Starts.Num() == Lengths.Num());

#if UE_LEARNING_ISPC
			return ispc::LearningRangesFindTime(
				OutRangeIdx,
				OutRangeTime,
				Starts.GetData(),
				Lengths.GetData(),
				Starts.Num(),
				Time,
				FrameDeltaTime,
				Tolerance);
#else
			const int32 RangeNum = Starts.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const float StartTime = Starts[RangeIdx] * FrameDeltaTime;
				const float EndTime = (Starts[RangeIdx] + Lengths[RangeIdx] - 1) * FrameDeltaTime;
				const float Duration = (Lengths[RangeIdx] - 1) * FrameDeltaTime;

				if (Time >= StartTime - Tolerance && Time < EndTime + Tolerance)
				{
					OutRangeIdx = RangeIdx;
					OutRangeTime = FMath::Clamp(Time - StartTime, 0.0f, Duration);
					return true;
				}
			}

			OutRangeIdx = INDEX_NONE;
			OutRangeTime = -1.0f;
			return false;
#endif
		}


		static inline int32 FramesRangesUnion(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::FramesRangesUnion);

#if UE_LEARNING_ISPC
			return ispc::LearningFramesRangesUnion(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsFrames.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsFrames.Num(),
				RhsStarts.Num());
#else
			FramesCheck(LhsFrames);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsFrames.IsEmpty())
			{
				RangesCopy(
					OutStarts.Slice(0, RhsStarts.Num()), 
					OutLengths.Slice(0, RhsLengths.Num()), 
					RhsStarts,
					RhsLengths);

				return RhsLengths.Num();
			}

			if (RhsStarts.IsEmpty())
			{
				RangesCopyFrames(
					OutStarts.Slice(0, LhsFrames.Num()),
					OutLengths.Slice(0, LhsFrames.Num()),
					LhsFrames);

				return LhsFrames.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsFrames[LhsIndex / 2] : LhsFrames[LhsIndex / 2] + 1;
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs is coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext && !bRhsActive)
					{

						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsIndex++;
				}
				// Event from rhs is coming first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActive && !bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && (bLhsActiveNext || bRhsActiveNext))
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !(bLhsActiveNext || bRhsActiveNext))
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs events
			while (LhsIndex < LhsNum * 2)
			{
				check(RhsIndex == RhsNum * 2);

				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const int32 LhsT = bLhsActiveNext ? LhsFrames[LhsIndex / 2] : LhsFrames[LhsIndex / 2] + 1;

				// Activate output
				if (!bOutActive && bLhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = LhsT;
				}
				// Deactivate output
				else if (bOutActive && !bLhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bLhsActive = bLhsActiveNext;
				LhsIndex++;
			}

			// Process any remaining rhs events
			while (RhsIndex < RhsNum * 2)
			{
				check(LhsIndex == LhsNum * 2);

				const bool bRhsActiveNext = RhsIndex % 2 == 0;
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Activate output
				if (!bOutActive && bRhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = RhsT;
				}
				// Deactivate output
				else if (bOutActive && !bRhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bRhsActive = bRhsActiveNext;
				RhsIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}

		static inline int32 FramesRangesIntersection(
			TLearningArrayView<1, int32> OutFrames,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::FramesRangesIntersection);

#if UE_LEARNING_ISPC
			return ispc::LearningFramesRangesIntersection(
				OutFrames.GetData(),
				LhsFrames.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsFrames.Num(),
				RhsStarts.Num());
#else
			FramesCheck(LhsFrames);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsFrames.IsEmpty()) { return 0; }
			if (RhsStarts.IsEmpty()) { return 0; }

			// Number of frames/ranges in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of ranges
			bool bRhsActive = false;

			// Event index for each list
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum && RhsIndex < RhsNum * 2)
			{
				// Are the rhs events active or inactive
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = LhsFrames[LhsIndex];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Add to output if range is active
					if (bRhsActive)
					{
						OutFrames[OutIndex] = LhsT;
						OutIndex++;
					}

					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					bRhsActive = bRhsActiveNext;
					RhsIndex++;

					// Add to output if range is active
					if (bRhsActive)
					{
						OutFrames[OutIndex] = LhsT;
						OutIndex++;
					}

					LhsIndex++;
				}
			}

			FramesCheck(OutFrames.Slice(0, OutIndex));

			// Return number of frames added
			return OutIndex;
#endif
		}

		static inline int32 FramesRangesDifference(
			TLearningArrayView<1, int32> OutFrames,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::FramesRangesDifference);

#if UE_LEARNING_ISPC
			return ispc::LearningFramesRangesDifference(
				OutFrames.GetData(),
				LhsFrames.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsFrames.Num(),
				RhsStarts.Num());
#else
			FramesCheck(LhsFrames);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsFrames.IsEmpty()) { return 0; }

			if (RhsStarts.IsEmpty())
			{
				FramesCopy(OutFrames.Slice(0, LhsFrames.Num()), LhsFrames);
				return LhsFrames.Num();
			}

			// Number of frames/ranges in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of ranges
			bool bRhsActive = false;

			// Event index for each list
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum && RhsIndex < RhsNum * 2)
			{
				// Are the rhs events active or inactive
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = LhsFrames[LhsIndex];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Add to output if range is active
					if (!bRhsActive)
					{
						OutFrames[OutIndex] = LhsT;
						OutIndex++;
					}

					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					bRhsActive = bRhsActiveNext;
					RhsIndex++;

					// Add to output if range is active
					if (!bRhsActive)
					{
						OutFrames[OutIndex] = LhsT;
						OutIndex++;
					}

					LhsIndex++;
				}
			}

			FramesCheck(OutFrames.Slice(0, OutIndex));

			// Return number of events added
			return OutIndex;
#endif
		}


		static inline int32 RangesFramesDifference(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsFrames)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesFramesDifference);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesFramesDifference(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsFrames.GetData(),
				LhsStarts.Num(),
				RhsFrames.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			FramesCheck(RhsFrames);

			if (LhsStarts.IsEmpty()) { return 0; }

			if (RhsFrames.IsEmpty())
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()), 
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsFrames.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsFrames[RhsIndex / 2] : RhsFrames[RhsIndex / 2] + 1;

				// Event coming from lhs first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && !bRhsActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsIndex++;
				}
				// Event coming from rhs first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActive && !bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && bLhsActiveNext && !bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs events
			while (LhsIndex < LhsNum * 2)
			{
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

				// Activate output
				if (!bOutActive && bLhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = LhsT;
				}
				// Deactivate output
				else if (bOutActive && !bLhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bLhsActive = bLhsActiveNext;
				LhsIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}


		static inline int32 RangesUnion(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesUnion);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesUnion(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsStarts.IsEmpty())
			{
				RangesCopy(
					OutStarts.Slice(0, RhsStarts.Num()),
					OutLengths.Slice(0, RhsLengths.Num()),
					RhsStarts,
					RhsLengths);

				return RhsLengths.Num();
			}

			if (RhsStarts.IsEmpty())
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			if (RangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths))
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs is coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext && !bRhsActive)
					{

						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsIndex++;
				}
				// Event from rhs is coming first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActive && !bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && (bLhsActiveNext || bRhsActiveNext))
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !(bLhsActiveNext || bRhsActiveNext))
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs events
			while (LhsIndex < LhsNum * 2)
			{
				check(RhsIndex == RhsNum * 2);

				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

				// Activate output
				if (!bOutActive && bLhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = LhsT;
				}
				// Deactivate output
				else if (bOutActive && !bLhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bLhsActive = bLhsActiveNext;
				LhsIndex++;
			}

			// Process any remaining rhs events
			while (RhsIndex < RhsNum * 2)
			{
				check(LhsIndex == LhsNum * 2);

				const bool bRhsActiveNext = RhsIndex % 2 == 0;
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Activate output
				if (!bOutActive && bRhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = RhsT;
				}
				// Deactivate output
				else if (bOutActive && !bRhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bRhsActive = bRhsActiveNext;
				RhsIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}

		static inline int32 RangesIntersection(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesIntersection);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesIntersection(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsStarts.IsEmpty()) { return 0; }
			if (RhsStarts.IsEmpty()) { return 0; }

			if (RangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths))
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bRhsActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActive && bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && !bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && (bLhsActiveNext && bRhsActiveNext))
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && (!bLhsActiveNext || !bRhsActiveNext))
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}

		static inline int32 RangesTotalFrameNum(const TLearningArrayView<1, const int32> RangeLengths)
		{
#if UE_LEARNING_ISPC
			return ispc::LearningRangesTotalFrameNum(RangeLengths.GetData(), RangeLengths.Num());
#else
			int32 Total = 0;

			const int32 RangeNum = RangeLengths.Num();
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				Total += RangeLengths[RangeIdx];
			}

			return Total;
#endif
		}

		static inline int32 RangesIntersectionWithOffsets(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			TLearningArrayView<1, int32> OutLhsOffsets,
			TLearningArrayView<1, int32> OutRhsOffsets,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths,
			const int32 LhsEntryOffset,
			const int32 RhsEntryOffset)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesIntersectionWithOffsets);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesIntersectionWithOffsets(
				OutStarts.GetData(),
				OutLengths.GetData(),
				OutLhsOffsets.GetData(),
				OutRhsOffsets.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsEntryOffset,
				RhsEntryOffset,
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsStarts.IsEmpty()) { return 0; }
			if (RhsStarts.IsEmpty()) { return 0; }

			if (RangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths))
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				ComputeRangeOffsets(OutLhsOffsets.Slice(0, LhsStarts.Num()), LhsLengths, LhsEntryOffset);
				ComputeRangeOffsets(OutRhsOffsets.Slice(0, RhsStarts.Num()), RhsLengths, RhsEntryOffset);
				OffsetsCheck(OutLhsOffsets.Slice(0, LhsStarts.Num()), LhsEntryOffset, LhsEntryOffset + RangesTotalFrameNum(LhsLengths));
				OffsetsCheck(OutRhsOffsets.Slice(0, RhsStarts.Num()), RhsEntryOffset, RhsEntryOffset + RangesTotalFrameNum(RhsLengths));
				return LhsLengths.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// Offsets
			int32 LhsOffset = LhsEntryOffset;
			int32 RhsOffset = RhsEntryOffset;
			int32 LhsActiveStart = INDEX_NONE;
			int32 RhsActiveStart = INDEX_NONE;
			int32 LhsActiveOffset = INDEX_NONE;
			int32 RhsActiveOffset = INDEX_NONE;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bRhsActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						check(bLhsActive);
						check(bRhsActive);
						check(LhsActiveStart != INDEX_NONE);
						check(LhsActiveOffset != INDEX_NONE);
						check(RhsActiveStart != INDEX_NONE);
						check(RhsActiveOffset != INDEX_NONE);

						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
						OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsActiveStart = bLhsActive ? LhsStarts[LhsIndex / 2] : INDEX_NONE;
					LhsActiveOffset = bLhsActive ? LhsOffset : INDEX_NONE;
					LhsOffset += bLhsActive ? 0 : LhsLengths[LhsIndex / 2];
					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActive && bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && !bRhsActiveNext)
					{
						check(bLhsActive);
						check(bRhsActive);
						check(LhsActiveStart != INDEX_NONE);
						check(LhsActiveOffset != INDEX_NONE);
						check(RhsActiveStart != INDEX_NONE);
						check(RhsActiveOffset != INDEX_NONE);

						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
						OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsActiveStart = bRhsActive ? RhsStarts[RhsIndex / 2] : INDEX_NONE;
					RhsActiveOffset = bRhsActive ? RhsOffset : INDEX_NONE;
					RhsOffset += bRhsActive ? 0 : RhsLengths[RhsIndex / 2];
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && (bLhsActiveNext && bRhsActiveNext))
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && (!bLhsActiveNext || !bRhsActiveNext))
					{
						check(bLhsActive);
						check(bRhsActive);
						check(LhsActiveStart != INDEX_NONE);
						check(LhsActiveOffset != INDEX_NONE);
						check(RhsActiveStart != INDEX_NONE);
						check(RhsActiveOffset != INDEX_NONE);

						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutLhsOffsets[OutIndex] = LhsActiveOffset + (OutStarts[OutIndex] - LhsActiveStart);
						OutRhsOffsets[OutIndex] = RhsActiveOffset + (OutStarts[OutIndex] - RhsActiveStart);
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsActiveStart = bLhsActive ? LhsStarts[LhsIndex / 2] : INDEX_NONE;
					LhsActiveOffset = bLhsActive ? LhsOffset : INDEX_NONE;
					bRhsActive = bRhsActiveNext;
					RhsActiveStart = bRhsActive ? RhsStarts[RhsIndex / 2] : INDEX_NONE;
					RhsActiveOffset = bRhsActive ? RhsOffset : INDEX_NONE;
					LhsOffset += bLhsActive ? 0 : LhsLengths[LhsIndex / 2];
					RhsOffset += bRhsActive ? 0 : RhsLengths[RhsIndex / 2];
					LhsIndex++; RhsIndex++;
				}
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));
			OffsetsCheck(OutLhsOffsets.Slice(0, OutIndex), LhsEntryOffset, LhsEntryOffset + RangesTotalFrameNum(LhsLengths));
			OffsetsCheck(OutRhsOffsets.Slice(0, OutIndex), RhsEntryOffset, RhsEntryOffset + RangesTotalFrameNum(RhsLengths));

			// Return number of ranges added
			return OutIndex;
#endif
		}

		static inline int32 RangesDifference(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesDifference);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesDifference(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsStarts.IsEmpty()) { return 0; }

			if (RhsStarts.IsEmpty())
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			if (RangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths))
			{
				return 0;
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bLhsActive = false;
			bool bRhsActive = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event coming from lhs first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && !bRhsActive && bLhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					LhsIndex++;
				}
				// Event coming from rhs first
				else if (RhsT < LhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActive && !bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = RhsT;
					}
					// Deactivate output
					else if (bOutActive && bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = RhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && bLhsActiveNext && !bRhsActiveNext)
					{
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && bRhsActiveNext)
					{
						bOutActive = false;
						OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
						OutIndex++;
					}

					bLhsActive = bLhsActiveNext;
					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs events
			while (LhsIndex < LhsNum * 2)
			{
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];

				// Activate output
				if (!bOutActive && bLhsActiveNext)
				{
					bOutActive = true;
					OutStarts[OutIndex] = LhsT;
				}
				// Deactivate output
				else if (bOutActive && !bLhsActiveNext)
				{
					bOutActive = false;
					OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
					OutIndex++;
				}

				bLhsActive = bLhsActiveNext;
				LhsIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}

		static inline int32 RangesIntersects(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsStarts,
			const TLearningArrayView<1, const int32> RhsLengths)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesIntersects);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesIntersects(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsStarts.GetData(),
				RhsLengths.GetData(),
				LhsStarts.Num(),
				RhsStarts.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			RangesCheck(RhsStarts, RhsLengths);

			if (LhsStarts.IsEmpty()) { return 0; }
			if (RhsStarts.IsEmpty()) { return 0; }

			if (RangesEqual(LhsStarts, LhsLengths, RhsStarts, RhsLengths))
			{
				RangesCopy(
					OutStarts.Slice(0, LhsStarts.Num()),
					OutLengths.Slice(0, LhsLengths.Num()),
					LhsStarts,
					LhsLengths);

				return LhsLengths.Num();
			}

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsStarts.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bRhsActive = false;
			bool bIntersects = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both ranges have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum * 2)
			{
				// Are the next lhs, and rhs events active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				const bool bRhsActiveNext = RhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = bRhsActiveNext ? RhsStarts[RhsIndex / 2] : RhsStarts[RhsIndex / 2] + RhsLengths[RhsIndex / 2];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bIntersects = bRhsActive;
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						if (bIntersects)
						{
							OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
							OutIndex++;
						}
					}

					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					// Mark as intersecting
					if (bOutActive && bRhsActiveNext)
					{
						bIntersects = true;
					}

					bRhsActive = bRhsActiveNext;
					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bIntersects = bRhsActiveNext;
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						if (bIntersects)
						{
							OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
							OutIndex++;
						}
					}

					bRhsActive = bRhsActiveNext;
					LhsIndex++; RhsIndex++;
				}
			}

			// While the output is still active and intersecting
			if (bOutActive && bIntersects)
			{
				// Next lhs event should be inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				check(!bLhsActiveNext);

				// Time of the next lhs event
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				
				// Finish writing range
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}
	
		static inline int32 RangesFramesIntersects(
			TLearningArrayView<1, int32> OutStarts,
			TLearningArrayView<1, int32> OutLengths,
			const TLearningArrayView<1, const int32> LhsStarts,
			const TLearningArrayView<1, const int32> LhsLengths,
			const TLearningArrayView<1, const int32> RhsFrames)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Private::RangesFramesIntersects);

#if UE_LEARNING_ISPC
			return ispc::LearningRangesFramesIntersects(
				OutStarts.GetData(),
				OutLengths.GetData(),
				LhsStarts.GetData(),
				LhsLengths.GetData(),
				RhsFrames.GetData(),
				LhsStarts.Num(),
				RhsFrames.Num());
#else
			RangesCheck(LhsStarts, LhsLengths);
			FramesCheck(RhsFrames);

			if (LhsStarts.IsEmpty()) { return 0; }
			if (RhsFrames.IsEmpty()) { return 0; }

			// Number of ranges in lhs and rhs
			const int32 LhsNum = LhsStarts.Num();
			const int32 RhsNum = RhsFrames.Num();

			// Activation state of each list of ranges
			bool bOutActive = false;
			bool bIntersects = false;

			// Event index for each list of ranges
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both have events to process
			while (LhsIndex < LhsNum * 2 && RhsIndex < RhsNum)
			{
				// Is the next lhs event active or inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;

				// Time of the next lhs, and rhs events
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				const int32 RhsT = RhsFrames[RhsIndex];

				// Event from lhs coming first
				if (LhsT < RhsT)
				{
					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bIntersects = false;
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						if (bIntersects)
						{
							OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
							OutIndex++;
						}
					}

					LhsIndex++;
				}
				// Event from rhs coming first
				else if (RhsT < LhsT)
				{
					// Mark as intersecting
					if (bOutActive)
					{
						bIntersects = true;
					}

					RhsIndex++;
				}
				// Event from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);

					// Activate output
					if (!bOutActive && bLhsActiveNext)
					{
						bIntersects = true;
						bOutActive = true;
						OutStarts[OutIndex] = LhsT;
					}
					// Deactivate output
					else if (bOutActive && !bLhsActiveNext)
					{
						bOutActive = false;
						if (bIntersects)
						{
							OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
							OutIndex++;
						}
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// While the output is still active and intersecting
			if (bOutActive && bIntersects)
			{
				// Next lhs event should be inactive
				const bool bLhsActiveNext = LhsIndex % 2 == 0;
				check(!bLhsActiveNext);

				// Time of the next lhs event
				const int32 LhsT = bLhsActiveNext ? LhsStarts[LhsIndex / 2] : LhsStarts[LhsIndex / 2] + LhsLengths[LhsIndex / 2];
				
				// Finish writing range
				OutLengths[OutIndex] = LhsT - OutStarts[OutIndex];
				OutIndex++;
			}

			RangesCheck(OutStarts.Slice(0, OutIndex), OutLengths.Slice(0, OutIndex));

			// Return number of ranges added
			return OutIndex;
#endif
		}

	}

	void FFrameRangeSet::Check() const
	{
		check(EntrySequences.Num() == EntryRangeOffsets.Num());
		check(EntrySequences.Num() == EntryRangeNums.Num());
		check(RangeStarts.Num() == RangeLengths.Num());
		check(RangeStarts.Num() == RangeOffsets.Num());

		const int32 EntryNum = GetEntryNum();
		for (int32 EntryIdx = 0; EntryIdx < EntryNum - 1; EntryIdx++)
		{
			check(EntrySequences[EntryIdx + 0] < EntrySequences[EntryIdx + 1])
		}

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(EntryRangeNums[EntryIdx] > 0);
			FrameRangeSet::Private::RangesCheck(GetEntryRangeStarts(EntryIdx), GetEntryRangeLengths(EntryIdx));
		}
	}

	bool FFrameRangeSet::IsEmpty() const { return EntrySequences.IsEmpty(); }

	void FFrameRangeSet::Empty()
	{
		EntrySequences.Empty();
		EntryRangeOffsets.Empty();
		EntryRangeNums.Empty();
		RangeStarts.Empty();
		RangeLengths.Empty();
		RangeOffsets.Empty();
	}

	int32 FFrameRangeSet::GetEntryNum() const { return EntrySequences.Num(); }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetEntrySequences() const { return EntrySequences; }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetEntryRangeNums() const { return EntryRangeNums; }
	int32 FFrameRangeSet::GetEntrySequence(const int32 EntryIdx) const { return EntrySequences[EntryIdx]; }
	int32 FFrameRangeSet::GetEntryRangeNum(const int32 EntryIdx) const { return EntryRangeNums[EntryIdx]; }

	int32 FFrameRangeSet::GetEntryTotalFrameNum(const int32 EntryIdx) const
	{
		const int32 EntryRangeNum = GetEntryRangeNum(EntryIdx);
		return 
			EntryRangeNum == 0 ? 0 : 
			GetEntryRangeOffset(EntryIdx, EntryRangeNum - 1) - 
			GetEntryRangeOffset(EntryIdx, 0) + 
			GetEntryRangeLength(EntryIdx, EntryRangeNum - 1);
	}

	TLearningArrayView<1, const int32> FFrameRangeSet::GetEntryRangeStarts(const int32 EntryIdx) const { return RangeStarts.Slice(EntryRangeOffsets[EntryIdx], EntryRangeNums[EntryIdx]); }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetEntryRangeLengths(const int32 EntryIdx) const { return RangeLengths.Slice(EntryRangeOffsets[EntryIdx], EntryRangeNums[EntryIdx]); }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetEntryRangeOffsets(const int32 EntryIdx) const { return RangeOffsets.Slice(EntryRangeOffsets[EntryIdx], EntryRangeNums[EntryIdx]); }
	int32 FFrameRangeSet::GetEntryRangeStart(const int32 EntryIdx, const int32 RangeIdx) const { return RangeStarts[EntryRangeOffsets[EntryIdx] + RangeIdx]; }
	int32 FFrameRangeSet::GetEntryRangeLength(const int32 EntryIdx, const int32 RangeIdx) const { return RangeLengths[EntryRangeOffsets[EntryIdx] + RangeIdx]; }
	int32 FFrameRangeSet::GetEntryRangeStop(const int32 EntryIdx, const int32 RangeIdx) const { return RangeStarts[EntryRangeOffsets[EntryIdx] + RangeIdx] + RangeLengths[EntryRangeOffsets[EntryIdx] + RangeIdx]; }
	int32 FFrameRangeSet::GetEntryRangeOffset(const int32 EntryIdx, const int32 RangeIdx) const { return RangeOffsets[EntryRangeOffsets[EntryIdx] + RangeIdx]; }

	float FFrameRangeSet::GetEntryRangeStartTime(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const
	{
		return GetEntryRangeStart(EntryIdx, RangeIdx) * FrameDeltaTime;
	}
	
	float FFrameRangeSet::GetEntryRangeEndTime(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const
	{
		return (GetEntryRangeStart(EntryIdx, RangeIdx) + GetEntryRangeLength(EntryIdx, RangeIdx) - 1) * FrameDeltaTime;
	}

	float FFrameRangeSet::GetEntryRangeDuration(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const
	{
		return (GetEntryRangeLength(EntryIdx, RangeIdx) - 1) * FrameDeltaTime;
	}
	
	int32 FFrameRangeSet::GetTotalRangeNum() const { return RangeStarts.Num(); }
	
	TLearningArrayView<1, const int32> FFrameRangeSet::GetAllRangeStarts() const { return RangeStarts; }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetAllRangeLengths() const { return RangeLengths; }
	TLearningArrayView<1, const int32> FFrameRangeSet::GetAllRangeOffsets() const { return RangeOffsets; }

	int32 FFrameRangeSet::GetTotalFrameNum() const
	{
		const int32 RangeNum = GetTotalRangeNum();
		return RangeNum == 0 ? 0 : RangeOffsets[RangeNum - 1] + RangeLengths[RangeNum - 1];
	}

	float FFrameRangeSet::GetAverageFrameNum() const
	{
		return GetTotalRangeNum() == 0 ? 0 : (float)GetTotalFrameNum() / (float)GetTotalRangeNum();
	}

	int32 FFrameRangeSet::GetMinFrameNum() const
	{
		const int32 RangeNum = GetTotalRangeNum();
		if (RangeNum == 0) { return 0; }
		int32 Minimum = INT32_MAX;
		for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
		{
			Minimum = FMath::Min(Minimum, RangeLengths[RangeIdx]);
		}
		return Minimum;
	}

	int32 FFrameRangeSet::GetMaxFrameNum() const
	{
		const int32 RangeNum = GetTotalRangeNum();
		if (RangeNum == 0) { return 0; }
		int32 Maximum = INT32_MIN;
		for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
		{
			Maximum = FMath::Max(Maximum, RangeLengths[RangeIdx]);
		}
		return Maximum;
	}

	bool FFrameRangeSet::ContainsSequence(const int32 Sequence) const
	{
		return FrameRangeSet::Private::ContainsSorted(EntrySequences, Sequence);
	}

	bool FFrameRangeSet::Contains(const int32 Sequence, const int32 Frame) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		return EntryIdx != INDEX_NONE && FrameRangeSet::Private::RangesContains(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Frame);
	}

	bool FFrameRangeSet::EntryContains(const int32 EntryIdx, const int32 Frame) const
	{
		return FrameRangeSet::Private::RangesContains(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Frame);
	}

	bool FFrameRangeSet::IntersectsRange(const int32 Sequence, const int32 Start, const int32 Length) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		return EntryIdx != INDEX_NONE && FrameRangeSet::Private::RangesIntersectsRange(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Start,
			Length);
	}

	bool FFrameRangeSet::EntryContainsRange(const int32 EntryIdx, const int32 Start, const int32 Length) const
	{
		return FrameRangeSet::Private::RangesContainsRange(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Start, Length);
	}

	bool FFrameRangeSet::EntryIntersectsRange(const int32 EntryIdx, const int32 Start, const int32 Length) const
	{
		return FrameRangeSet::Private::RangesIntersectsRange(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Start,
			Length);
	}

	bool FFrameRangeSet::ContainsTime(const int32 Sequence, const float Time, const float FrameDeltaTime, const float Tolerance) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		return EntryIdx != INDEX_NONE && FrameRangeSet::Private::RangesContainsTime(
			GetEntryRangeStarts(EntryIdx),
			GetEntryRangeLengths(EntryIdx),
			Time,
			FrameDeltaTime,
			Tolerance);
	}

	int32 FFrameRangeSet::FindSequenceEntry(const int32 Sequence) const
	{
		return FrameRangeSet::Private::FindSorted(EntrySequences, Sequence);
	}

	bool FFrameRangeSet::Find(int32& OutEntryIdx, int32& OutRangeIdx, int32& OutRangeFrame, const int32 Sequence, const int32 Frame) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		if (EntryIdx != INDEX_NONE)
		{
			int32 FoundRangeIdx = INDEX_NONE;
			int32 FoundRangeFrame = INDEX_NONE;
			if (FrameRangeSet::Private::RangesFind(
				FoundRangeIdx,
				FoundRangeFrame,
				GetEntryRangeStarts(EntryIdx),
				GetEntryRangeLengths(EntryIdx),
				Frame))
			{
				OutEntryIdx = EntryIdx;
				OutRangeIdx = FoundRangeIdx;
				OutRangeFrame = FoundRangeFrame;
				return true;
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutRangeIdx = INDEX_NONE;
		OutRangeFrame = INDEX_NONE;
		return false;
	}

	bool FFrameRangeSet::FindTime(int32& OutEntryIdx, int32& OutRangeIdx, float& OutRangeTime, const int32 Sequence, const float Time, const float FrameDeltaTime, const float Tolerance) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		if (EntryIdx != INDEX_NONE)
		{
			int32 FoundRangeIdx = INDEX_NONE;
			float FoundRangeTime = -1.0f;
			if (FrameRangeSet::Private::RangesFindTime(
				FoundRangeIdx,
				FoundRangeTime,
				GetEntryRangeStarts(EntryIdx),
				GetEntryRangeLengths(EntryIdx),
				Time,
				FrameDeltaTime,
				Tolerance))
			{
				OutEntryIdx = EntryIdx;
				OutRangeIdx = FoundRangeIdx;
				OutRangeTime = FoundRangeTime;
				return true;
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutRangeIdx = INDEX_NONE;
		OutRangeTime = -1.0f;
		return false;
	}

	bool FFrameRangeSet::FindTotalRange(int32& OutEntryIdx, int32& OutRangeIdx, const int32 TotalRangeIdx) const
	{
		const int32 EntryNum = GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const int32 RangeOffset = EntryRangeOffsets[EntryIdx];
			const int32 RangeNum = EntryRangeNums[EntryIdx];

			if (TotalRangeIdx >= RangeOffset && TotalRangeIdx < RangeOffset + RangeNum)
			{
				OutEntryIdx = EntryIdx;
				OutRangeIdx = TotalRangeIdx - RangeOffset;
				return true;
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutRangeIdx = INDEX_NONE;
		return false;
	}

	bool FFrameRangeSet::FindOffset(int32& OutEntryIdx, int32& OutRangeIdx, int32& OutRangeFrame, const int32 Offset) const
	{
		const int32 EntryNum = GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const int32 RangeNum = GetEntryRangeNum(EntryIdx);

			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const int32 RangeStart = GetEntryRangeStart(EntryIdx, RangeIdx);
				const int32 RangeLength = GetEntryRangeLength(EntryIdx, RangeIdx);
				const int32 RangeOffset = GetEntryRangeOffset(EntryIdx, RangeIdx);

				if (Offset >= RangeOffset && Offset < RangeOffset + RangeLength)
				{
					OutEntryIdx = EntryIdx;
					OutRangeIdx = RangeIdx;
					OutRangeFrame = Offset - RangeOffset;
					return true;
				}
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutRangeIdx = INDEX_NONE;
		OutRangeFrame = INDEX_NONE;
		return false;
	}

	void FFrameRangeSet::AddEntry(
		const int32 InSequence,
		const TLearningArrayView<1, const int32> InStarts,
		const TLearningArrayView<1, const int32> InLengths)
	{
		check(InStarts.Num() == InLengths.Num());
		check(!ContainsSequence(InSequence));
		UE::Learning::FrameRangeSet::Private::RangesCheck(InStarts, InLengths);

		if (InStarts.IsEmpty()) { return; }

		const int32 CurrRangeNum = RangeStarts.Num();
		const int32 AddRangeNum = InStarts.Num();
		RangeStarts.SetNumUninitialized({ CurrRangeNum + AddRangeNum });
		RangeLengths.SetNumUninitialized({ CurrRangeNum + AddRangeNum });
		RangeOffsets.SetNumUninitialized({ CurrRangeNum + AddRangeNum });
		FrameRangeSet::Private::RangesCopy(
			RangeStarts.Slice(CurrRangeNum, AddRangeNum),
			RangeLengths.Slice(CurrRangeNum, AddRangeNum),
			InStarts,
			InLengths);
		for (int32 Idx = CurrRangeNum; Idx < CurrRangeNum + AddRangeNum; Idx++)
		{
			RangeOffsets[Idx] = Idx == 0 ? 0 : RangeOffsets[Idx - 1] + RangeLengths[Idx - 1];
		}

		const int32 CurrEntryNum = EntrySequences.Num();
		EntrySequences.SetNumUninitialized({ CurrEntryNum + 1 });
		EntryRangeOffsets.SetNumUninitialized({ CurrEntryNum + 1 });
		EntryRangeNums.SetNumUninitialized({ CurrEntryNum + 1 });
		EntrySequences[CurrEntryNum] = InSequence;
		EntryRangeOffsets[CurrEntryNum] = CurrRangeNum;
		EntryRangeNums[CurrEntryNum] = AddRangeNum;

		Check();
	}

	namespace FrameRangeSet
	{
		bool Equal(const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs)
		{
#if UE_LEARNING_ISPC
			return
				Lhs.EntrySequences.Num() == Rhs.EntrySequences.Num() &&
				Lhs.RangeStarts.Num() == Rhs.RangeStarts.Num() &&
				ispc::LearningInt32ArraysEqual(Lhs.EntrySequences.GetData(), Rhs.EntrySequences.GetData(), Lhs.EntrySequences.Num()) &&
				ispc::LearningInt32ArraysEqual(Lhs.EntryRangeOffsets.GetData(), Rhs.EntryRangeOffsets.GetData(), Lhs.EntryRangeOffsets.Num()) &&
				ispc::LearningInt32ArraysEqual(Lhs.EntryRangeNums.GetData(), Rhs.EntryRangeNums.GetData(), Lhs.EntryRangeNums.Num()) &&
				ispc::LearningInt32ArraysEqual(Lhs.RangeStarts.GetData(), Rhs.RangeStarts.GetData(), Lhs.RangeStarts.Num()) &&
				ispc::LearningInt32ArraysEqual(Lhs.RangeLengths.GetData(), Rhs.RangeLengths.GetData(), Lhs.RangeLengths.Num()) &&
				ispc::LearningInt32ArraysEqual(Lhs.RangeOffsets.GetData(), Rhs.RangeOffsets.GetData(), Lhs.RangeOffsets.Num());
#else
			return
				Lhs.EntrySequences.Num() == Rhs.EntrySequences.Num() &&
				Lhs.RangeStarts.Num() == Rhs.RangeStarts.Num() &&
				Array::Equal(Lhs.EntrySequences, Rhs.EntrySequences) &&
				Array::Equal(Lhs.EntryRangeOffsets, Rhs.EntryRangeOffsets) &&
				Array::Equal(Lhs.EntryRangeNums, Rhs.EntryRangeNums) &&
				Array::Equal(Lhs.RangeStarts, Rhs.RangeStarts) &&
				Array::Equal(Lhs.RangeLengths, Rhs.RangeLengths) &&
				Array::Equal(Lhs.RangeOffsets, Rhs.RangeOffsets);
#endif
		}

		void Union(FFrameRangeSet& OutFrameRangeSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Union);

			FrameSet.Check(); FrameRangeSet.Check();

			if (FrameSet.IsEmpty()) { OutFrameRangeSet = FrameRangeSet; return; }

			// Allocate potential maximum number of entries and ranges we might to output

			const int32 LhsEntryNum = FrameSet.GetEntryNum();
			const int32 RhsEntryNum = FrameRangeSet.GetEntryNum();
			const int32 LhsRangeNum = FrameSet.GetTotalFrameNum();
			const int32 RhsRangeNum = FrameRangeSet.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			OutFrameRangeSet.EntrySequences.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			OutFrameRangeSet.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			OutFrameRangeSet.EntryRangeNums.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			OutFrameRangeSet.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			OutFrameRangeSet.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Entry index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (FrameSet.GetEntrySequence(LhsIndex) < FrameRangeSet.GetEntrySequence(RhsIndex))
				{
					check(FrameSet.GetEntryFrameNum(LhsIndex) > 0);

					// Append subranges to output
					OutFrameRangeSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
					OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
					OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameSet.GetEntryFrameNum(LhsIndex);

					FrameRangeSet::Private::RangesCopyFrames(
						OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
						OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
						FrameSet.GetEntryFrames(LhsIndex));

					RangeIndex += FrameSet.GetEntryFrameNum(LhsIndex);
					OutIndex++;
					LhsIndex++;
				}
				// If entry from rhs is first
				else if (FrameRangeSet.GetEntrySequence(RhsIndex) < FrameSet.GetEntrySequence(LhsIndex))
				{
					check(FrameRangeSet.GetEntryRangeNum(RhsIndex) > 0);

					// Append subranges to output
					OutFrameRangeSet.EntrySequences[OutIndex] = FrameRangeSet.GetEntrySequence(RhsIndex);
					OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
					OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameRangeSet.GetEntryRangeNum(RhsIndex);

					FrameRangeSet::Private::RangesCopy(
						OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(RhsIndex)),
						OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(RhsIndex)),
						FrameRangeSet.GetEntryRangeStarts(RhsIndex),
						FrameRangeSet.GetEntryRangeLengths(RhsIndex));

					RangeIndex += FrameRangeSet.GetEntryRangeNum(RhsIndex);
					OutIndex++;
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(FrameSet.GetEntryFrameNum(LhsIndex) > 0);
					check(FrameRangeSet.GetEntryRangeNum(RhsIndex) > 0);

					// Append union of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::FramesRangesUnion(
						OutFrameRangeSet.RangeStarts.Slice(RangeIndex, OutFrameRangeSet.RangeStarts.Num() - RangeIndex),
						OutFrameRangeSet.RangeLengths.Slice(RangeIndex, OutFrameRangeSet.RangeLengths.Num() - RangeIndex),
						FrameSet.GetEntryFrames(LhsIndex),
						FrameRangeSet.GetEntryRangeStarts(RhsIndex),
						FrameRangeSet.GetEntryRangeLengths(RhsIndex));

					check(RangeNum > 0);
					check(RangeNum <= FrameSet.GetEntryFrameNum(LhsIndex) + FrameRangeSet.GetEntryRangeNum(RhsIndex));

					OutFrameRangeSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
					OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
					OutFrameRangeSet.EntryRangeNums[OutIndex] = RangeNum;
					RangeIndex += RangeNum;
					OutIndex++;
					LhsIndex++;
					RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				check(RhsIndex == RhsEntryNum);
				check(FrameSet.GetEntryFrameNum(LhsIndex) > 0);

				// Append subranges to output
				OutFrameRangeSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
				OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
				OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameSet.GetEntryFrameNum(LhsIndex);

				FrameRangeSet::Private::RangesCopyFrames(
					OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
					OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
					FrameSet.GetEntryFrames(LhsIndex));

				RangeIndex += FrameSet.GetEntryFrameNum(LhsIndex);
				OutIndex++;
				LhsIndex++;
			}

			// Process any remaining rhs entries
			while (RhsIndex < RhsEntryNum)
			{
				check(LhsIndex == LhsEntryNum);
				check(FrameRangeSet.GetEntryRangeNum(RhsIndex) > 0);

				// Append subranges to output
				OutFrameRangeSet.EntrySequences[OutIndex] = FrameRangeSet.GetEntrySequence(RhsIndex);
				OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
				OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameRangeSet.GetEntryRangeNum(RhsIndex);

				FrameRangeSet::Private::RangesCopy(
					OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(RhsIndex)),
					OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(RhsIndex)),
					FrameRangeSet.GetEntryRangeStarts(RhsIndex),
					FrameRangeSet.GetEntryRangeLengths(RhsIndex));

				RangeIndex += FrameRangeSet.GetEntryRangeNum(RhsIndex);
				OutIndex++;
				RhsIndex++;
			}

			// Resize output to match what was added
			OutFrameRangeSet.EntrySequences.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.EntryRangeNums.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.RangeStarts.SetNumUninitialized({ RangeIndex });
			OutFrameRangeSet.RangeLengths.SetNumUninitialized({ RangeIndex });
			OutFrameRangeSet.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(OutFrameRangeSet.RangeOffsets, OutFrameRangeSet.RangeLengths);
			OutFrameRangeSet.Check();
		}

		void Intersection(FFrameSet& OutFrameSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Intersection);

			FrameSet.Check(); FrameRangeSet.Check();

			if (FrameSet.IsEmpty()) { OutFrameSet.Empty(); return; }
			if (FrameRangeSet.IsEmpty()) { OutFrameSet.Empty(); return; }

			const int32 LhsEntryNum = FrameSet.GetEntryNum();
			const int32 RhsEntryNum = FrameRangeSet.GetEntryNum();
			const int32 LhsFrameNum = FrameSet.GetTotalFrameNum();

			// Allocate potential maximum number of entries and frames we might to output
			OutFrameSet.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.EntryFrameOffsets.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.EntryFrameNums.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.Frames.SetNumUninitialized({ LhsFrameNum });

			// Entry index for each list of entries
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output entry index
			int32 FrameIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (FrameSet.GetEntrySequence(LhsIndex) < FrameRangeSet.GetEntrySequence(RhsIndex))
				{
					LhsIndex++;
				}
				// If entry from rhs is first
				else if (FrameRangeSet.GetEntrySequence(RhsIndex) < FrameSet.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(FrameRangeSet.GetEntrySequence(RhsIndex) == FrameSet.GetEntrySequence(LhsIndex));

					// Append intersection of frames to output
					const int32 EventNum = Private::FramesRangesIntersection(
						OutFrameSet.Frames.Slice(FrameIndex, OutFrameSet.Frames.Num() - FrameIndex),
						FrameSet.GetEntryFrames(LhsIndex),
						FrameRangeSet.GetEntryRangeStarts(RhsIndex),
						FrameRangeSet.GetEntryRangeLengths(RhsIndex));

					check(EventNum <= FrameSet.GetEntryFrameNum(LhsIndex));

					if (EventNum > 0)
					{
						OutFrameSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
						OutFrameSet.EntryFrameOffsets[OutIndex] = FrameIndex;
						OutFrameSet.EntryFrameNums[OutIndex] = EventNum;
						FrameIndex += EventNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			OutFrameSet.EntrySequences.SetNumUninitialized({ OutIndex });
			OutFrameSet.EntryFrameOffsets.SetNumUninitialized({ OutIndex });
			OutFrameSet.EntryFrameNums.SetNumUninitialized({ OutIndex });
			OutFrameSet.Frames.SetNumUninitialized({ FrameIndex });
			OutFrameSet.Check();
		}

		void Difference(FFrameSet& OutFrameSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Difference);

			FrameSet.Check(); FrameRangeSet.Check();

			if (FrameSet.IsEmpty()) { OutFrameSet.Empty(); return; }
			if (FrameRangeSet.IsEmpty()) { OutFrameSet = FrameSet; return; }

			const int32 LhsEntryNum = FrameSet.GetEntryNum();
			const int32 RhsEntryNum = FrameRangeSet.GetEntryNum();
			const int32 LhsFrameNum = FrameSet.GetTotalFrameNum();

			// Allocate potential maximum number of entries and frames we might to output
			OutFrameSet.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.EntryFrameOffsets.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.EntryFrameNums.SetNumUninitialized({ LhsEntryNum });
			OutFrameSet.Frames.SetNumUninitialized({ LhsFrameNum });

			// Entry index for each list of entries
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output event index
			int32 FrameIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (FrameSet.GetEntrySequence(LhsIndex) < FrameRangeSet.GetEntrySequence(RhsIndex))
				{
					// Append frames to output
					if (FrameSet.GetEntryFrameNum(LhsIndex) > 0)
					{
						OutFrameSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
						OutFrameSet.EntryFrameOffsets[OutIndex] = FrameIndex;
						OutFrameSet.EntryFrameNums[OutIndex] = FrameSet.GetEntryFrameNum(LhsIndex);

						FrameRangeSet::Private::FramesCopy(
							OutFrameSet.Frames.Slice(FrameIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
							FrameSet.GetEntryFrames(LhsIndex));

						FrameIndex += FrameSet.GetEntryFrameNum(LhsIndex);
						OutIndex++;
					}

					LhsIndex++;
				}
				// If entry from rhs is first
				else if (FrameRangeSet.GetEntrySequence(RhsIndex) < FrameSet.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(FrameRangeSet.GetEntrySequence(RhsIndex) == FrameSet.GetEntrySequence(LhsIndex));

					// Append difference of frames to output
					const int32 EventNum = Private::FramesRangesDifference(
						OutFrameSet.Frames.Slice(FrameIndex, OutFrameSet.Frames.Num() - FrameIndex),
						FrameSet.GetEntryFrames(LhsIndex),
						FrameRangeSet.GetEntryRangeStarts(RhsIndex),
						FrameRangeSet.GetEntryRangeLengths(RhsIndex));

					check(EventNum <= FrameSet.GetEntryFrameNum(LhsIndex));

					if (EventNum > 0)
					{
						OutFrameSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
						OutFrameSet.EntryFrameOffsets[OutIndex] = FrameIndex;
						OutFrameSet.EntryFrameNums[OutIndex] = EventNum;
						FrameIndex += EventNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				// Append frames to output
				if (FrameSet.GetEntryFrameNum(LhsIndex) > 0)
				{
					OutFrameSet.EntrySequences[OutIndex] = FrameSet.GetEntrySequence(LhsIndex);
					OutFrameSet.EntryFrameOffsets[OutIndex] = FrameIndex;
					OutFrameSet.EntryFrameNums[OutIndex] = FrameSet.GetEntryFrameNum(LhsIndex);

					FrameRangeSet::Private::FramesCopy(
						OutFrameSet.Frames.Slice(FrameIndex, FrameSet.GetEntryFrameNum(LhsIndex)),
						FrameSet.GetEntryFrames(LhsIndex));

					FrameIndex += FrameSet.GetEntryFrameNum(LhsIndex);
					OutIndex++;
				}

				LhsIndex++;
			}

			// Resize output to match what was added
			OutFrameSet.EntrySequences.SetNumUninitialized({ OutIndex });
			OutFrameSet.EntryFrameOffsets.SetNumUninitialized({ OutIndex });
			OutFrameSet.EntryFrameNums.SetNumUninitialized({ OutIndex });
			OutFrameSet.Frames.SetNumUninitialized({ FrameIndex });
			OutFrameSet.Check();
		}

		void Difference(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Difference);

			FrameRangeSet.Check(); FrameSet.Check();

			if (FrameRangeSet.IsEmpty()) { OutFrameRangeSet.Empty(); return; }
			if (FrameSet.IsEmpty()) { OutFrameRangeSet = FrameRangeSet; return; }

			const int32 LhsEntryNum = FrameRangeSet.GetEntryNum();
			const int32 RhsEntryNum = FrameSet.GetEntryNum();
			const int32 LhsRangeNum = FrameRangeSet.GetTotalRangeNum();
			const int32 RhsRangeNum = FrameSet.GetTotalFrameNum();

			// Allocate potential maximum number of entries and ranges we might to output
			OutFrameRangeSet.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			OutFrameRangeSet.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum });
			OutFrameRangeSet.EntryRangeNums.SetNumUninitialized({ LhsEntryNum });
			OutFrameRangeSet.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			OutFrameRangeSet.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (FrameRangeSet.GetEntrySequence(LhsIndex) < FrameSet.GetEntrySequence(RhsIndex))
				{
					check(FrameRangeSet.GetEntryRangeNum(LhsIndex) > 0);

					// Append subranges to output
					OutFrameRangeSet.EntrySequences[OutIndex] = FrameRangeSet.GetEntrySequence(LhsIndex);
					OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
					OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameRangeSet.GetEntryRangeNum(LhsIndex);

					FrameRangeSet::Private::RangesCopy(
						OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(LhsIndex)),
						OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(LhsIndex)),
						FrameRangeSet.GetEntryRangeStarts(LhsIndex),
						FrameRangeSet.GetEntryRangeLengths(LhsIndex));

					RangeIndex += FrameRangeSet.GetEntryRangeNum(LhsIndex);
					OutIndex++;
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (FrameSet.GetEntrySequence(RhsIndex) < FrameRangeSet.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(FrameRangeSet.GetEntryRangeNum(LhsIndex) > 0);
					check(FrameSet.GetEntryFrameNum(RhsIndex) > 0);

					// Append difference of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::RangesFramesDifference(
						OutFrameRangeSet.RangeStarts.Slice(RangeIndex, OutFrameRangeSet.RangeStarts.Num() - RangeIndex),
						OutFrameRangeSet.RangeLengths.Slice(RangeIndex, OutFrameRangeSet.RangeLengths.Num() - RangeIndex),
						FrameRangeSet.GetEntryRangeStarts(LhsIndex),
						FrameRangeSet.GetEntryRangeLengths(LhsIndex),
						FrameSet.GetEntryFrames(RhsIndex));

					check(RangeNum <= FrameRangeSet.GetEntryRangeNum(LhsIndex) + FrameSet.GetEntryFrameNum(RhsIndex));

					if (RangeNum > 0)
					{
						OutFrameRangeSet.EntrySequences[OutIndex] = FrameRangeSet.GetEntrySequence(LhsIndex);
						OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
						OutFrameRangeSet.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				check(FrameRangeSet.GetEntryRangeNum(LhsIndex) > 0);

				// Append subranges to output
				OutFrameRangeSet.EntrySequences[OutIndex] = FrameRangeSet.GetEntrySequence(LhsIndex);
				OutFrameRangeSet.EntryRangeOffsets[OutIndex] = RangeIndex;
				OutFrameRangeSet.EntryRangeNums[OutIndex] = FrameRangeSet.GetEntryRangeNum(LhsIndex);

				FrameRangeSet::Private::RangesCopy(
					OutFrameRangeSet.RangeStarts.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(LhsIndex)),
					OutFrameRangeSet.RangeLengths.Slice(RangeIndex, FrameRangeSet.GetEntryRangeNum(LhsIndex)),
					FrameRangeSet.GetEntryRangeStarts(LhsIndex),
					FrameRangeSet.GetEntryRangeLengths(LhsIndex));

				RangeIndex += FrameRangeSet.GetEntryRangeNum(LhsIndex);
				OutIndex++;
				LhsIndex++;
			}

			// Resize output to match what was added
			OutFrameRangeSet.EntrySequences.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.EntryRangeNums.SetNumUninitialized({ OutIndex });
			OutFrameRangeSet.RangeStarts.SetNumUninitialized({ RangeIndex });
			OutFrameRangeSet.RangeLengths.SetNumUninitialized({ RangeIndex });
			OutFrameRangeSet.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(OutFrameRangeSet.RangeOffsets, OutFrameRangeSet.RangeLengths);
			OutFrameRangeSet.Check();
		}

		void Union(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Union);

			Lhs.Check(); Rhs.Check();

			if (Lhs.IsEmpty()) { Out = Rhs; return; }
			if (Rhs.IsEmpty()) { Out = Lhs; return; }
			if (Equal(Lhs, Rhs)) { Out = Lhs; return; }

			// Allocate potential maximum number of entries and ranges we might to output

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();
			const int32 RhsRangeNum = Rhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Entry index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);

					// Append subranges to output
					Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
					Out.EntryRangeOffsets[OutIndex] = RangeIndex;
					Out.EntryRangeNums[OutIndex] = Lhs.GetEntryRangeNum(LhsIndex);

					FrameRangeSet::Private::RangesCopy(
						Out.RangeStarts.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
						Out.RangeLengths.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex));

					RangeIndex += Lhs.GetEntryRangeNum(LhsIndex);
					OutIndex++;
					LhsIndex++;
				}
				// If entry from rhs is first
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append subranges to output
					Out.EntrySequences[OutIndex] = Rhs.GetEntrySequence(RhsIndex);
					Out.EntryRangeOffsets[OutIndex] = RangeIndex;
					Out.EntryRangeNums[OutIndex] = Rhs.GetEntryRangeNum(RhsIndex);

					FrameRangeSet::Private::RangesCopy(
						Out.RangeStarts.Slice(RangeIndex, Rhs.GetEntryRangeNum(RhsIndex)),
						Out.RangeLengths.Slice(RangeIndex, Rhs.GetEntryRangeNum(RhsIndex)),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex));

					RangeIndex += Rhs.GetEntryRangeNum(RhsIndex);
					OutIndex++;
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append union of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::RangesUnion(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex));

					check(RangeNum > 0);
					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex) + Rhs.GetEntryRangeNum(RhsIndex));

					Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
					Out.EntryRangeOffsets[OutIndex] = RangeIndex;
					Out.EntryRangeNums[OutIndex] = RangeNum;
					RangeIndex += RangeNum;
					OutIndex++;
					LhsIndex++;
					RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				check(RhsIndex == RhsEntryNum);
				check(Lhs.GetEntryRangeNum(LhsIndex) > 0);

				// Append subranges to output
				Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
				Out.EntryRangeOffsets[OutIndex] = RangeIndex;
				Out.EntryRangeNums[OutIndex] = Lhs.GetEntryRangeNum(LhsIndex);

				FrameRangeSet::Private::RangesCopy(
					Out.RangeStarts.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
					Out.RangeLengths.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
					Lhs.GetEntryRangeStarts(LhsIndex),
					Lhs.GetEntryRangeLengths(LhsIndex));

				RangeIndex += Lhs.GetEntryRangeNum(LhsIndex);
				OutIndex++;
				LhsIndex++;
			}

			// Process any remaining rhs entries
			while (RhsIndex < RhsEntryNum)
			{
				check(LhsIndex == LhsEntryNum);
				check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

				// Append subranges to output
				Out.EntrySequences[OutIndex] = Rhs.GetEntrySequence(RhsIndex);
				Out.EntryRangeOffsets[OutIndex] = RangeIndex;
				Out.EntryRangeNums[OutIndex] = Rhs.GetEntryRangeNum(RhsIndex);

				FrameRangeSet::Private::RangesCopy(
					Out.RangeStarts.Slice(RangeIndex, Rhs.GetEntryRangeNum(RhsIndex)),
					Out.RangeLengths.Slice(RangeIndex, Rhs.GetEntryRangeNum(RhsIndex)),
					Rhs.GetEntryRangeStarts(RhsIndex),
					Rhs.GetEntryRangeLengths(RhsIndex));

				RangeIndex += Rhs.GetEntryRangeNum(RhsIndex);
				OutIndex++;
				RhsIndex++;
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void Intersection(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Intersection);

			Lhs.Check(); Rhs.Check();

			if (Lhs.IsEmpty()) { Out.Empty(); return; }
			if (Rhs.IsEmpty()) { Out.Empty(); return; }
			if (Equal(Lhs, Rhs)) { Out = Lhs; return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();
			const int32 RhsRangeNum = Rhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryRangeOffsets.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryRangeNums.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry is in lhs but not rhs skip
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append intersection of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::RangesIntersection(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex));

					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex) + Rhs.GetEntryRangeNum(RhsIndex));

					if (RangeNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryRangeOffsets[OutIndex] = RangeIndex;
						Out.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void Difference(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Difference);

			Lhs.Check(); Rhs.Check();

			if (Lhs.IsEmpty()) { Out.Empty(); return; }
			if (Rhs.IsEmpty()) { Out = Lhs; return; }
			if (Equal(Lhs, Rhs)) { Out.Empty(); return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();
			const int32 RhsRangeNum = Rhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ LhsEntryNum });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);

					// Append subranges to output
					Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
					Out.EntryRangeOffsets[OutIndex] = RangeIndex;
					Out.EntryRangeNums[OutIndex] = Lhs.GetEntryRangeNum(LhsIndex);

					FrameRangeSet::Private::RangesCopy(
						Out.RangeStarts.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
						Out.RangeLengths.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex));

					RangeIndex += Lhs.GetEntryRangeNum(LhsIndex);
					OutIndex++;
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append difference of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::RangesDifference(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex));

					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex) + Rhs.GetEntryRangeNum(RhsIndex));

					if (RangeNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryRangeOffsets[OutIndex] = RangeIndex;
						Out.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				check(Lhs.GetEntryRangeNum(LhsIndex) > 0);

				// Append subranges to output
				Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
				Out.EntryRangeOffsets[OutIndex] = RangeIndex;
				Out.EntryRangeNums[OutIndex] = Lhs.GetEntryRangeNum(LhsIndex);

				FrameRangeSet::Private::RangesCopy(
					Out.RangeStarts.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
					Out.RangeLengths.Slice(RangeIndex, Lhs.GetEntryRangeNum(LhsIndex)),
					Lhs.GetEntryRangeStarts(LhsIndex),
					Lhs.GetEntryRangeLengths(LhsIndex));

				RangeIndex += Lhs.GetEntryRangeNum(LhsIndex);
				OutIndex++;
				LhsIndex++;
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		int32 IntersectionWithOffsets(
			FFrameRangeSet& Out,
			TLearningArrayView<1, int32> OutLhsOffsets,
			TLearningArrayView<1, int32> OutRhsOffsets,
			const FFrameRangeSet& Lhs,
			const FFrameRangeSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::IntersectionWithOffsets);

			Lhs.Check(); Rhs.Check();

			if (Equal(Lhs, Rhs))
			{
				Out = Lhs;
				Array::Copy(OutLhsOffsets.Slice(0, Lhs.GetTotalRangeNum()), Lhs.GetAllRangeOffsets());
				Array::Copy(OutRhsOffsets.Slice(0, Rhs.GetTotalRangeNum()), Rhs.GetAllRangeOffsets());
				return Out.GetTotalRangeNum();
			}

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();
			const int32 RhsRangeNum = Rhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryRangeOffsets.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryRangeNums.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum + RhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// Frame Offsets
			int32 OutIntersectionIdx = 0;
			int32 LhsOffset = 0;
			int32 RhsOffset = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry is in lhs but not rhs skip
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					LhsOffset += Lhs.GetEntryTotalFrameNum(LhsIndex);
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsOffset += Rhs.GetEntryTotalFrameNum(RhsIndex);
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append intersection of subranges to output
					const int32 RangeNum = FrameRangeSet::Private::RangesIntersectionWithOffsets(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						OutLhsOffsets.Slice(RangeIndex, OutLhsOffsets.Num() - RangeIndex),
						OutRhsOffsets.Slice(RangeIndex, OutRhsOffsets.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex),
						LhsOffset,
						RhsOffset);

					Private::OffsetsCheck(OutLhsOffsets.Slice(0, RangeIndex + RangeNum), 0, LhsOffset + Lhs.GetEntryTotalFrameNum(LhsIndex));
					Private::OffsetsCheck(OutRhsOffsets.Slice(0, RangeIndex + RangeNum), 0, RhsOffset + Rhs.GetEntryTotalFrameNum(RhsIndex));

					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex) + Rhs.GetEntryRangeNum(RhsIndex));

					if (RangeNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryRangeOffsets[OutIndex] = RangeIndex;
						Out.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsOffset += Lhs.GetEntryTotalFrameNum(LhsIndex);
					RhsOffset += Rhs.GetEntryTotalFrameNum(RhsIndex);
					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();

			Private::OffsetsCheck(OutLhsOffsets.Slice(0, RangeIndex), 0, Lhs.GetTotalFrameNum());
			Private::OffsetsCheck(OutRhsOffsets.Slice(0, RangeIndex), 0, Rhs.GetTotalFrameNum());

			check(RangeIndex == Out.GetTotalRangeNum());
			return RangeIndex;
		}

		void Bounds(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet)
		{
			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			Out.EntrySequences.SetNumUninitialized({ EntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ EntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ EntryNum });
			Out.RangeStarts.SetNumUninitialized({ EntryNum });
			Out.RangeLengths.SetNumUninitialized({ EntryNum });
			
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				Out.EntrySequences[EntryIdx] = FrameRangeSet.GetEntrySequence(EntryIdx);
				Out.EntryRangeOffsets[EntryIdx] = EntryIdx;
				Out.EntryRangeNums[EntryIdx] = 1;
				Out.RangeStarts[EntryIdx] = FrameRangeSet.GetEntryRangeStart(EntryIdx, 0);
				Out.RangeLengths[EntryIdx] = 
					FrameRangeSet.GetEntryRangeStop(EntryIdx, FrameRangeSet.GetEntryRangeNum(EntryIdx) - 1) - 
					FrameRangeSet.GetEntryRangeStart(EntryIdx, 0);
			}
			
			Out.RangeOffsets.SetNumUninitialized({ EntryNum });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void Bounds(FFrameRangeSet& Out, const FFrameSet& FrameSet)
		{
			const int32 EntryNum = FrameSet.GetEntryNum();

			Out.EntrySequences.SetNumUninitialized({ EntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ EntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ EntryNum });
			Out.RangeStarts.SetNumUninitialized({ EntryNum });
			Out.RangeLengths.SetNumUninitialized({ EntryNum });

			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				Out.EntrySequences[EntryIdx] = FrameSet.GetEntrySequence(EntryIdx);
				Out.EntryRangeOffsets[EntryIdx] = EntryIdx;
				Out.EntryRangeNums[EntryIdx] = 1;
				Out.RangeStarts[EntryIdx] = FrameSet.GetEntryFrame(EntryIdx, 0);
				Out.RangeLengths[EntryIdx] =
					FrameSet.GetEntryFrame(EntryIdx, FrameSet.GetEntryFrameNum(EntryIdx) - 1) + 1 -
					FrameSet.GetEntryFrame(EntryIdx, 0);
			}

			Out.RangeOffsets.SetNumUninitialized({ EntryNum });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void Intersects(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Intersects);

			Lhs.Check(); Rhs.Check();

			if (Lhs.IsEmpty()) { Out.Empty(); return; }
			if (Rhs.IsEmpty()) { Out.Empty(); return; }
			if (Equal(Lhs, Rhs)) { Out = Lhs; return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ LhsEntryNum });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry is in lhs but not rhs skip
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryRangeNum(RhsIndex) > 0);

					// Append intersecting frames to output
					const int32 RangeNum = FrameRangeSet::Private::RangesIntersects(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryRangeStarts(RhsIndex),
						Rhs.GetEntryRangeLengths(RhsIndex));

					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex));

					if (RangeNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryRangeOffsets[OutIndex] = RangeIndex;
						Out.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void Intersects(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameRangeSet::Intersects);

			Lhs.Check(); Rhs.Check();

			if (Lhs.IsEmpty()) { Out.Empty(); return; }
			if (Rhs.IsEmpty()) { Out.Empty(); return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsRangeNum = Lhs.GetTotalRangeNum();

			// Allocate potential maximum number of entries and ranges we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeOffsets.SetNumUninitialized({ LhsEntryNum });
			Out.EntryRangeNums.SetNumUninitialized({ LhsEntryNum });
			Out.RangeStarts.SetNumUninitialized({ LhsRangeNum });
			Out.RangeLengths.SetNumUninitialized({ LhsRangeNum });

			// Anim index for each list of ranges
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output ranges index
			int32 RangeIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry is in lhs but not rhs skip
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					LhsIndex++;
				}
				// If entry is in rhs but not lhs skip
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If entry is in both lhs and rhs
				else
				{
					check(Lhs.GetEntryRangeNum(LhsIndex) > 0);
					check(Rhs.GetEntryFrameNum(RhsIndex) > 0);

					// Append intersecting frames to output
					const int32 RangeNum = FrameRangeSet::Private::RangesFramesIntersects(
						Out.RangeStarts.Slice(RangeIndex, Out.RangeStarts.Num() - RangeIndex),
						Out.RangeLengths.Slice(RangeIndex, Out.RangeLengths.Num() - RangeIndex),
						Lhs.GetEntryRangeStarts(LhsIndex),
						Lhs.GetEntryRangeLengths(LhsIndex),
						Rhs.GetEntryFrames(RhsIndex));

					check(RangeNum <= Lhs.GetEntryRangeNum(LhsIndex));

					if (RangeNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryRangeOffsets[OutIndex] = RangeIndex;
						Out.EntryRangeNums[OutIndex] = RangeNum;
						RangeIndex += RangeNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryRangeOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryRangeNums.SetNumUninitialized({ OutIndex });
			Out.RangeStarts.SetNumUninitialized({ RangeIndex });
			Out.RangeLengths.SetNumUninitialized({ RangeIndex });
			Out.RangeOffsets.SetNumUninitialized({ RangeIndex });
			Private::ComputeRangeOffsets(Out.RangeOffsets, Out.RangeLengths);
			Out.Check();
		}

		void TrimStart(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimFrameNum)
		{
			Trim(Out, FrameRangeSet, TrimFrameNum, 0);
		}

		void TrimEnd(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimFrameNum)
		{
			Trim(Out, FrameRangeSet, 0, TrimFrameNum);
		}

		void Trim(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimStartFrameNum, const int32 TrimEndFrameNum)
		{
			check(TrimStartFrameNum >= 0);
			check(TrimEndFrameNum >= 0);
			FrameRangeSet.Check();

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			Out.Empty();

			TArray<int32, TInlineAllocator<64>> RangeStartsAdded;
			TArray<int32, TInlineAllocator<64>> RangeLengthsAdded;

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 Sequence = FrameRangeSet.GetEntrySequence(EntryIdx);
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				RangeStartsAdded.Reset();
				RangeLengthsAdded.Reset();
				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 RangeStart = FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx) + TrimStartFrameNum;
					const int32 RangeLength = FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx) - TrimStartFrameNum - TrimEndFrameNum;
					if (RangeLength > 0)
					{
						RangeStartsAdded.Add(RangeStart);
						RangeLengthsAdded.Add(RangeLength);
					}
				}

				Out.AddEntry(Sequence, RangeStartsAdded, RangeLengthsAdded);
			}

			Out.Check();
		}

		void PadStart(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadFrameNum)
		{
			Pad(Out, FrameRangeSet, PadFrameNum, 0);
		}

		void PadEnd(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadFrameNum)
		{
			Pad(Out, FrameRangeSet, 0, PadFrameNum);
		}

		void Pad(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadStartFrameNum, int32 PadEndFrameNum)
		{
			check(PadStartFrameNum >= 0);
			check(PadEndFrameNum >= 0);
			FrameRangeSet.Check();

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			Out.Empty();

			TArray<int32, TInlineAllocator<64>> RangeStartsAdded;
			TArray<int32, TInlineAllocator<64>> RangeLengthsAdded;

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 Sequence = FrameRangeSet.GetEntrySequence(EntryIdx);
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				RangeStartsAdded.Reset();
				RangeLengthsAdded.Reset();

				bool bRangeActive = false;
				int32 RangeActiveStart = INDEX_NONE;
				int32 RangeActiveLength = INDEX_NONE;

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					if (!bRangeActive)
					{
						bRangeActive = true;
						RangeActiveStart = FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx) - PadStartFrameNum;
						RangeActiveLength = FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx) + PadStartFrameNum + PadEndFrameNum;
					}

					if (RangeIdx < RangeNum - 1)
					{
						const int32 NextRangeStart = FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx + 1) - PadStartFrameNum;
						const int32 NextRangeLength = FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx + 1) + PadStartFrameNum + PadEndFrameNum;

						if (NextRangeStart < RangeActiveStart + RangeActiveLength)
						{
							RangeActiveLength = NextRangeStart + NextRangeLength - RangeActiveStart;
							continue;
						}
						else
						{
							RangeStartsAdded.Add(RangeActiveStart);
							RangeLengthsAdded.Add(RangeActiveLength);
							bRangeActive = false;
							RangeActiveStart = INDEX_NONE;
							RangeActiveLength = INDEX_NONE;
						}
					}
					else
					{
						RangeStartsAdded.Add(RangeActiveStart);
						RangeLengthsAdded.Add(RangeActiveLength);
						bRangeActive = false;
						RangeActiveStart = INDEX_NONE;
						RangeActiveLength = INDEX_NONE;
					}
				}

				Out.AddEntry(Sequence, RangeStartsAdded, RangeLengthsAdded);
			}

			Out.Check();
		}

		void GatherRanges(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const FIndexSet Indices)
		{
			FrameRangeSet.Check();

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			Out.Empty();

			TArray<int32, TInlineAllocator<64>> RangeStartsAdded;
			TArray<int32, TInlineAllocator<64>> RangeLengthsAdded;

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 Sequence = FrameRangeSet.GetEntrySequence(EntryIdx);
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				RangeStartsAdded.Reset();
				RangeLengthsAdded.Reset();

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					if (Indices.Contains(RangeOffset))
					{
						RangeStartsAdded.Add(FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx));
						RangeLengthsAdded.Add(FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx));
					}

					RangeOffset++;
				}

				Out.AddEntry(Sequence, RangeStartsAdded, RangeLengthsAdded);
			}

			Out.Check();
		}

		void MakeFromFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameSet& FrameSet)
		{
			OutFrameRangeSet.EntrySequences = FrameSet.EntrySequences;
			OutFrameRangeSet.EntryRangeNums = FrameSet.EntryFrameNums;
			OutFrameRangeSet.EntryRangeOffsets = FrameSet.EntryFrameOffsets;
			OutFrameRangeSet.RangeStarts = FrameSet.Frames;
			OutFrameRangeSet.RangeLengths.SetNumUninitialized({ FrameSet.GetTotalFrameNum() });
			OutFrameRangeSet.RangeOffsets.SetNumUninitialized({ FrameSet.GetTotalFrameNum() });
			Array::Set(OutFrameRangeSet.RangeLengths, 1);
			Private::ComputeRangeOffsets(OutFrameRangeSet.RangeOffsets, OutFrameRangeSet.RangeLengths);
		}

		void MakeFrameSetFromRangeStarts(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet)
		{
			OutFrameSet.EntrySequences = FrameRangeSet.EntrySequences;
			OutFrameSet.EntryFrameOffsets = FrameRangeSet.EntryRangeOffsets;
			OutFrameSet.EntryFrameNums = FrameRangeSet.EntryRangeNums;
			OutFrameSet.Frames = FrameRangeSet.RangeStarts;
		}

		void MakeFrameSetFromRangeEnds(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet)
		{
			OutFrameSet.EntrySequences = FrameRangeSet.EntrySequences;
			OutFrameSet.EntryFrameOffsets = FrameRangeSet.EntryRangeOffsets;
			OutFrameSet.EntryFrameNums = FrameRangeSet.EntryRangeNums;

			const int32 RangeNum = FrameRangeSet.RangeStarts.Num();
			OutFrameSet.Frames.SetNumUninitialized({ RangeNum });
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				OutFrameSet.Frames[RangeIdx] = FrameRangeSet.RangeStarts[RangeIdx] + FrameRangeSet.RangeLengths[RangeIdx] - 1;
			}
		}

		void MakeFrameSetFromRangeInterval(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet, const int32 IntervalFrames)
		{
			check(IntervalFrames > 0);

			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			OutFrameSet.EntrySequences = FrameRangeSet.EntrySequences;
			OutFrameSet.EntryFrameOffsets.SetNumUninitialized({ EntryNum });
			OutFrameSet.EntryFrameNums.SetNumUninitialized({ EntryNum });
			Array::Zero(OutFrameSet.EntryFrameOffsets);
			Array::Zero(OutFrameSet.EntryFrameNums);

			int32 TotalFrameNum = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					check(FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx) > 0);
					TotalFrameNum += (FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx) - 1) / IntervalFrames + 1;
				}
			}

			OutFrameSet.Frames.SetNumUninitialized({ TotalFrameNum });

			int32 FrameOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				OutFrameSet.EntryFrameOffsets[EntryIdx] = FrameOffset;

				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 RangeLength = FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
					const int32 RangeStart = FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);

					for (int32 FrameIdx = RangeStart; FrameIdx < RangeStart + RangeLength; FrameIdx += IntervalFrames)
					{
						OutFrameSet.EntryFrameNums[EntryIdx]++;
						OutFrameSet.Frames[FrameOffset] = FrameIdx;
						FrameOffset++;
					}
				}
			}

			check(FrameOffset == TotalFrameNum);
		}

		void RangesBeforeFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet)
		{
			OutFrameRangeSet.Empty();

			const int32 RangeSetEntryNum = FrameRangeSet.GetEntryNum();
			const int32 FrameSetEntryNum = FrameSet.GetEntryNum();

			TArray<int32, TInlineAllocator<64>> RangeStartsAdded;
			TArray<int32, TInlineAllocator<64>> RangeLengthsAdded;

			int32 RangeOffset = 0;
			for (int32 RangeSetEntryIdx = 0; RangeSetEntryIdx < RangeSetEntryNum; RangeSetEntryIdx++)
			{
				const int32 Sequence = FrameRangeSet.GetEntrySequence(RangeSetEntryIdx);
				const int32 FrameSetEntryIdx = FrameSet.FindSequenceEntry(Sequence);

				if (FrameSetEntryIdx != INDEX_NONE)
				{
					const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(RangeSetEntryIdx);
					const int32 FrameNum = FrameSet.GetEntryFrameNum(FrameSetEntryIdx);

					RangeStartsAdded.Reset();
					RangeLengthsAdded.Reset();
					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						const int32 RangeStart = FrameRangeSet.GetEntryRangeStart(RangeSetEntryIdx, RangeIdx);
						const int32 RangeLength = FrameRangeSet.GetEntryRangeLength(RangeSetEntryIdx, RangeIdx);

						for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
						{
							const int32 Frame = FrameSet.GetEntryFrame(FrameSetEntryIdx, FrameIdx);
							if (Frame > RangeStart && Frame < RangeStart + RangeLength)
							{
								RangeStartsAdded.Add(RangeStart);
								RangeLengthsAdded.Add(Frame - RangeStart);
							}
						}
					}

					OutFrameRangeSet.AddEntry(Sequence, RangeStartsAdded, RangeLengthsAdded);
				}
			}

			OutFrameRangeSet.Check();
		}

		void RangesAfterFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet)
		{
			OutFrameRangeSet.Empty();

			const int32 RangeSetEntryNum = FrameRangeSet.GetEntryNum();
			const int32 FrameSetEntryNum = FrameSet.GetEntryNum();

			TArray<int32, TInlineAllocator<64>> RangeStartsAdded;
			TArray<int32, TInlineAllocator<64>> RangeLengthsAdded;

			int32 RangeOffset = 0;
			for (int32 RangeSetEntryIdx = 0; RangeSetEntryIdx < RangeSetEntryNum; RangeSetEntryIdx++)
			{
				const int32 Sequence = FrameRangeSet.GetEntrySequence(RangeSetEntryIdx);
				const int32 FrameSetEntryIdx = FrameSet.FindSequenceEntry(Sequence);

				if (FrameSetEntryIdx != INDEX_NONE)
				{
					const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(RangeSetEntryIdx);
					const int32 FrameNum = FrameSet.GetEntryFrameNum(FrameSetEntryIdx);

					RangeStartsAdded.Reset();
					RangeLengthsAdded.Reset();
					for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
					{
						const int32 RangeStart = FrameRangeSet.GetEntryRangeStart(RangeSetEntryIdx, RangeIdx);
						const int32 RangeLength = FrameRangeSet.GetEntryRangeLength(RangeSetEntryIdx, RangeIdx);

						for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
						{
							const int32 Frame = FrameSet.GetEntryFrame(FrameSetEntryIdx, FrameIdx);
							if (Frame >= RangeStart && Frame < RangeStart + RangeLength)
							{
								RangeStartsAdded.Add(Frame);
								RangeLengthsAdded.Add(RangeLength - (Frame - RangeStart));
							}
						}
					}

					OutFrameRangeSet.AddEntry(Sequence, RangeStartsAdded, RangeLengthsAdded);
				}
			}

			OutFrameRangeSet.Check();
		}

		void AllRangeEntries(
			TLearningArrayView<1, int32> OutRangeEntries,
			const FFrameRangeSet& FrameRangeSet)
		{
			check(OutRangeEntries.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);
				Array::Set(OutRangeEntries.Slice(RangeOffset, RangeNum), EntryIdx);
				RangeOffset += RangeNum;
			}

			check(RangeOffset == TotalRangeNum);
		}

		void AllRangeIndices(
			TLearningArrayView<1, int32> OutRangeIndices,
			const FFrameRangeSet& FrameRangeSet)
		{
			check(OutRangeIndices.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);
				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					OutRangeIndices[RangeOffset + RangeIdx] = RangeIdx;
				}
				RangeOffset += RangeNum;
			}

			check(RangeOffset == TotalRangeNum);
		}

		void AllRangeSequences(
			TLearningArrayView<1, int32> OutRangeSequences,
			const FFrameRangeSet& FrameRangeSet)
		{
			check(OutRangeSequences.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			int32 RangeOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);
				Array::Set(OutRangeSequences.Slice(RangeOffset, RangeNum), FrameRangeSet.GetEntrySequence(EntryIdx));
				RangeOffset += RangeNum;
			}

			check(RangeOffset == TotalRangeNum);
		}

		void AllRangeStartTimes(
			TLearningArrayView<1, float> OutRangeStartTimes,
			const FFrameRangeSet& FrameRangeSet, 
			const float FrameDeltaTime)
		{
			check(OutRangeStartTimes.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
			{
				OutRangeStartTimes[RangeIdx] = FrameRangeSet.RangeStarts[RangeIdx] * FrameDeltaTime;
			}
		}

		void AllRangeEndTimes(
			TLearningArrayView<1, float> OutRangeEndTimes,
			const FFrameRangeSet& FrameRangeSet, 
			const float FrameDeltaTime)
		{
			check(OutRangeEndTimes.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
			{
				OutRangeEndTimes[RangeIdx] = (FrameRangeSet.RangeStarts[RangeIdx] + FrameRangeSet.RangeLengths[RangeIdx] - 1) * FrameDeltaTime;
			}
		}

		void AllRangeDurations(
			TLearningArrayView<1, float> OutRangeDurations,
			const FFrameRangeSet& FrameRangeSet,
			const float FrameDeltaTime)
		{
			check(OutRangeDurations.Num() == FrameRangeSet.GetTotalRangeNum());

			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();
			for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
			{
				OutRangeDurations[RangeIdx] = (FrameRangeSet.RangeLengths[RangeIdx] - 1) * FrameDeltaTime;
			}
		}

		void ForEachRange(
			const FFrameRangeSet& FrameRangeSet,
			const TFunctionRef<void(
				const int32 TotalRangeIdx,
				const int32 EntryIdx,
				const int32 RangeIdx)> Body)
		{
			const int32 EntryNum = FrameRangeSet.GetEntryNum();

			int32 TotalRangeIdx = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 RangeNum = FrameRangeSet.GetEntryRangeNum(EntryIdx);
				
				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					Body(
						TotalRangeIdx,
						EntryIdx,
						RangeIdx);

					TotalRangeIdx++;
				}
			}
		}

		void ParallelForEachRange(
			const FFrameRangeSet& FrameRangeSet,
			const TFunctionRef<void(
				const int32 TotalRangeIdx,
				const int32 EntryIdx,
				const int32 RangeIdx)> Body)
		{
			const int32 TotalRangeNum = FrameRangeSet.GetTotalRangeNum();

			TLearningArray<1, int32> RangeEntries;
			TLearningArray<1, int32> RangeIndices;

			RangeEntries.SetNumUninitialized({ TotalRangeNum });
			RangeIndices.SetNumUninitialized({ TotalRangeNum });

			AllRangeEntries(RangeEntries, FrameRangeSet);
			AllRangeIndices(RangeIndices, FrameRangeSet);

			ParallelFor(TotalRangeNum, [&Body, &RangeEntries, &RangeIndices](const int32 TotalRangeIdx) {
				Body(
					TotalRangeIdx,
					RangeEntries[TotalRangeIdx],
					RangeIndices[TotalRangeIdx]);
			});
		}
	}
}
