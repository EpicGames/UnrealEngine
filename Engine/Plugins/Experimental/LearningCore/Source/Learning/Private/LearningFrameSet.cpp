// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningFrameSet.h"

#include "Async/ParallelFor.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	namespace FrameSet::Private
	{
		static inline bool ContainsSorted(const TLearningArrayView<1, const int32> Elements, const int32 Element)
		{
			const int32 Num = Elements.Num();

#if UE_LEARNING_ISPC
			return ispc::LearningContainsSorted(Elements.GetData(), Element, Num);
#else
			if (Num <= 512)
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					if (Elements[Idx] == Element) { return true; }
				}
				return false;
			}

			int32 Low = 0;
			int32 High = Num - 1;

			while (Low <= High)
			{
				const int32 Mid = Low + (High - Low) / 2;

				if (Elements[Mid] == Element)
				{
					return true;
				}
				else if (Elements[Mid] < Element)
				{
					Low = Mid + 1;
				}
				else
				{
					High = Mid - 1;
				}
			}

			return false;
#endif
		}

		static inline int32 FindSorted(const TLearningArrayView<1, const int32> Elements, const int32 Element)
		{
			const int32 Num = Elements.Num();

#if UE_LEARNING_ISPC
			return ispc::LearningFindSorted(Elements.GetData(), Element, Num);
#else
			if (Num <= 512)
			{
				for (int32 Idx = 0; Idx < Num; Idx++)
				{
					if (Elements[Idx] == Element) { return Idx; }
				}
				return INDEX_NONE;
			}

			int32 Low = 0;
			int32 High = Num - 1;

			while (Low <= High)
			{
				const int32 Mid = Low + (High - Low) / 2;

				if (Elements[Mid] == Element)
				{
					return Mid;
				}
				else if (Elements[Mid] < Element)
				{
					Low = Mid + 1;
				}
				else
				{
					High = Mid - 1;
				}
			}

			return INDEX_NONE;
#endif
		}

		static inline void FramesCheck(const TLearningArrayView<1, const int32> Frames)
		{
			const int32 FrameNum = Frames.Num();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum - 1; FrameIdx++)
			{
				check(Frames[FrameIdx + 0] >= 0);
				check(Frames[FrameIdx + 1] >= 0);
				check(Frames[FrameIdx + 0] < Frames[FrameIdx + 1]);
			}
		}

		static inline int32 FramesUnion(
			TLearningArrayView<1, int32> OutFrames,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsFrames)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Private::FramesUnion);

			FramesCheck(LhsFrames);
			FramesCheck(RhsFrames);

			// Number of frames in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsFrames.Num();

			// Event index for each list of frames
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both frames to process
			while (LhsIndex < LhsNum && RhsIndex < RhsNum)
			{
				// Time of the next lhs, and rhs frames
				const int32 LhsT = LhsFrames[LhsIndex];
				const int32 RhsT = RhsFrames[RhsIndex];

				// Frame from lhs is coming first
				if (LhsT < RhsT)
				{
					OutFrames[OutIndex] = LhsT;
					OutIndex++;
					LhsIndex++;
				}
				// Frame from rhs is coming first
				else if (RhsT < LhsT)
				{
					OutFrames[OutIndex] = RhsT;
					OutIndex++;
					RhsIndex++;
				}
				// Frame from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);
					OutFrames[OutIndex] = LhsT;
					OutIndex++;
					LhsIndex++;
					RhsIndex++;
				}
			}

			// Process any remaining lhs frames
			while (LhsIndex < LhsNum)
			{
				OutFrames[OutIndex] = LhsFrames[LhsIndex];
				OutIndex++;
				LhsIndex++;
			}

			// Process any remaining rhs frames
			while (RhsIndex < RhsNum)
			{
				OutFrames[OutIndex] = RhsFrames[RhsIndex];
				OutIndex++;
				RhsIndex++;
			}

			FramesCheck(OutFrames.Slice(0, OutIndex));

			// Return number of frames added
			return OutIndex;
		}

		static inline int32 FramesIntersection(
			TLearningArrayView<1, int32> OutFrames,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsFrames)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Private::FramesIntersection);

			FramesCheck(LhsFrames);
			FramesCheck(RhsFrames);

			// Number of frames in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsFrames.Num();

			// Event index for each list of frames
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both frames to process
			while (LhsIndex < LhsNum && RhsIndex < RhsNum)
			{
				// Time of the next lhs, and rhs frames
				const int32 LhsT = LhsFrames[LhsIndex];
				const int32 RhsT = RhsFrames[RhsIndex];

				// Frame from lhs is coming first
				if (LhsT < RhsT)
				{
					LhsIndex++;
				}
				// Frame from rhs is coming first
				else if (RhsT < LhsT)
				{
					RhsIndex++;
				}
				// Frame from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);
					OutFrames[OutIndex] = LhsT;
					OutIndex++;
					LhsIndex++;
					RhsIndex++;
				}
			}

			FramesCheck(OutFrames.Slice(0, OutIndex));

			// Return number of frames added
			return OutIndex;
		}

		static inline int32 FramesDifference(
			TLearningArrayView<1, int32> OutFrames,
			const TLearningArrayView<1, const int32> LhsFrames,
			const TLearningArrayView<1, const int32> RhsFrames)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Private::FramesDifference);

			FramesCheck(LhsFrames);
			FramesCheck(RhsFrames);

			// Number of frames in lhs and rhs
			const int32 LhsNum = LhsFrames.Num();
			const int32 RhsNum = RhsFrames.Num();

			// Event index for each list of frames
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;
			int32 OutIndex = 0;

			// While both frames to process
			while (LhsIndex < LhsNum && RhsIndex < RhsNum)
			{
				// Time of the next lhs, and rhs frames
				const int32 LhsT = LhsFrames[LhsIndex];
				const int32 RhsT = RhsFrames[RhsIndex];

				// Frame from lhs is coming first
				if (LhsT < RhsT)
				{
					OutFrames[OutIndex] = LhsT;
					OutIndex++;
					LhsIndex++;
				}
				// Frame from rhs is coming first
				else if (RhsT < LhsT)
				{
					RhsIndex++;
				}
				// Frame from lhs and rhs coming at same time
				else
				{
					check(LhsT == RhsT);
					LhsIndex++;
					RhsIndex++;
				}
			}

			// Process any remaining lhs frames
			while (LhsIndex < LhsNum)
			{
				OutFrames[OutIndex] = LhsFrames[LhsIndex];
				OutIndex++;
				LhsIndex++;
			}

			FramesCheck(OutFrames.Slice(0, OutIndex));

			// Return number of frames added
			return OutIndex;
		}

	}

	void FFrameSet::Check() const
	{
		check(EntrySequences.Num() == EntryFrameOffsets.Num());
		check(EntrySequences.Num() == EntryFrameNums.Num());

		const int32 EntryNum = EntrySequences.Num();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum - 1; EntryIdx++)
		{
			check(EntrySequences[EntryIdx + 0] < EntrySequences[EntryIdx + 1]);
		}

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(EntryFrameNums[EntryIdx] > 0);
			FrameSet::Private::FramesCheck(GetEntryFrames(EntryIdx));
		}
	}

	bool FFrameSet::IsEmpty() const { return EntrySequences.IsEmpty(); }
	
	void FFrameSet::Empty()
	{
		EntrySequences.Empty();
		EntryFrameOffsets.Empty();
		EntryFrameNums.Empty();
		Frames.Empty();
	}

	int32 FFrameSet::GetEntryNum() const { return EntrySequences.Num(); }
	TLearningArrayView<1, const int32> FFrameSet::GetEntrySequences() const { return EntrySequences; }
	TLearningArrayView<1, const int32> FFrameSet::GetEntryFrameNums() const { return EntryFrameNums; }
	int32 FFrameSet::GetEntrySequence(const int32 EntryIdx) const { return EntrySequences[EntryIdx]; }
	int32 FFrameSet::GetEntryFrameNum(const int32 EntryIdx) const { return EntryFrameNums[EntryIdx]; }
	TLearningArrayView<1, const int32> FFrameSet::GetEntryFrames(const int32 EntryIdx) const { return Frames.Slice(EntryFrameOffsets[EntryIdx], EntryFrameNums[EntryIdx]); }
	int32 FFrameSet::GetEntryFrame(const int32 EntryIdx, const int32 FrameIdx) const { return Frames[EntryFrameOffsets[EntryIdx] + FrameIdx]; }
	float FFrameSet::GetEntryFrameTime(const int32 EntryIdx, const int32 FrameIdx, const float FrameDeltaTime) const { return GetEntryFrame(EntryIdx, FrameIdx) * FrameDeltaTime; }
	int32 FFrameSet::GetEntryOffset(const int32 EntryIdx) const { return EntryFrameOffsets[EntryIdx]; }
	TLearningArrayView<1, const int32> FFrameSet::GetAllFrames() const { return Frames; }

	int32 FFrameSet::GetTotalFrameNum() const { return Frames.Num(); }
	
	bool FFrameSet::ContainsSequence(const int32 Sequence) const
	{
		return FrameSet::Private::ContainsSorted(EntrySequences, Sequence);
	}

	bool FFrameSet::Contains(const int32 Sequence, const int32 Frame) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		return EntryIdx != INDEX_NONE && GetEntryFrames(EntryIdx).Contains(Frame);
	}

	int32 FFrameSet::FindSequenceEntry(const int32 Sequence) const
	{
		return FrameSet::Private::FindSorted(EntrySequences, Sequence);
	}

	bool FFrameSet::Find(int32& OutEntryIdx, int32& OutFrameIdx, const int32 Sequence, const int32 Frame) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		if (EntryIdx != INDEX_NONE)
		{
			const int32 FoundFrameIdx = FrameSet::Private::FindSorted(GetEntryFrames(EntryIdx), Frame);
			if (FoundFrameIdx != INDEX_NONE)
			{
				OutEntryIdx = EntryIdx;
				OutFrameIdx = FoundFrameIdx;
				return true;
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutFrameIdx = INDEX_NONE;
		return false;
	}

	bool FFrameSet::FindNearest(int32& OutEntryIdx, int32& OutFrameIdx, int32& OutFrameDifference, const int32 Sequence, const int32 Frame) const
	{
		const int32 EntryIdx = FindSequenceEntry(Sequence);

		if (EntryIdx != INDEX_NONE)
		{
			OutFrameDifference = INT32_MAX;
			OutEntryIdx = INDEX_NONE;
			OutFrameIdx = INDEX_NONE;

			const int32 FrameNum = GetEntryFrameNum(EntryIdx);
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const int32 FrameDifference = GetEntryFrame(EntryIdx, FrameIdx) - Frame;

				if (OutFrameDifference == INT32_MAX || FMath::Abs(FrameDifference) < FMath::Abs(OutFrameDifference))
				{
					OutFrameDifference = FrameDifference;
					OutEntryIdx = EntryIdx;
					OutFrameIdx = FrameIdx;
				}
			}

			return OutFrameDifference != INT32_MAX;
		}

		OutFrameDifference = INT32_MAX;
		OutEntryIdx = INDEX_NONE;
		OutFrameIdx = INDEX_NONE;
		return false;
	}

	bool FFrameSet::FindNearestInRange(int32& OutEntryIdx, int32& OutFrameIdx, int32& OutFrameDifference, const int32 Sequence, const int32 Frame, const int32 RangeStart, const int32 RangeLength) const
	{
		check(Frame >= RangeStart && Frame < RangeStart + RangeLength);

		const int32 EntryIdx = FindSequenceEntry(Sequence);

		if (EntryIdx != INDEX_NONE)
		{
			OutFrameDifference = INT32_MAX;
			OutEntryIdx = INDEX_NONE;
			OutFrameIdx = INDEX_NONE;

			const int32 FrameNum = GetEntryFrameNum(EntryIdx);
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const int32 EntryFrame = GetEntryFrame(EntryIdx, FrameIdx);
				if (EntryFrame >= RangeStart && EntryFrame < RangeStart + RangeLength)
				{
					const int32 FrameDifference = EntryFrame - Frame;

					if (OutFrameDifference == INT32_MAX || FMath::Abs(FrameDifference) < FMath::Abs(OutFrameDifference))
					{
						OutFrameDifference = FrameDifference;
						OutEntryIdx = EntryIdx;
						OutFrameIdx = FrameIdx;
					}
				}
			}

			return OutFrameDifference != INT32_MAX;
		}

		OutFrameDifference = INT32_MAX;
		OutEntryIdx = INDEX_NONE;
		OutFrameIdx = INDEX_NONE;
		return false;
	}

	bool FFrameSet::FindOffset(int32& OutEntryIdx, int32& OutFrameIdx, const int32 Offset) const
	{
		const int32 EntryNum = GetEntryNum();
		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const int32 FrameNum = GetEntryFrameNum(EntryIdx);
			const int32 FrameOffset = GetEntryOffset(EntryIdx);

			if (Offset >= FrameOffset && Offset < FrameOffset + FrameNum)
			{
				OutEntryIdx = EntryIdx;
				OutFrameIdx = Offset - FrameOffset;
				return true;
			}
		}

		OutEntryIdx = INDEX_NONE;
		OutFrameIdx = INDEX_NONE;
		return false;
	}

	bool FFrameSet::FindFramesInRange(int32& OutStartFrameIdx, int32& OutFrameNum, const int32 EntryIdx, const int32 RangeStart, const int32 RangeLength) const
	{	
		OutStartFrameIdx = INDEX_NONE;
		OutFrameNum = 0;

		const int32 FrameNum = GetEntryFrameNum(EntryIdx);

		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			const int32 Frame = GetEntryFrame(EntryIdx, FrameIdx);

			if (Frame >= RangeStart && Frame < RangeStart + RangeLength)
			{
				if (OutStartFrameIdx == INDEX_NONE)
				{
					OutStartFrameIdx = FrameIdx;
				}
				OutFrameNum++;
			}
		}

		return OutStartFrameIdx != INDEX_NONE;
	}

	void FFrameSet::AddEntry(
		const int32 InSequence,
		const TLearningArrayView<1, const int32> InFrames)
	{
		check(!EntrySequences.ArrayView().Contains(InSequence));
		FrameSet::Private::FramesCheck(InFrames);

		if (InFrames.IsEmpty()) { return; }

		const int32 CurrFrameOffset = Frames.Num();
		const int32 AddFrameNum = InFrames.Num();
		Frames.SetNumUninitialized({ CurrFrameOffset + AddFrameNum });
		Array::Copy(Frames.Slice(CurrFrameOffset, AddFrameNum), InFrames);

		const int32 CurrEntryNum = EntrySequences.Num();
		EntrySequences.SetNumUninitialized({ CurrEntryNum + 1 });
		EntryFrameOffsets.SetNumUninitialized({ CurrEntryNum + 1 });
		EntryFrameNums.SetNumUninitialized({ CurrEntryNum + 1 });
		EntrySequences[CurrEntryNum] = InSequence;
		EntryFrameOffsets[CurrEntryNum] = CurrFrameOffset;
		EntryFrameNums[CurrEntryNum] = AddFrameNum;

		Check();
	}

	namespace FrameSet
	{
		bool Equal(const FFrameSet& Lhs, const FFrameSet& Rhs)
		{
			return
				Lhs.EntrySequences.Num() == Rhs.EntrySequences.Num() &&
				Lhs.Frames.Num() == Rhs.Frames.Num() &&
				Array::Equal(Lhs.EntrySequences, Rhs.EntrySequences) &&
				Array::Equal(Lhs.EntryFrameOffsets, Rhs.EntryFrameOffsets) &&
				Array::Equal(Lhs.EntryFrameNums, Rhs.EntryFrameNums) &&
				Array::Equal(Lhs.Frames, Rhs.Frames);
		}

		void Union(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Union);

			if (Equal(Lhs, Rhs)) { Out = Lhs; return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsFrameNum = Lhs.GetTotalFrameNum();
			const int32 RhsFrameNum = Rhs.GetTotalFrameNum();

			// Allocate potential maximum number of entries and frames we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.EntryFrameOffsets.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.EntryFrameNums.SetNumUninitialized({ LhsEntryNum + RhsEntryNum });
			Out.Frames.SetNumUninitialized({ LhsFrameNum + RhsFrameNum });

			// Entry index for each list of frames
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output entry index
			int32 EventIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					// Append frames to output
					if (Lhs.GetEntryFrameNum(LhsIndex) > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = Lhs.GetEntryFrameNum(LhsIndex);
						Array::Copy(Out.Frames.Slice(EventIndex, Lhs.GetEntryFrameNum(LhsIndex)), Lhs.GetEntryFrames(LhsIndex));
						EventIndex += Lhs.GetEntryFrameNum(LhsIndex);
						OutIndex++;
					}

					LhsIndex++;
				}
				// If entry from rhs is first
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					// Append frames to output
					if (Rhs.GetEntryFrameNum(RhsIndex) > 0)
					{
						Out.EntrySequences[OutIndex] = Rhs.GetEntrySequence(RhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = Rhs.GetEntryFrameNum(RhsIndex);
						Array::Copy(Out.Frames.Slice(EventIndex, Rhs.GetEntryFrameNum(RhsIndex)), Rhs.GetEntryFrames(RhsIndex));
						EventIndex += Rhs.GetEntryFrameNum(RhsIndex);
						OutIndex++;
					}

					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(Rhs.GetEntrySequence(RhsIndex) == Lhs.GetEntrySequence(LhsIndex));

					// Append union to output
					const int32 FrameNum = Private::FramesUnion(
						Out.Frames.Slice(EventIndex, LhsFrameNum + RhsFrameNum - EventIndex),
						Lhs.GetEntryFrames(LhsIndex),
						Rhs.GetEntryFrames(RhsIndex));

					check(FrameNum <= Lhs.GetEntryFrameNum(LhsIndex) + Rhs.GetEntryFrameNum(RhsIndex));

					if (FrameNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = FrameNum;
						EventIndex += FrameNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				// Append frames to output
				if (Lhs.GetEntryFrameNum(LhsIndex) > 0)
				{
					Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
					Out.EntryFrameOffsets[OutIndex] = EventIndex;
					Out.EntryFrameNums[OutIndex] = Lhs.GetEntryFrameNum(LhsIndex);
					Array::Copy(Out.Frames.Slice(EventIndex, Lhs.GetEntryFrameNum(LhsIndex)), Lhs.GetEntryFrames(LhsIndex));
					EventIndex += Lhs.GetEntryFrameNum(LhsIndex);
					OutIndex++;
				}

				LhsIndex++;
			}

			// Process any remaining rhs entries
			while (RhsIndex < RhsEntryNum)
			{
				// Append frames to output
				if (Rhs.GetEntryFrameNum(RhsIndex) > 0)
				{
					Out.EntrySequences[OutIndex] = Rhs.GetEntrySequence(RhsIndex);
					Out.EntryFrameOffsets[OutIndex] = EventIndex;
					Out.EntryFrameNums[OutIndex] = Rhs.GetEntryFrameNum(RhsIndex);
					Array::Copy(Out.Frames.Slice(EventIndex, Rhs.GetEntryFrameNum(RhsIndex)), Rhs.GetEntryFrames(RhsIndex));
					EventIndex += Rhs.GetEntryFrameNum(RhsIndex);
					OutIndex++;
				}

				RhsIndex++;
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryFrameOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryFrameNums.SetNumUninitialized({ OutIndex });
			Out.Frames.SetNumUninitialized({ EventIndex });
			Out.Check();
		}

		void Intersection(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Intersection);

			if (Equal(Lhs, Rhs)) { Out = Lhs; return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsFrameNum = Lhs.GetTotalFrameNum();
			const int32 RhsFrameNum = Rhs.GetTotalFrameNum();

			// Allocate potential maximum number of entries and frames we might to output
			Out.EntrySequences.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryFrameOffsets.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.EntryFrameNums.SetNumUninitialized({ FMath::Max(LhsEntryNum, RhsEntryNum) });
			Out.Frames.SetNumUninitialized({ LhsFrameNum + RhsFrameNum });

			// Entry index for each list of frames
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output entry index
			int32 EventIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					LhsIndex++;
				}
				// If entry from rhs is first
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(Rhs.GetEntrySequence(RhsIndex) == Lhs.GetEntrySequence(LhsIndex));

					// Append intersection to output
					const int32 FrameNum = Private::FramesIntersection(
						Out.Frames.Slice(EventIndex, LhsFrameNum + RhsFrameNum - EventIndex),
						Lhs.GetEntryFrames(LhsIndex),
						Rhs.GetEntryFrames(RhsIndex));

					check(FrameNum <= Lhs.GetEntryFrameNum(LhsIndex) + Rhs.GetEntryFrameNum(RhsIndex));

					if (FrameNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = FrameNum;
						EventIndex += FrameNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryFrameOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryFrameNums.SetNumUninitialized({ OutIndex });
			Out.Frames.SetNumUninitialized({ EventIndex });
			Out.Check();
		}

		void Difference(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FrameSet::Difference);

			if (Equal(Lhs, Rhs)) { Out = FFrameSet(); return; }

			const int32 LhsEntryNum = Lhs.GetEntryNum();
			const int32 RhsEntryNum = Rhs.GetEntryNum();
			const int32 LhsFrameNum = Lhs.GetTotalFrameNum();
			const int32 RhsFrameNum = Rhs.GetTotalFrameNum();

			// Allocate potential maximum number of entries and frames we might to output
			Out.EntrySequences.SetNumUninitialized({ LhsEntryNum });
			Out.EntryFrameOffsets.SetNumUninitialized({ LhsEntryNum });
			Out.EntryFrameNums.SetNumUninitialized({ LhsEntryNum });
			Out.Frames.SetNumUninitialized({ LhsFrameNum + RhsFrameNum });

			// Entry index for each list of frames
			int32 OutIndex = 0;
			int32 LhsIndex = 0;
			int32 RhsIndex = 0;

			// Output entry index
			int32 EventIndex = 0;

			// While both sets have entries
			while (LhsIndex < LhsEntryNum && RhsIndex < RhsEntryNum)
			{
				// If entry from lhs is first
				if (Lhs.GetEntrySequence(LhsIndex) < Rhs.GetEntrySequence(RhsIndex))
				{
					// Append frames to output
					if (Lhs.GetEntryFrameNum(LhsIndex) > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = Lhs.GetEntryFrameNum(LhsIndex);
						Array::Copy(Out.Frames.Slice(EventIndex, Lhs.GetEntryFrameNum(LhsIndex)), Lhs.GetEntryFrames(LhsIndex));
						EventIndex += Lhs.GetEntryFrameNum(LhsIndex);
						OutIndex++;
					}

					LhsIndex++;
				}
				// If entry from rhs is first
				else if (Rhs.GetEntrySequence(RhsIndex) < Lhs.GetEntrySequence(LhsIndex))
				{
					RhsIndex++;
				}
				// If both contain the same entry
				else
				{
					check(Rhs.GetEntrySequence(RhsIndex) == Lhs.GetEntrySequence(LhsIndex));

					// Append difference to output
					const int32 FrameNum = Private::FramesDifference(
						Out.Frames.Slice(EventIndex, LhsFrameNum + RhsFrameNum - EventIndex),
						Lhs.GetEntryFrames(LhsIndex),
						Rhs.GetEntryFrames(RhsIndex));

					check(FrameNum <= Lhs.GetEntryFrameNum(LhsIndex) + Rhs.GetEntryFrameNum(RhsIndex));

					if (FrameNum > 0)
					{
						Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
						Out.EntryFrameOffsets[OutIndex] = EventIndex;
						Out.EntryFrameNums[OutIndex] = FrameNum;
						EventIndex += FrameNum;
						OutIndex++;
					}

					LhsIndex++; RhsIndex++;
				}
			}

			// Process any remaining lhs entries
			while (LhsIndex < LhsEntryNum)
			{
				// Append frames to output
				if (Lhs.GetEntryFrameNum(LhsIndex) > 0)
				{
					Out.EntrySequences[OutIndex] = Lhs.GetEntrySequence(LhsIndex);
					Out.EntryFrameOffsets[OutIndex] = EventIndex;
					Out.EntryFrameNums[OutIndex] = Lhs.GetEntryFrameNum(LhsIndex);
					Array::Copy(Out.Frames.Slice(EventIndex, Lhs.GetEntryFrameNum(LhsIndex)), Lhs.GetEntryFrames(LhsIndex));
					EventIndex += Lhs.GetEntryFrameNum(LhsIndex);
					OutIndex++;
				}

				LhsIndex++;
			}

			// Resize output to match what was added
			Out.EntrySequences.SetNumUninitialized({ OutIndex });
			Out.EntryFrameOffsets.SetNumUninitialized({ OutIndex });
			Out.EntryFrameNums.SetNumUninitialized({ OutIndex });
			Out.Frames.SetNumUninitialized({ EventIndex });
			Out.Check();
		}

		void AllFrameEntries(
			TLearningArrayView<1, int32> OutFrameEntries,
			const FFrameSet& FrameSet)
		{
			check(OutFrameEntries.Num() == FrameSet.GetTotalFrameNum());

			const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();
			const int32 EntryNum = FrameSet.GetEntryNum();

			int32 FrameOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 FrameNum = FrameSet.GetEntryFrameNum(EntryIdx);
				Array::Set(OutFrameEntries.Slice(FrameOffset, FrameNum), EntryIdx);
				FrameOffset += FrameNum;
			}

			check(FrameOffset == TotalFrameNum);
		}

		void AllFrameIndices(
			TLearningArrayView<1, int32> OutFrameIndices,
			const FFrameSet& FrameSet)
		{
			check(OutFrameIndices.Num() == FrameSet.GetTotalFrameNum());

			const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();
			const int32 EntryNum = FrameSet.GetEntryNum();

			int32 FrameOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 FrameNum = FrameSet.GetEntryFrameNum(EntryIdx);
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					OutFrameIndices[FrameOffset + FrameIdx] = FrameIdx;
				}
				FrameOffset += FrameNum;
			}

			check(FrameOffset == TotalFrameNum);
		}

		void AllFrameSequences(
			TLearningArrayView<1, int32> OutFrameSequences,
			const FFrameSet& FrameSet)
		{
			check(OutFrameSequences.Num() == FrameSet.GetTotalFrameNum());

			const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();
			const int32 EntryNum = FrameSet.GetEntryNum();

			int32 FrameOffset = 0;
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 FrameNum = FrameSet.GetEntryFrameNum(EntryIdx);
				Array::Set(OutFrameSequences.Slice(FrameOffset, FrameNum), FrameSet.GetEntrySequence(EntryIdx));
				FrameOffset += FrameNum;
			}

			check(FrameOffset == TotalFrameNum);
		}

		void AllFrameTimes(
			TLearningArrayView<1, float> OutFrameTimes,
			const FFrameSet& FrameSet,
			const float FrameDeltaTime)
		{
			check(OutFrameTimes.Num() == FrameSet.GetTotalFrameNum());

			const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();
			for (int32 FrameIdx = 0; FrameIdx < TotalFrameNum; FrameIdx++)
			{
				OutFrameTimes[FrameIdx] = FrameSet.Frames[FrameIdx] * FrameDeltaTime;
			}
		}

		void ForEachFrame(
			const FFrameSet& FrameSet,
			const TFunctionRef<void(
				const int32 TotalFrameIdx,
				const int32 EntryIdx,
				const int32 FrameIdx)> Body)
		{
			const int32 EntryNum = FrameSet.GetEntryNum();
			for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
			{
				const int32 FrameNum = FrameSet.GetEntryFrameNum(EntryIdx);
				const int32 FrameOffset = FrameSet.GetEntryOffset(EntryIdx);
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					Body(
						FrameOffset + FrameIdx,
						EntryIdx,
						FrameIdx);
				}
			}
		}

		void ParallelForEachFrame(
			const FFrameSet& FrameSet,
			const TFunctionRef<void(
				const int32 TotalFrameIdx,
				const int32 EntryIdx,
				const int32 FrameIdx)> Body)
		{
			const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();

			TLearningArray<1, int32> FrameEntries;
			TLearningArray<1, int32> FrameIndices;

			FrameEntries.SetNumUninitialized({ TotalFrameNum });
			FrameIndices.SetNumUninitialized({ TotalFrameNum });

			AllFrameEntries(FrameEntries, FrameSet);
			AllFrameIndices(FrameIndices, FrameSet);

			ParallelFor(TotalFrameNum, [&Body, &FrameEntries, &FrameIndices](const int32 TotalFrameIdx) {
				Body(
					TotalFrameIdx,
					FrameEntries[TotalFrameIdx],
					FrameIndices[TotalFrameIdx]);
				});
		}

	}
}

