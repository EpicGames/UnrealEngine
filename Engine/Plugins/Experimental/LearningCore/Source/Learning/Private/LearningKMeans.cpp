// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningKMeans.h"

#include "LearningRandom.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning::KMeans
{
	void InitCenters(
		TLearningArrayView<2, float> OutCenters,
		const TLearningArrayView<2, const float> Samples,
		const uint32 Seed)
	{
		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = OutCenters.Num<0>();
		const int32 DimNum = OutCenters.Num<1>();

#if UE_LEARNING_ISPC
		ispc::LearningKMeansInitCenters(
			OutCenters.GetData(),
			Samples.GetData(),
			SampleNum,
			ClusterNum,
			DimNum,
			Seed);
#else
		for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			const int32 SampleIdx = Random::IntInRange(Random::Int(Seed ^ ClusterIdx), 0, SampleNum - 1);
			Array::Copy(OutCenters[ClusterIdx], Samples[SampleIdx]);
		}
#endif
	}

	void InitBounds(
		TLearningArrayView<2, float> OutMins,
		TLearningArrayView<2, float> OutMaxs,
		const TLearningArrayView<2, const float> Samples,
		const uint32 Seed)
	{
		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = OutMins.Num<0>();
		const int32 DimNum = OutMins.Num<1>();

#if UE_LEARNING_ISPC
		ispc::LearningKMeansInitBounds(
			OutMins.GetData(),
			OutMaxs.GetData(),
			Samples.GetData(),
			SampleNum,
			ClusterNum,
			DimNum,
			Seed);
#else
		for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			const int32 SampleIdx = Random::IntInRange(Random::Int(Seed ^ ClusterIdx), 0, SampleNum - 1);
			Array::Copy(OutMins[ClusterIdx], Samples[SampleIdx]);
			Array::Copy(OutMaxs[ClusterIdx], Samples[SampleIdx]);
		}
#endif
	}

	void UpdateAssignmentsFromCenters(
		TLearningArrayView<1, int32> OutAssignments,
		const TLearningArrayView<2, const float> Centers,
		const TLearningArrayView<2, const float> Samples,
		const bool bParallelEvaluation,
		const uint16 MinParallelBatchSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::UpdateAssignmentsFromCenters);

		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = Centers.Num<0>();
		const int32 DimNum = Samples.Num<1>();

		auto UpdateAssignmentsFunction = [&](const int32 SliceStart, const int32 SliceNum)
		{
#if UE_LEARNING_ISPC
			ispc::LearningKMeansUpdateAssignmentsFromCenters(
				OutAssignments.Slice(SliceStart, SliceNum).GetData(),
				Centers.GetData(),
				Samples.Slice(SliceStart, SliceNum).GetData(),
				SliceNum,
				ClusterNum,
				DimNum);
#else
			for (int32 SampleIdx = SliceStart; SampleIdx < SliceStart + SliceNum; SampleIdx++)
			{
				float BestDistance = MAX_flt;
				int32 BestIndex = INDEX_NONE;

				for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
				{
					float CurrDistance = 0.0f;

					for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
					{
						CurrDistance += FMath::Square(Centers[ClusterIdx][DimIdx] - Samples[SampleIdx][DimIdx]);

						if (CurrDistance >= BestDistance)
						{
							break;
						}
					}

					if (CurrDistance < BestDistance)
					{
						BestDistance = CurrDistance;
						BestIndex = ClusterIdx;
					}
				}

				check(BestDistance != MAX_flt);
				check(BestIndex != INDEX_NONE);

				OutAssignments[SampleIdx] = BestIndex;
			}
#endif
		};

		if (bParallelEvaluation && SampleNum > MinParallelBatchSize)
		{
			SlicedParallelFor(SampleNum, MinParallelBatchSize, UpdateAssignmentsFunction);
		}
		else
		{
			UpdateAssignmentsFunction(0, SampleNum);
		}
	}

	void UpdateAssignmentsFromBounds(
		TLearningArrayView<1, int32> OutAssignments,
		const TLearningArrayView<2, const float> Mins,
		const TLearningArrayView<2, const float> Maxs,
		const TLearningArrayView<2, const float> Samples,
		const bool bParallelEvaluation,
		const uint16 MinParallelBatchSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::UpdateAssignmentsFromBounds);

		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = Mins.Num<0>();
		const int32 DimNum = Samples.Num<1>();

		auto UpdateAssignmentsFunction = [&](const int32 SliceStart, const int32 SliceNum)
		{
#if UE_LEARNING_ISPC
			ispc::LearningKMeansUpdateAssignmentsFromBounds(
				OutAssignments.Slice(SliceStart, SliceNum).GetData(),
				Mins.GetData(),
				Maxs.GetData(),
				Samples.Slice(SliceStart, SliceNum).GetData(),
				SliceNum,
				ClusterNum,
				DimNum);
#else
			for (int32 SampleIdx = SliceStart; SampleIdx < SliceStart + SliceNum; SampleIdx++)
			{
				float BestDistance = MAX_flt;
				int32 BestIndex = INDEX_NONE;

				for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
				{
					float CurrDistance = 0.0f;

					for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
					{
						float Center = (Maxs[ClusterIdx][DimIdx] - Mins[ClusterIdx][DimIdx]) / 2.0f + Mins[ClusterIdx][DimIdx];
						CurrDistance += FMath::Abs(Center - Samples[SampleIdx][DimIdx]);

						if (CurrDistance >= BestDistance)
						{
							break;
						}
					}

					if (CurrDistance < BestDistance)
					{
						BestDistance = CurrDistance;
						BestIndex = ClusterIdx;
					}
				}

				check(BestDistance != MAX_flt);
				check(BestIndex != INDEX_NONE);

				OutAssignments[SampleIdx] = BestIndex;
			}
#endif
		};

		if (bParallelEvaluation && SampleNum > MinParallelBatchSize)
		{
			SlicedParallelFor(SampleNum, MinParallelBatchSize, UpdateAssignmentsFunction);
		}
		else
		{
			UpdateAssignmentsFunction(0, SampleNum);
		}
	}

	void CountClusterAssignments(
		TLearningArrayView<1, int32> OutAssignmentCounts,
		const TLearningArrayView<1, const int32> Assignments)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::CountClusterAssignments);

		const int32 SampleNum = Assignments.Num();

		Array::Zero(OutAssignmentCounts);

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			OutAssignmentCounts[Assignments[SampleIdx]]++;
		}
	}

	void UpdateCenters(
		TLearningArrayView<2, float> OutCenters,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<1, const int32> AssignmentCounts,
		const TLearningArrayView<2, const float> Samples)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::UpdateCenters);

		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = OutCenters.Num<0>();
		const int32 DimNum = Samples.Num<1>();

#if UE_LEARNING_ISPC
		ispc::LearningKMeansUpdateCenters(
			OutCenters.GetData(),
			Assignments.GetData(),
			AssignmentCounts.GetData(),
			Samples.GetData(),
			SampleNum,
			ClusterNum,
			DimNum);
#else
		Array::Zero(OutCenters);

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			const int32 CenterIdx = Assignments[SampleIdx];

			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				OutCenters[CenterIdx][DimIdx] += Samples[SampleIdx][DimIdx] / AssignmentCounts[CenterIdx];
			}
		}
#endif

		Array::Check(OutCenters);
	}

	void UpdateBounds(
		TLearningArrayView<2, float> OutMins,
		TLearningArrayView<2, float> OutMaxs,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<2, const float> Samples)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::UpdateBounds);

		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = OutMins.Num<0>();
		const int32 DimNum = Samples.Num<1>();

#if UE_LEARNING_ISPC
		ispc::LearningKMeansUpdateBounds(
			OutMins.GetData(),
			OutMaxs.GetData(),
			Assignments.GetData(),
			Samples.GetData(),
			SampleNum,
			ClusterNum,
			DimNum);
#else
		Array::Set(OutMins, +MAX_flt);
		Array::Set(OutMaxs, -MAX_flt);

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			const int32 CenterIdx = Assignments[SampleIdx];

			for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				OutMins[CenterIdx][DimIdx] = FMath::Min(OutMins[CenterIdx][DimIdx], Samples[SampleIdx][DimIdx]);
				OutMaxs[CenterIdx][DimIdx] = FMath::Max(OutMaxs[CenterIdx][DimIdx], Samples[SampleIdx][DimIdx]);
			}
		}
#endif
	}

	void ComputeClusteredIndex(
		TLearningArrayView<2, float> OutClusteredSamples,
		TLearningArrayView<1, int32> OutClusterStarts,
		TLearningArrayView<1, int32> OutClusterLengths,
		TLearningArrayView<1, int32> OutSampleMapping,
		TLearningArrayView<1, int32> OutInverseSampleMapping,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<1, const int32> AssignmentCounts,
		const TLearningArrayView<2, const float> Samples)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::KMeans::ComputeClusteredIndex);

		const int32 SampleNum = Samples.Num<0>();
		const int32 ClusterNum = OutClusterStarts.Num();
		const int32 DimNum = Samples.Num<1>();

#if UE_LEARNING_ISPC
		ispc::LearningKMeansComputeClusteredIndex(
			OutClusteredSamples.GetData(),
			OutClusterStarts.GetData(),
			OutClusterLengths.GetData(),
			OutSampleMapping.GetData(),
			OutInverseSampleMapping.GetData(),
			Assignments.GetData(),
			AssignmentCounts.GetData(),
			Samples.GetData(),
			SampleNum,
			ClusterNum,
			DimNum);
#else
		int32 ClusterOffset = 0;
		for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			OutClusterStarts[ClusterIdx] = ClusterOffset;
			ClusterOffset += AssignmentCounts[ClusterIdx];
		}

		// Keep running total of assignments here
		Array::Zero(OutClusterLengths);

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			const int32 ClusterIdx = Assignments[SampleIdx];
			const int32 SortedIdx = OutClusterStarts[ClusterIdx] + OutClusterLengths[ClusterIdx];

			Array::Copy(OutClusteredSamples[SortedIdx], Samples[SampleIdx]);
			OutClusterLengths[ClusterIdx]++;

			OutSampleMapping[SampleIdx] = SortedIdx;
			OutInverseSampleMapping[SortedIdx] = SampleIdx;
		}

		// Validate Number of assignments matched
		for (int32 ClusterIdx = 0; ClusterIdx < ClusterNum; ClusterIdx++)
		{
			check(OutClusterLengths[ClusterIdx] == AssignmentCounts[ClusterIdx]);
		}
#endif
	}

}