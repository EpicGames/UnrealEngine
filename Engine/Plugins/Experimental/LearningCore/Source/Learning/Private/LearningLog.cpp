// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningLog.h"

DEFINE_LOG_CATEGORY(LogLearning);
