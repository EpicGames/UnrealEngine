// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningNeuralNetwork.h"

#include "LearningProgress.h"

#include "Async/ParallelFor.h"
#include "Modules/ModuleManager.h"

#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeCPU.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningNeuralNetwork)

void ULearningNeuralNetworkData::PostLoad()
{
	Super::PostLoad();

	UpdateNetwork();
}

void ULearningNeuralNetworkData::Init(const int32 InInputSize, const int32 InOutputSize, const int32 InCompatibilityHash, const TArrayView<const uint8> InFileData)
{
	InputSize = InInputSize;
	OutputSize = InOutputSize;
	CompatibilityHash = InCompatibilityHash;
	FileData = InFileData;
	UpdateNumber = 0;
	UpdateNetwork();
}

void ULearningNeuralNetworkData::InitFrom(const ULearningNeuralNetworkData* OtherNetworkData)
{
	InputSize = OtherNetworkData->InputSize;
	OutputSize = OtherNetworkData->OutputSize;
	CompatibilityHash = OtherNetworkData->CompatibilityHash;
	FileData = OtherNetworkData->FileData;
	UpdateNumber = 0;
	UpdateNetwork();
}

namespace UE::Learning::NeuralNetwork::Private
{
	static constexpr int32 MagicNumber = 0x1e9b0c80;
	static constexpr int32 VersionNumber = 1;
}

bool ULearningNeuralNetworkData::LoadFromSnapshot(const TArrayView<const uint8> InBytes)
{
	if (!ensureMsgf(InBytes.Num() > sizeof(int32) * 6, TEXT("Snapshot format invalid. File not large enough.")))
	{
		return false;
	}

	int64 Offset = 0;
	
	int32 MagicNumber = 0;
	UE::Learning::DeserializeFromBytes(Offset, InBytes, MagicNumber);
	if (!ensureMsgf(MagicNumber == UE::Learning::NeuralNetwork::Private::MagicNumber, TEXT("Snapshot format invalid. Incorrect Magic Number. Got %i, expected %i."), MagicNumber, UE::Learning::NeuralNetwork::Private::MagicNumber))
	{
		return false;
	}

	int32 VersionNumber = 0;
	UE::Learning::DeserializeFromBytes(Offset, InBytes, VersionNumber);
	if (!ensureMsgf(VersionNumber == UE::Learning::NeuralNetwork::Private::VersionNumber, TEXT("Snapshot format invalid. Unsupported version number. Got %i, expected %i."), VersionNumber, UE::Learning::NeuralNetwork::Private::VersionNumber))
	{
		return false;
	}

	UE::Learning::DeserializeFromBytes(Offset, InBytes, InputSize);
	UE::Learning::DeserializeFromBytes(Offset, InBytes, OutputSize);
	UE::Learning::DeserializeFromBytes(Offset, InBytes, CompatibilityHash);

	int32 FileDataSize = 0;
	UE::Learning::DeserializeFromBytes(Offset, InBytes, FileDataSize);
	FileData = InBytes.Slice(Offset, FileDataSize);
	UpdateNumber++;

	UpdateNetwork();
	return true;
}

void ULearningNeuralNetworkData::SaveToSnapshot(TArrayView<uint8> OutBytes) const
{
	int64 Offset = 0;
	UE::Learning::SerializeToBytes(Offset, OutBytes, UE::Learning::NeuralNetwork::Private::MagicNumber);
	UE::Learning::SerializeToBytes(Offset, OutBytes, UE::Learning::NeuralNetwork::Private::VersionNumber);
	UE::Learning::SerializeToBytes(Offset, OutBytes, InputSize);
	UE::Learning::SerializeToBytes(Offset, OutBytes, OutputSize);
	UE::Learning::SerializeToBytes(Offset, OutBytes, CompatibilityHash);
	UE::Learning::SerializeToBytes(Offset, OutBytes, FileData.Num());
	FMemory::Memcpy(OutBytes.Slice(Offset, FileData.Num()).GetData(), FileData.GetData(), FileData.Num());
}

int32 ULearningNeuralNetworkData::GetSnapshotByteNum() const
{
	return
		sizeof(int32) + // Magic Number
		sizeof(int32) + // Version Number
		sizeof(int32) + // Input Size
		sizeof(int32) + // Output Size
		sizeof(int32) + // Compatibility Hash
		sizeof(int32) + // FileData size
		FileData.Num(); // FileData
}

TSharedPtr<UE::Learning::FNeuralNetwork>& ULearningNeuralNetworkData::GetNetwork()
{
	return Network;
}

bool ULearningNeuralNetworkData::IsEmpty() const
{
	return !Network.IsValid() || Network->IsEmpty();
}

int32 ULearningNeuralNetworkData::GetInputSize() const
{
	return InputSize;
}

int32 ULearningNeuralNetworkData::GetOutputSize() const
{
	return OutputSize;
}

int32 ULearningNeuralNetworkData::GetCompatibilityHash() const
{
	return CompatibilityHash;
}

int32 ULearningNeuralNetworkData::GetContentHash() const
{
	return ContentHash;
}

int32 ULearningNeuralNetworkData::GetUpdateNumber() const
{
	return UpdateNumber;
}

void ULearningNeuralNetworkData::UpdateNetwork()
{
	if (FileData.Num() > 0)
	{
		if (!ModelData)
		{
			ModelData = NewObject<UNNEModelData>(this);
		}

		ModelData->Init(TEXT("ubnne"), FileData);

		if (!Network)
		{
			Network = MakeShared<UE::Learning::FNeuralNetwork>();
		}

		ensureMsgf(FModuleManager::Get().LoadModule(TEXT("NNERuntimeBasicCpu")), TEXT("Unable to load module for NNE runtime NNERuntimeBasicCpu."));

		TWeakInterfacePtr<INNERuntimeCPU> RuntimeCPU = UE::NNE::GetRuntime<INNERuntimeCPU>(TEXT("NNERuntimeBasicCpu"));

		TSharedPtr<UE::NNE::IModelCPU> UpdatedModel = nullptr;

		if (ensureMsgf(RuntimeCPU.IsValid(), TEXT("Could not find requested NNE Runtime")))
		{
			UpdatedModel = RuntimeCPU->CreateModelCPU(ModelData);
		}

		// Compute the content hash

		ContentHash = CityHash32((const char*)FileData.GetData(), FileData.Num());

		// If we are not in the editor we can now clear the FileData and FileType since these will be
		// using additional memory and we are not going to save this asset and so don't require them.

#if !WITH_EDITOR
		ModelData->ClearFileDataAndFileType();
#endif

		Network->UpdateModel(UpdatedModel, InputSize, OutputSize);
	}
}

namespace UE::Learning
{
	TSharedRef<FNeuralNetworkInference> FNeuralNetwork::CreateInferenceObject(
		const int32 MaxBatchSize,
		const FNeuralNetworkInferenceSettings& Settings)
	{

		TSharedRef<FNeuralNetworkInference> InferenceObject = MakeShared<FNeuralNetworkInference>(
			*Model,
			MaxBatchSize,
			InputSize,
			OutputSize,
			Settings);

		{
			UE::Learning::FScopeNullableWriteLock ScopeLock(&Lock);
			InferenceObjects.Emplace(InferenceObject.ToWeakPtr());
		}

		return InferenceObject;
	}

	bool FNeuralNetwork::IsEmpty() const
	{
		return Model == nullptr;
	}
	
	int32 FNeuralNetwork::GetInputSize() const
	{
		return InputSize;
	}

	int32 FNeuralNetwork::GetOutputSize() const
	{
		return OutputSize;
	}

	void FNeuralNetwork::UpdateModel(const TSharedPtr<NNE::IModelCPU>& InModel, const int32 InInputSize, const int32 InOutputSize)
	{
		UE::Learning::FScopeNullableWriteLock ScopeLock(&Lock);

		// Update Model

		InputSize = InInputSize;
		OutputSize = InOutputSize;
		Model = InModel;

		if (Model)
		{
			// We need to tell all the created inference objects to update their instances since the Model has changed
			// and so the instance objects themselves are either no longer valid or tied to the old model.

			for (const TWeakPtr<UE::Learning::FNeuralNetworkInference>& InferenceObject : InferenceObjects)
			{
				if (TSharedPtr<UE::Learning::FNeuralNetworkInference> InferenceObjectPtr = InferenceObject.Pin())
				{
					InferenceObjectPtr->ReloadModelInstances(*Model, InputSize, OutputSize);
				}
			}

			// Remove all invalid weak pointers

			InferenceObjects.RemoveAll(
				[](const TWeakPtr<UE::Learning::FNeuralNetworkInference>& InferenceObject) { return !InferenceObject.IsValid(); });
		}
		else
		{
			InferenceObjects.Empty();
		}
	}

	FNeuralNetworkInference::FNeuralNetworkInference(
		UE::NNE::IModelCPU& InModel,
		const int32 InMaxBatchSize,
		const int32 InInputSize,
		const int32 InOutputSize,
		const FNeuralNetworkInferenceSettings& InSettings)
		: MaxBatchSize(InMaxBatchSize)
		, Settings(InSettings)
	{
		ReloadModelInstances(InModel, InInputSize, InOutputSize);
	}

	void FNeuralNetworkInference::ReloadModelInstances(NNE::IModelCPU& InModel, const int32 InInputSize, const int32 InOutputSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FNeuralNetworkInference::ReloadModelInstances);

		InputSize = InInputSize;
		OutputSize = InOutputSize;

		if (Settings.bParallelEvaluation)
		{
			const int32 IdealInferenceInstanceNum = ParallelForImpl::GetNumberOfThreadTasks(MaxBatchSize, Settings.MinParallelBatchSize, EParallelForFlags::None);
			const int32 InferenceInstanceBatchSize = FMath::DivideAndRoundUp((int32)MaxBatchSize, IdealInferenceInstanceNum);
			const int32 InferenceInstanceNum = FMath::DivideAndRoundUp((int32)MaxBatchSize, InferenceInstanceBatchSize);

			ModelInstances.Empty(InferenceInstanceNum);

			for (int32 InferenceInstanceIdx = 0; InferenceInstanceIdx < InferenceInstanceNum; InferenceInstanceIdx++)
			{
				ModelInstances.Emplace(InModel.CreateModelInstanceCPU());
				ModelInstances.Last()->SetInputTensorShapes({ NNE::FTensorShape::Make({(uint32)InferenceInstanceBatchSize, (uint32)InputSize}) });
			}
		}
		else
		{
			ModelInstances.Empty(1);
			ModelInstances.Emplace(InModel.CreateModelInstanceCPU());
			ModelInstances.Last()->SetInputTensorShapes({ NNE::FTensorShape::Make({(uint32)MaxBatchSize, (uint32)InputSize}) });
		}
	}

	int32 FNeuralNetworkInference::GetMaxBatchSize() const
	{
		return MaxBatchSize;
	}

	int32 FNeuralNetworkInference::GetInputSize() const
	{
		return InputSize;
	}

	int32 FNeuralNetworkInference::GetOutputSize() const
	{
		return OutputSize;
	}

	void FNeuralNetworkInference::Evaluate(TLearningArrayView<2, float> Output, const TLearningArrayView<2, const float> Input)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FNeuralNetworkInference::Evaluate);

		check(Output.Num<0>() == Input.Num<0>());
		check(Input.Num<1>() == InputSize);
		check(Output.Num<1>() == OutputSize);

		const int32 BatchSize = Output.Num<0>();

		check(BatchSize <= MaxBatchSize);

		if (BatchSize == 0) { return; }

		Array::Check(Input);

		if (Settings.bParallelEvaluation && BatchSize > Settings.MinParallelBatchSize)
		{
			const int32 IdealInferenceInstanceNum = ParallelForImpl::GetNumberOfThreadTasks(BatchSize, Settings.MinParallelBatchSize, EParallelForFlags::None);
			const int32 InferenceInstanceBatchSize = FMath::DivideAndRoundUp(BatchSize, IdealInferenceInstanceNum);
			const int32 InferenceInstanceNum = FMath::DivideAndRoundUp(BatchSize, InferenceInstanceBatchSize);

			check(InferenceInstanceNum <= ModelInstances.Num());

			ParallelFor(InferenceInstanceNum, [this, Input, Output, BatchSize, InferenceInstanceBatchSize](int32 InferenceInstanceIdx)
			{
				const int32 StartIndex = InferenceInstanceIdx * InferenceInstanceBatchSize;
				const int32 StopIndex = FMath::Min((InferenceInstanceIdx + 1) * InferenceInstanceBatchSize, BatchSize);
				const int32 InstanceBatchSize = StopIndex - StartIndex;

				const TLearningArrayView<2, const float> InputSlice = Input.Slice(StartIndex, InstanceBatchSize);
				const TLearningArrayView<2, float> OutputSlice = Output.Slice(StartIndex, InstanceBatchSize);

				ModelInstances[InferenceInstanceIdx]->SetInputTensorShapes({ NNE::FTensorShape::Make({(uint32)InstanceBatchSize, (uint32)InputSize}) });
				ModelInstances[InferenceInstanceIdx]->RunSync(
					{ { (void*)InputSlice.GetData(), InputSlice.Num() * sizeof(float) } },
					{ { (void*)OutputSlice.GetData(), OutputSlice.Num() * sizeof(float) } });
			});
		}
		else
		{
			ModelInstances[0]->SetInputTensorShapes({ NNE::FTensorShape::Make({(uint32)BatchSize, (uint32)InputSize}) });
			ModelInstances[0]->RunSync(
				{ { (void*)Input.GetData(), Input.Num() * sizeof(float) } },
				{ { (void*)Output.GetData(), Output.Num() * sizeof(float) } });
		}

		Array::Check(Output);
	}


	FNeuralNetworkFunction::FNeuralNetworkFunction(
		const int32 InMaxInstanceNum,
		const TSharedPtr<FNeuralNetwork>& InNeuralNetwork,
		const FNeuralNetworkInferenceSettings& InInferenceSettings) 
		: MaxInstanceNum(InMaxInstanceNum)
		, NeuralNetwork(InNeuralNetwork)
		, InferenceSettings(InInferenceSettings)
	{
		NeuralNetworkInference = NeuralNetwork->CreateInferenceObject(InMaxInstanceNum, InInferenceSettings);
		InputBuffer.SetNumUninitialized({ MaxInstanceNum, InNeuralNetwork->GetInputSize() });
		OutputBuffer.SetNumUninitialized({ MaxInstanceNum, InNeuralNetwork->GetOutputSize() });
	}

	void FNeuralNetworkFunction::Evaluate(
		TLearningArrayView<2, float> Output,
		const TLearningArrayView<2, const float> Input,
		const FIndexSet Instances)
	{
		// Evaluation is not thread-safe due to the scatter and gather we do into the InputBuffer and OutputBuffer
		// so we take a lock here since the interface for this function makes it appear like this might be thread safe as 
		// long as the Instances provided are not an overlapping set.
		FScopeNullableWriteLock ScopeLock(&EvaluationLock);

		const int32 InstanceNum = Instances.Num();

		if (InstanceNum == 0) { return; }

		// Gather Inputs

		Array::Check(Input, Instances);

		for (int32 InstanceIdx = 0; InstanceIdx < InstanceNum; InstanceIdx++)
		{
			Array::Copy(InputBuffer[InstanceIdx], Input[Instances[InstanceIdx]]);
		}

		// Evaluate Network

		NeuralNetworkInference->Evaluate(OutputBuffer.Slice(0, InstanceNum), InputBuffer.Slice(0, InstanceNum));

		// Scatter Outputs

		for (int32 InstanceIdx = 0; InstanceIdx < InstanceNum; InstanceIdx++)
		{
			Array::Copy(Output[Instances[InstanceIdx]], OutputBuffer[InstanceIdx]);
		}

		Array::Check(Output, Instances);
	}

	void FNeuralNetworkFunction::UpdateNeuralNetwork(const TSharedPtr<FNeuralNetwork>& NewNeuralNetwork)
	{
		if (NeuralNetwork != NewNeuralNetwork)
		{
			check(NeuralNetwork->GetInputSize() == NewNeuralNetwork->GetInputSize());
			check(NeuralNetwork->GetOutputSize() == NewNeuralNetwork->GetOutputSize());
			NeuralNetwork = NewNeuralNetwork;
			NeuralNetworkInference = NeuralNetwork->CreateInferenceObject(MaxInstanceNum, InferenceSettings);
		}
	}

	const TSharedPtr<FNeuralNetwork>& FNeuralNetworkFunction::GetNeuralNetwork() const
	{
		return NeuralNetwork;
	}

}
