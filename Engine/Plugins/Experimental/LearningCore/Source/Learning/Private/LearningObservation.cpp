// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningObservation.h"
#include "LearningRandom.h"

#include "NNERuntimeBasicCpuBuilder.h"

namespace UE::Learning::Observation
{
	namespace Private
	{
		template<typename T>
		static inline void ArraySerializeRaw(FArchive& Ar, TArray<T>& Array)
		{
			if (Ar.IsSaving())
			{
				int32 Num = Array.Num();
				Ar << Num;
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(T));
			}
			else if (Ar.IsLoading())
			{
				int32 Num = 0;
				Ar << Num;
				Array.SetNumUninitialized(Num);
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(T));
			}
		}

		static inline bool ContainsDuplicates(const TArrayView<const FName> ElementNames)
		{
			TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<32>> ElementNameSet;
			ElementNameSet.Append(ElementNames);
			return ElementNames.Num() != ElementNameSet.Num();
		}

		static inline bool CheckAllValid(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			for (const FSchemaElement SubElement : Elements)
			{
				if (!Schema.IsValid(SubElement)) { return false; }
			}
			return true;
		}

		static inline int32 GetMaxObservationVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size = FMath::Max(Size, Schema.GetObservationVectorSize(SubElement));
			}
			return Size;
		}

		static inline int32 GetTotalObservationVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetObservationVectorSize(SubElement);
			}
			return Size;
		}

		static inline int32 GetTotalEncodedObservationVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetEncodedVectorSize(SubElement);
			}
			return Size;
		}

		static inline int32 GetTotalObservationDistributionVectorSize(const FSchema& Schema, const TArrayView<const FSchemaElement> Elements)
		{
			int32 Size = 0;
			for (const FSchemaElement SubElement : Elements)
			{
				Size += Schema.GetObservationDistributionVectorSize(SubElement);
			}
			return Size;
		}

		static inline bool CheckAllValid(const FObject& Object, const TArrayView<const FObjectElement> Elements)
		{
			for (const FObjectElement SubElement : Elements)
			{
				if (!Object.IsValid(SubElement)) { return false; }
			}
			return true;
		}

		struct FContinuousDataDeprecated
		{
			int32 Num = INDEX_NONE;
			float Scale = 0.0f;
		};
	}

	FSchemaElement FSchema::CreateNull(const FName Tag)
	{
		const int32 Index = Types.Add(EType::Null);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(0);
		EncodedVectorSizes.Add(0);
		ObservationDistributionVectorSizes.Add(0);
		TypeDataIndices.Add(INDEX_NONE);

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateContinuous(const FSchemaContinuousParameters Parameters, const FName Tag)
	{
		check(Parameters.Num >= 0);

		FContinuousData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.NormalizationOption = Parameters.NormalizationOption;

		if (Parameters.NormalizationOption == ENormalization::Manual)
		{
			check(Parameters.Scales.Num() == Parameters.Num or Parameters.Scales.IsEmpty());
			check(Parameters.Offsets.Num() == Parameters.Num or Parameters.Offsets.IsEmpty());

			ElementData.NormalizationIndexOffset = NormalizationScales.Num();

			// Use provided arrays or identity (std=1, mean=0)
			if (Parameters.Scales.Num() == Parameters.Num)
			{
				NormalizationScales.Append(Parameters.Scales.GetData(), Parameters.Scales.Num());
			}
			else
			{
				for (int32 i = 0; i < Parameters.Num; i++)
				{
					NormalizationScales.Add(1.0f);
				}
			}

			if (Parameters.Offsets.Num() == Parameters.Num)
			{
				NormalizationOffsets.Append(Parameters.Offsets.GetData(), Parameters.Offsets.Num());
			}
			else
			{
				for (int32 i = 0; i < Parameters.Num; i++)
				{
					NormalizationOffsets.Add(0.0f);
				}
			}
		}

		const int32 Index = Types.Add(EType::Continuous);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.Num);
		EncodedVectorSizes.Add(Parameters.Num);
		ObservationDistributionVectorSizes.Add(Parameters.Num);
		TypeDataIndices.Add(ContinuousData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateDiscreteExclusive(const FSchemaDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		FDiscreteExclusiveData ElementData;
		ElementData.Num = Parameters.Num;

		const int32 Index = Types.Add(EType::DiscreteExclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.Num);
		EncodedVectorSizes.Add(Parameters.Num);
		ObservationDistributionVectorSizes.Add(Parameters.Num);
		TypeDataIndices.Add(DiscreteExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateDiscreteInclusive(const FSchemaDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		FDiscreteInclusiveData ElementData;
		ElementData.Num = Parameters.Num;

		const int32 Index = Types.Add(EType::DiscreteInclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.Num);
		EncodedVectorSizes.Add(Parameters.Num);
		ObservationDistributionVectorSizes.Add(Parameters.Num);
		TypeDataIndices.Add(DiscreteInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateNamedDiscreteExclusive(const FSchemaNamedDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		FNamedDiscreteExclusiveData ElementData;
		ElementData.Num = Parameters.ElementNames.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();

		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < ElementData.Num; Idx++) { SubElementObjects.Add(FSchemaElement()); }

		const int32 Index = Types.Add(EType::NamedDiscreteExclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.ElementNames.Num());
		EncodedVectorSizes.Add(Parameters.ElementNames.Num());
		ObservationDistributionVectorSizes.Add(Parameters.ElementNames.Num());
		TypeDataIndices.Add(NamedDiscreteExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateNamedDiscreteInclusive(const FSchemaNamedDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		FNamedDiscreteInclusiveData ElementData;
		ElementData.Num = Parameters.ElementNames.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();

		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < ElementData.Num; Idx++) { SubElementObjects.Add(FSchemaElement()); }

		const int32 Index = Types.Add(EType::NamedDiscreteInclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.ElementNames.Num());
		EncodedVectorSizes.Add(Parameters.ElementNames.Num());
		ObservationDistributionVectorSizes.Add(Parameters.ElementNames.Num());
		TypeDataIndices.Add(NamedDiscreteInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateAnd(const FSchemaAndParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		FAndData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		const int32 Index = Types.Add(EType::And);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Private::GetTotalObservationVectorSize(*this, Parameters.Elements));
		EncodedVectorSizes.Add(Private::GetTotalEncodedObservationVectorSize(*this, Parameters.Elements));
		ObservationDistributionVectorSizes.Add(Private::GetTotalObservationDistributionVectorSize(*this, Parameters.Elements));
		TypeDataIndices.Add(AndData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateOrExclusive(const FSchemaOrExclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		FOrExclusiveData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();
		ElementData.EncodingSize = Parameters.EncodingSize;

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		const int32 Index = Types.Add(EType::OrExclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Private::GetMaxObservationVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		EncodedVectorSizes.Add(Parameters.EncodingSize + Parameters.Elements.Num());
		ObservationDistributionVectorSizes.Add(Private::GetTotalObservationDistributionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		TypeDataIndices.Add(OrExclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateOrInclusive(const FSchemaOrInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		FOrInclusiveData ElementData;
		ElementData.Num = Parameters.Elements.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();
		ElementData.AttentionEncodingSize = Parameters.AttentionEncodingSize;
		ElementData.AttentionHeadNum = Parameters.AttentionHeadNum;
		ElementData.ValueEncodingSize = Parameters.ValueEncodingSize;

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		const int32 Index = Types.Add(EType::OrInclusive);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Private::GetTotalObservationVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		EncodedVectorSizes.Add(Parameters.AttentionHeadNum * Parameters.ValueEncodingSize + Parameters.Elements.Num());
		ObservationDistributionVectorSizes.Add(Private::GetTotalObservationDistributionVectorSize(*this, Parameters.Elements) + Parameters.Elements.Num());
		TypeDataIndices.Add(OrInclusiveData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateArray(const FSchemaArrayParameters Parameters, const FName Tag)
	{
		check(Parameters.Num >= 0);
		check(IsValid(Parameters.Element));

		FArrayData ElementData;
		ElementData.Num = Parameters.Num;
		ElementData.ElementIndex = SubElementObjects.Num();

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Array);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(GetObservationVectorSize(Parameters.Element) * Parameters.Num);
		EncodedVectorSizes.Add(GetEncodedVectorSize(Parameters.Element) * Parameters.Num);
		ObservationDistributionVectorSizes.Add(GetObservationDistributionVectorSize(Parameters.Element) * Parameters.Num);
		TypeDataIndices.Add(ArrayData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateSet(const FSchemaSetParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		FSetData ElementData;
		ElementData.MaxNum = Parameters.MaxNum;
		ElementData.ElementIndex = SubElementObjects.Num();
		ElementData.AttentionEncodingSize = Parameters.AttentionEncodingSize;
		ElementData.AttentionHeadNum = Parameters.AttentionHeadNum;
		ElementData.ValueEncodingSize = Parameters.ValueEncodingSize;

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Set);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(GetObservationVectorSize(Parameters.Element) * Parameters.MaxNum + Parameters.MaxNum);
		EncodedVectorSizes.Add(Parameters.ValueEncodingSize * Parameters.AttentionHeadNum + 1);
		ObservationDistributionVectorSizes.Add(GetObservationDistributionVectorSize(Parameters.Element));
		TypeDataIndices.Add(SetData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateEncoding(const FSchemaEncodingParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		FEncodingData ElementData;
		ElementData.ElementIndex = SubElementObjects.Num();
		ElementData.EncodingSize = Parameters.EncodingSize;
		ElementData.LayerNum = Parameters.LayerNum;
		ElementData.ActivationFunction = Parameters.ActivationFunction;

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Encoding);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(GetObservationVectorSize(Parameters.Element));
		EncodedVectorSizes.Add(Parameters.EncodingSize);
		ObservationDistributionVectorSizes.Add(GetObservationDistributionVectorSize(Parameters.Element));
		TypeDataIndices.Add(EncodingData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateConv1d(const FSchemaConv1dParameters Parameters, const FName Tag)
	{
		FConv1dData ElementData;
		ElementData.ElementIndex = SubElementObjects.Num();
		ElementData.InputLength = Parameters.InputLength;
		ElementData.InChannels = Parameters.InChannels;
		ElementData.OutChannels = Parameters.OutChannels;
		ElementData.KernelSize = Parameters.KernelSize;
		ElementData.Padding = Parameters.Padding;
		ElementData.PaddingMode = Parameters.PaddingMode;
		ElementData.ActivationFunction = Parameters.ActivationFunction;

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Conv1d);
		Tags.Add(Tag);
		
		ObservationVectorSizes.Add(GetObservationVectorSize(Parameters.Element));
		const int32 OutputLength = Parameters.InputLength + 2 * Parameters.Padding - Parameters.KernelSize + 1;
		EncodedVectorSizes.Add(OutputLength * Parameters.OutChannels);
		ObservationDistributionVectorSizes.Add(GetObservationDistributionVectorSize(Parameters.Element));

		TypeDataIndices.Add(Conv1dData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateConv2d(const FSchemaConv2dParameters Parameters, const FName Tag)
	{
		FConv2dData ElementData;
		ElementData.ElementIndex = SubElementObjects.Num();
		ElementData.InputHeight = Parameters.InputHeight;
		ElementData.InputWidth = Parameters.InputWidth;
		ElementData.InChannels = Parameters.InChannels;
		ElementData.OutChannels = Parameters.OutChannels;
		ElementData.KernelSize = Parameters.KernelSize;
		ElementData.Stride = Parameters.Stride;
		ElementData.Padding = Parameters.Padding;
		ElementData.PaddingMode = Parameters.PaddingMode;
		ElementData.ActivationFunction = Parameters.ActivationFunction;

		SubElementNames.Add(NAME_None);
		SubElementObjects.Add(Parameters.Element);

		const int32 Index = Types.Add(EType::Conv2d);
		Tags.Add(Tag);

		ObservationVectorSizes.Add(GetObservationVectorSize(Parameters.Element));
		const int32 OutputHeight = (Parameters.InputHeight + 2 * Parameters.Padding - Parameters.KernelSize) / Parameters.Stride + 1;
		const int32 OutputWidth = (Parameters.InputWidth + 2 * Parameters.Padding - Parameters.KernelSize) / Parameters.Stride + 1;
		EncodedVectorSizes.Add(OutputHeight * OutputWidth * Parameters.OutChannels);
		ObservationDistributionVectorSizes.Add(GetObservationDistributionVectorSize(Parameters.Element));

		TypeDataIndices.Add(Conv2dData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateSparse(const FSchemaSparseParameters Parameters, const FName Tag)
	{
		FSparseData ElementData;
		ElementData.Num = Parameters.Num;

		const int32 Index = Types.Add(EType::Sparse);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.Num);
		EncodedVectorSizes.Add(Parameters.Num);
		ObservationDistributionVectorSizes.Add(Parameters.Num);
		TypeDataIndices.Add(SparseData.Add(ElementData));

		return { Index, Generation };
	}

	FSchemaElement FSchema::CreateNamedSparse(const FSchemaNamedSparseParameters Parameters, const FName Tag)
	{
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		FNamedSparseData ElementData;
		ElementData.Num = Parameters.ElementNames.Num();
		ElementData.ElementsOffset = SubElementObjects.Num();

		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < ElementData.Num; Idx++) { SubElementObjects.Add(FSchemaElement()); }

		const int32 Index = Types.Add(EType::NamedSparse);
		Tags.Add(Tag);
		ObservationVectorSizes.Add(Parameters.ElementNames.Num());
		EncodedVectorSizes.Add(Parameters.ElementNames.Num());
		ObservationDistributionVectorSizes.Add(Parameters.ElementNames.Num());
		TypeDataIndices.Add(NamedSparseData.Add(ElementData));

		return { Index, Generation };
	}

	bool FSchema::IsValid(const FSchemaElement Element) const
	{
		return Element.Generation == Generation && Element.Index != INDEX_NONE;
	}

	EType FSchema::GetType(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return Types[Element.Index];
	}

	FName FSchema::GetTag(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return Tags[Element.Index];
	}

	int32 FSchema::GetObservationVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return ObservationVectorSizes[Element.Index];
	}

	int32 FSchema::GetEncodedVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return EncodedVectorSizes[Element.Index];
	}

	int32 FSchema::GetObservationDistributionVectorSize(const FSchemaElement Element) const
	{
		check(IsValid(Element));
		return ObservationDistributionVectorSizes[Element.Index];
	}

	FSchemaContinuousParameters FSchema::GetContinuous(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Continuous);
		const FContinuousData& ElementData = ContinuousData[TypeDataIndices[Element.Index]];

		FSchemaContinuousParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.NormalizationOption = ElementData.NormalizationOption;
		
		if (Parameters.NormalizationOption == ENormalization::Manual)
		{
			Parameters.Scales = TArrayView<const float>(NormalizationScales.GetData() + ElementData.NormalizationIndexOffset, ElementData.Num);
			Parameters.Offsets = TArrayView<const float>(NormalizationOffsets.GetData() + ElementData.NormalizationIndexOffset, ElementData.Num);
		}

		return Parameters;
	}

	FSchemaDiscreteExclusiveParameters FSchema::GetDiscreteExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteExclusive);
		const FDiscreteExclusiveData& ElementData = DiscreteExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaDiscreteExclusiveParameters Parameters;
		Parameters.Num = ElementData.Num;
		return Parameters;
	}

	FSchemaDiscreteInclusiveParameters FSchema::GetDiscreteInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteInclusive);
		const FDiscreteInclusiveData& ElementData = DiscreteInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaDiscreteInclusiveParameters Parameters;
		Parameters.Num = ElementData.Num;
		return Parameters;
	}

	FSchemaNamedDiscreteExclusiveParameters FSchema::GetNamedDiscreteExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteExclusive);
		const FNamedDiscreteExclusiveData& ElementData = NamedDiscreteExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaNamedDiscreteExclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaNamedDiscreteInclusiveParameters FSchema::GetNamedDiscreteInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteInclusive);
		const FNamedDiscreteInclusiveData& ElementData = NamedDiscreteInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaNamedDiscreteInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaAndParameters FSchema::GetAnd(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::And);
		const FAndData& ElementData = AndData[TypeDataIndices[Element.Index]];

		FSchemaAndParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		return Parameters;
	}

	FSchemaOrExclusiveParameters FSchema::GetOrExclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrExclusive);
		const FOrExclusiveData& ElementData = OrExclusiveData[TypeDataIndices[Element.Index]];

		FSchemaOrExclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.EncodingSize = ElementData.EncodingSize;
		return Parameters;
	}

	FSchemaOrInclusiveParameters FSchema::GetOrInclusive(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrInclusive);
		const FOrInclusiveData& ElementData = OrInclusiveData[TypeDataIndices[Element.Index]];

		FSchemaOrInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.Elements = TArrayView<const FSchemaElement>(SubElementObjects.GetData() + ElementData.ElementsOffset, ElementData.Num);
		Parameters.AttentionEncodingSize = ElementData.AttentionEncodingSize;
		Parameters.AttentionHeadNum = ElementData.AttentionHeadNum;
		Parameters.ValueEncodingSize = ElementData.ValueEncodingSize;
		return Parameters;
	}

	FSchemaArrayParameters FSchema::GetArray(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Array);
		const FArrayData& ElementData = ArrayData[TypeDataIndices[Element.Index]];

		FSchemaArrayParameters Parameters;
		Parameters.Num = ElementData.Num;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		return Parameters;
	}

	FSchemaSetParameters FSchema::GetSet(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Set);
		const FSetData& ElementData = SetData[TypeDataIndices[Element.Index]];

		FSchemaSetParameters Parameters;
		Parameters.MaxNum = ElementData.MaxNum;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		Parameters.AttentionEncodingSize = ElementData.AttentionEncodingSize;
		Parameters.AttentionHeadNum = ElementData.AttentionHeadNum;
		Parameters.ValueEncodingSize = ElementData.ValueEncodingSize;
		return Parameters;
	}

	FSchemaEncodingParameters FSchema::GetEncoding(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Encoding);
		const FEncodingData& ElementData = EncodingData[TypeDataIndices[Element.Index]];

		FSchemaEncodingParameters Parameters;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		Parameters.EncodingSize = ElementData.EncodingSize;
		Parameters.LayerNum = ElementData.LayerNum;
		Parameters.ActivationFunction = ElementData.ActivationFunction;
		return Parameters;
	}

	FSchemaConv1dParameters FSchema::GetConv1d(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Conv1d);
		const FConv1dData& ElementData = Conv1dData[TypeDataIndices[Element.Index]];

		FSchemaConv1dParameters Parameters;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		Parameters.InputLength = ElementData.InputLength;
		Parameters.InChannels = ElementData.InChannels;
		Parameters.OutChannels = ElementData.OutChannels;
		Parameters.KernelSize = ElementData.KernelSize;
		Parameters.Padding = ElementData.Padding;
		Parameters.PaddingMode = ElementData.PaddingMode;
		Parameters.ActivationFunction = ElementData.ActivationFunction;
		return Parameters;
	}

	FSchemaConv2dParameters FSchema::GetConv2d(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Conv2d);
		const FConv2dData& ElementData = Conv2dData[TypeDataIndices[Element.Index]];

		FSchemaConv2dParameters Parameters;
		Parameters.Element = SubElementObjects[ElementData.ElementIndex];
		Parameters.InputHeight = ElementData.InputHeight;
		Parameters.InputWidth = ElementData.InputWidth;
		Parameters.InChannels = ElementData.InChannels;
		Parameters.OutChannels = ElementData.OutChannels;
		Parameters.KernelSize = ElementData.KernelSize;
		Parameters.Stride = ElementData.Stride;
		Parameters.Padding = ElementData.Padding;
		Parameters.PaddingMode = ElementData.PaddingMode;
		Parameters.ActivationFunction = ElementData.ActivationFunction;
		return Parameters;
	}

	FSchemaSparseParameters FSchema::GetSparse(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Sparse);
		const FSparseData& ElementData = SparseData[TypeDataIndices[Element.Index]];

		FSchemaSparseParameters Parameters;
		Parameters.Num = ElementData.Num;
		return Parameters;
	}

	FSchemaNamedSparseParameters FSchema::GetNamedSparse(const FSchemaElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedSparse);
		const FNamedSparseData& ElementData = NamedSparseData[TypeDataIndices[Element.Index]];

		FSchemaNamedSparseParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + ElementData.ElementsOffset, ElementData.Num);
		return Parameters;
	}

	void FSchema::SetGeneration(const uint32 InGeneration)
	{
		Generation = InGeneration;
	}

	uint32 FSchema::GetGeneration() const
	{
		return Generation;
	}

	void FSchema::Empty()
	{
		Types.Empty();
		Tags.Empty();
		ObservationVectorSizes.Empty();
		EncodedVectorSizes.Empty();
		ObservationDistributionVectorSizes.Empty();
		TypeDataIndices.Empty();

		ContinuousData.Empty();
		DiscreteExclusiveData.Empty();
		DiscreteInclusiveData.Empty();
		NamedDiscreteExclusiveData.Empty();
		NamedDiscreteInclusiveData.Empty();	
		AndData.Empty();
		OrExclusiveData.Empty();
		OrInclusiveData.Empty();
		ArrayData.Empty();
		SetData.Empty();
		EncodingData.Empty();
		Conv1dData.Empty();
		Conv2dData.Empty();
		SparseData.Empty();
		NamedSparseData.Empty();

		SubElementNames.Empty();
		SubElementObjects.Empty();

		NormalizationScales.Empty();
		NormalizationOffsets.Empty();

		Generation++;
	}

	bool FSchema::IsEmpty() const
	{
		return Types.IsEmpty();
	}

	void FSchema::Reset()
	{
		Types.Reset();
		Tags.Reset();
		ObservationVectorSizes.Reset();
		EncodedVectorSizes.Reset();
		ObservationDistributionVectorSizes.Reset();
		TypeDataIndices.Reset();

		ContinuousData.Reset();
		DiscreteExclusiveData.Reset();
		DiscreteInclusiveData.Reset();
		NamedDiscreteExclusiveData.Reset();
		NamedDiscreteInclusiveData.Reset();			
		AndData.Reset();
		OrExclusiveData.Reset();
		OrInclusiveData.Reset();
		ArrayData.Reset();
		SetData.Reset();
		EncodingData.Reset();
		Conv1dData.Reset();
		Conv2dData.Reset();
		SparseData.Reset();
		NamedSparseData.Reset();

		SubElementNames.Reset();
		SubElementObjects.Reset();

		NormalizationScales.Reset();
		NormalizationOffsets.Reset();

		Generation++;
	}

	bool FSchema::Serialize(FArchive& Ar)
	{
		static constexpr int32 Magic = 0x7db41964;
		static constexpr int32 Version = 2;

		int32 LoadMagic = 0;
		int32 LoadVersion = Version;

		if (Ar.IsLoading())
		{
			int64 Offset = Ar.Tell();
			Ar << LoadMagic;
			if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); Empty(); return false; }
			Ar << LoadVersion;

			if (LoadVersion > Version) { Ar.Seek(Offset); Empty(); return false; }
		}
		else
		{
			int32 SaveMagic = Magic;
			int32 SaveVersion = Version;
			Ar << SaveMagic;
			Ar << SaveVersion;
		}

		Ar << Generation;

		Ar << Types;
		Ar << Tags;
		Ar << ObservationVectorSizes;
		Ar << EncodedVectorSizes;
		Ar << ObservationDistributionVectorSizes;
		Ar << TypeDataIndices;

		if (LoadVersion == 0)
		{
			TArray<Private::FContinuousDataDeprecated> DeprecatedContinuousData;
			Private::ArraySerializeRaw(Ar, DeprecatedContinuousData);

			int32 TotalSize = 0;
			for (const Private::FContinuousDataDeprecated& DeprecatedData : DeprecatedContinuousData)
			{
				TotalSize += DeprecatedData.Num;
			}

			ContinuousData.Reserve(DeprecatedContinuousData.Num());
			NormalizationScales.Reserve(TotalSize);
			NormalizationOffsets.Reserve(TotalSize);

			for (const Private::FContinuousDataDeprecated& DeprecatedData : DeprecatedContinuousData)
			{
				FContinuousData CurrentData;
				CurrentData.Num = DeprecatedData.Num;
				CurrentData.NormalizationOption = ENormalization::Manual;
				CurrentData.NormalizationIndexOffset = NormalizationScales.Num();
				ContinuousData.Add(CurrentData);

				for (int32 Idx = 0; Idx < DeprecatedData.Num; Idx++)
				{
					NormalizationScales.Add(DeprecatedData.Scale);
					NormalizationOffsets.Add(0.0f);
				}
			}
		}
		else
		{
			Private::ArraySerializeRaw(Ar, ContinuousData);
		}

		Private::ArraySerializeRaw(Ar, DiscreteExclusiveData);
		Private::ArraySerializeRaw(Ar, DiscreteInclusiveData);
		Private::ArraySerializeRaw(Ar, NamedDiscreteExclusiveData);
		Private::ArraySerializeRaw(Ar, NamedDiscreteInclusiveData);
		Private::ArraySerializeRaw(Ar, AndData);
		Private::ArraySerializeRaw(Ar, OrExclusiveData);
		Private::ArraySerializeRaw(Ar, OrInclusiveData);
		Private::ArraySerializeRaw(Ar, ArrayData);
		Private::ArraySerializeRaw(Ar, SetData);
		Private::ArraySerializeRaw(Ar, EncodingData);
		Private::ArraySerializeRaw(Ar, Conv1dData);
		Private::ArraySerializeRaw(Ar, Conv2dData);

		if (LoadVersion > 1)
		{
			Private::ArraySerializeRaw(Ar, SparseData);
			Private::ArraySerializeRaw(Ar, NamedSparseData);
		}

		Ar << SubElementNames;
		Ar << SubElementObjects;

		if (LoadVersion > 0)
		{
			Private::ArraySerializeRaw(Ar, NormalizationScales);
			Private::ArraySerializeRaw(Ar, NormalizationOffsets);
		}

		return true;
	}

	FObjectElement FObject::CreateNull(const FName Tag)
	{
		const int32 Index = Types.Add(EType::Null);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(0);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateContinuous(const FObjectContinuousParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::Continuous);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(Parameters.Values.Num());

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(0);

		ContinuousValues.Append(Parameters.Values);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateDiscreteExclusive(const FObjectDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(1);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(0);

		DiscreteValues.Add(Parameters.DiscreteIndex);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateDiscreteInclusive(const FObjectDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::DiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(Parameters.DiscreteIndices.Num());

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(0);

		DiscreteValues.Append(Parameters.DiscreteIndices);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateNamedDiscreteExclusive(const FObjectNamedDiscreteExclusiveParameters Parameters, const FName Tag)
	{
		const int32 Index = Types.Add(EType::NamedDiscreteExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(1);

		SubElementNames.Add(Parameters.ElementName);
		SubElementObjects.Add(FObjectElement());

		return { Index, Generation };
	}

	FObjectElement FObject::CreateNamedDiscreteInclusive(const FObjectNamedDiscreteInclusiveParameters Parameters, const FName Tag)
	{
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		const int32 Index = Types.Add(EType::NamedDiscreteInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.ElementNames.Num());

		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < Parameters.ElementNames.Num(); Idx++) { SubElementObjects.Add(FObjectElement()); }

		return { Index, Generation };
	}

	FObjectElement FObject::CreateAnd(const FObjectAndParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::And);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.Elements.Num());

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateOrExclusive(const FObjectOrExclusiveParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::OrExclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(1);

		SubElementNames.Add(Parameters.ElementName);
		SubElementObjects.Add(Parameters.Element);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateOrInclusive(const FObjectOrInclusiveParameters Parameters, const FName Tag)
	{
		check(Parameters.Elements.Num() == Parameters.ElementNames.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::OrInclusive);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.Elements.Num());

		SubElementNames.Append(Parameters.ElementNames);
		SubElementObjects.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateArray(const FObjectArrayParameters Parameters, const FName Tag)
	{
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::Array);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.Elements.Num());

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			SubElementNames.Add(NAME_Name);
		}
		SubElementObjects.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateSet(const FObjectSetParameters Parameters, const FName Tag)
	{
		check(Private::CheckAllValid(*this, Parameters.Elements));

		const int32 Index = Types.Add(EType::Set);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.Elements.Num());

		for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
		{
			SubElementNames.Add(NAME_Name);
		}
		SubElementObjects.Append(Parameters.Elements);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateEncoding(const FObjectEncodingParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::Encoding);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(1);

		SubElementNames.Add(NAME_Name);
		SubElementObjects.Add(Parameters.Element);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateConv1d(const FObjectConv1dParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::Conv1d);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(1);

		SubElementNames.Add(NAME_Name);
		SubElementObjects.Add(Parameters.Element);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateConv2d(const FObjectConv2dParameters Parameters, const FName Tag)
	{
		check(IsValid(Parameters.Element));

		const int32 Index = Types.Add(EType::Conv2d);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(0);

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(1);

		SubElementNames.Add(NAME_Name);
		SubElementObjects.Add(Parameters.Element);

		return { Index, Generation };
	}


	FObjectElement FObject::CreateSparse(const FObjectSparseParameters Parameters, const FName Tag)
	{
		check(Parameters.DiscreteIndices.Num() == Parameters.Values.Num());

		const int32 Index = Types.Add(EType::Sparse);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(Parameters.Values.Num());

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(Parameters.DiscreteIndices.Num());

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(0);

		DiscreteValues.Append(Parameters.DiscreteIndices);
		ContinuousValues.Append(Parameters.Values);

		return { Index, Generation };
	}

	FObjectElement FObject::CreateNamedSparse(const FObjectNamedSparseParameters Parameters, const FName Tag)
	{
		check(Parameters.ElementNames.Num() == Parameters.Values.Num());
		check(!Private::ContainsDuplicates(Parameters.ElementNames));

		const int32 Index = Types.Add(EType::NamedSparse);
		Tags.Add(Tag);

		ContinuousDataOffsets.Add(ContinuousValues.Num());
		ContinuousDataNums.Add(Parameters.Values.Num());

		DiscreteDataOffsets.Add(DiscreteValues.Num());
		DiscreteDataNums.Add(0);

		SubElementDataOffsets.Add(SubElementObjects.Num());
		SubElementDataNums.Add(Parameters.ElementNames.Num());

		SubElementNames.Append(Parameters.ElementNames);
		for (int32 Idx = 0; Idx < Parameters.ElementNames.Num(); Idx++) { SubElementObjects.Add(FObjectElement()); }

		ContinuousValues.Append(Parameters.Values);

		return { Index, Generation };
	}

	bool FObject::IsValid(const FObjectElement Element) const
	{
		return Element.Generation == Generation && Element.Index != INDEX_NONE;
	}

	EType FObject::GetType(const FObjectElement Element) const
	{
		check(IsValid(Element));
		return Types[Element.Index];
	}

	FName FObject::GetTag(const FObjectElement Element) const
	{
		check(IsValid(Element));
		return Tags[Element.Index];
	}

	FObjectContinuousParameters FObject::GetContinuous(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Continuous);

		FObjectContinuousParameters Parameters;
		Parameters.Values = TArrayView<const float>(ContinuousValues.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		return Parameters;
	}

	FObjectDiscreteExclusiveParameters FObject::GetDiscreteExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteExclusive);

		FObjectDiscreteExclusiveParameters Parameters;
		Parameters.DiscreteIndex = DiscreteValues[DiscreteDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectDiscreteInclusiveParameters FObject::GetDiscreteInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::DiscreteInclusive);

		FObjectDiscreteInclusiveParameters Parameters;
		Parameters.DiscreteIndices = TArrayView<const int32>(DiscreteValues.GetData() + DiscreteDataOffsets[Element.Index], DiscreteDataNums[Element.Index]);
		return Parameters;
	}

	FObjectNamedDiscreteExclusiveParameters FObject::GetNamedDiscreteExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteExclusive);

		FObjectNamedDiscreteExclusiveParameters Parameters;
		Parameters.ElementName = SubElementNames[SubElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectNamedDiscreteInclusiveParameters FObject::GetNamedDiscreteInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedDiscreteInclusive);

		FObjectNamedDiscreteInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectAndParameters FObject::GetAnd(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::And);

		FObjectAndParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectOrExclusiveParameters FObject::GetOrExclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrExclusive);

		FObjectOrExclusiveParameters Parameters;
		Parameters.ElementName = SubElementNames[SubElementDataOffsets[Element.Index]];
		Parameters.Element = SubElementObjects[SubElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectOrInclusiveParameters FObject::GetOrInclusive(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::OrInclusive);

		FObjectOrInclusiveParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectArrayParameters FObject::GetArray(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Array);

		FObjectArrayParameters Parameters;
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectSetParameters FObject::GetSet(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Set);

		FObjectSetParameters Parameters;
		Parameters.Elements = TArrayView<const FObjectElement>(SubElementObjects.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		return Parameters;
	}

	FObjectEncodingParameters FObject::GetEncoding(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Encoding);

		FObjectEncodingParameters Parameters;
		Parameters.Element = SubElementObjects[SubElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectConv1dParameters FObject::GetConv1d(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Conv1d);

		FObjectConv1dParameters Parameters;
		Parameters.Element = SubElementObjects[SubElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectConv2dParameters FObject::GetConv2d(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Conv2d);

		FObjectConv2dParameters Parameters;
		Parameters.Element = SubElementObjects[SubElementDataOffsets[Element.Index]];
		return Parameters;
	}

	FObjectSparseParameters FObject::GetSparse(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::Sparse);

		FObjectSparseParameters Parameters;
		Parameters.DiscreteIndices = TArrayView<const int32>(DiscreteValues.GetData() + DiscreteDataOffsets[Element.Index], DiscreteDataNums[Element.Index]);
		Parameters.Values = TArrayView<const float>(ContinuousValues.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		return Parameters;
	}

	FObjectNamedSparseParameters FObject::GetNamedSparse(const FObjectElement Element) const
	{
		check(IsValid(Element) && GetType(Element) == EType::NamedSparse);

		FObjectNamedSparseParameters Parameters;
		Parameters.ElementNames = TArrayView<const FName>(SubElementNames.GetData() + SubElementDataOffsets[Element.Index], SubElementDataNums[Element.Index]);
		Parameters.Values = TArrayView<const float>(ContinuousValues.GetData() + ContinuousDataOffsets[Element.Index], ContinuousDataNums[Element.Index]);
		return Parameters;
	}

	void FObject::SetGeneration(const uint32 InGeneration)
	{
		Generation = InGeneration;
	}

	uint32 FObject::GetGeneration() const
	{
		return Generation;
	}

	void FObject::Empty()
	{
		Types.Empty();
		Tags.Empty();

		ContinuousDataOffsets.Empty();
		ContinuousDataNums.Empty();

		DiscreteDataOffsets.Empty();
		DiscreteDataNums.Empty();

		SubElementDataOffsets.Empty();
		SubElementDataNums.Empty();

		ContinuousValues.Empty();
		DiscreteValues.Empty();
		SubElementNames.Empty();
		SubElementObjects.Empty();

		Generation++;
	}

	bool FObject::IsEmpty() const
	{
		return Types.IsEmpty();
	}

	void FObject::Reset()
	{
		Types.Reset();
		Tags.Reset();

		ContinuousDataOffsets.Reset();
		ContinuousDataNums.Reset();

		DiscreteDataOffsets.Reset();
		DiscreteDataNums.Reset();

		SubElementDataOffsets.Reset();
		SubElementDataNums.Reset();

		ContinuousValues.Reset();
		DiscreteValues.Reset();
		SubElementNames.Reset();
		SubElementObjects.Reset();

		Generation++;
	}

	bool FObject::Serialize(FArchive& Ar)
	{
		static constexpr int32 Magic = 0xf71282d9;
		static constexpr int32 Version = 0;

		if (Ar.IsLoading())
		{
			int64 Offset = Ar.Tell();
			int32 LoadMagic = 0;
			int32 LoadVersion = 0;
			Ar << LoadMagic;
			if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); Empty(); return false; }
			Ar << LoadVersion;
			if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); Empty(); return false; }
		}
		else
		{
			int32 SaveMagic = Magic;
			int32 SaveVersion = Version;
			Ar << SaveMagic;
			Ar << SaveVersion;
		}

		Ar << Generation;

		Ar << Types;
		Ar << Tags;

		Ar << ContinuousDataOffsets;
		Ar << ContinuousDataNums;

		Ar << DiscreteDataOffsets;
		Ar << DiscreteDataNums;

		Ar << SubElementDataOffsets;
		Ar << SubElementDataNums;

		Ar << ContinuousValues;
		Ar << DiscreteValues;
		Ar << SubElementNames;
		Ar << SubElementObjects;

		return true;
	}

	namespace Private
	{
		static inline NNE::RuntimeBasic::FModelBuilder::EActivationFunction GetNNEActivationFunction(const EEncodingActivationFunction ActivationFunction)
		{
			switch (ActivationFunction)
			{
			case EEncodingActivationFunction::ReLU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
			case EEncodingActivationFunction::ELU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ELU;
			case EEncodingActivationFunction::TanH: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::TanH;
			case EEncodingActivationFunction::GELU: return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::GELU;
			default: checkNoEntry(); return NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
			}
		}

		static inline NNE::RuntimeBasic::FModelBuilder::EPaddingMode GetNNEPaddingMode(const EPaddingMode PaddingMode)
		{
			switch (PaddingMode)
			{
			case EPaddingMode::Zeros: return NNE::RuntimeBasic::FModelBuilder::EPaddingMode::Zeros;
			case EPaddingMode::Circular: return NNE::RuntimeBasic::FModelBuilder::EPaddingMode::Circular;
			default: checkNoEntry(); return NNE::RuntimeBasic::FModelBuilder::EPaddingMode::Zeros;
			}
		}

		static inline int32 HashFNameStable(const FName Name)
		{
			const FString NameString = Name.ToString().ToLower();
			return (int32)CityHash32(
				(const char*)NameString.GetCharArray().GetData(), 
				sizeof(NameString[0]) * NameString.GetCharArray().Num());
		}

		static inline int32 HashInt(const int32 Int)
		{
			return (int32)CityHash32((const char*)&Int, sizeof(int32));
		}

		static inline int32 HashCombine(const TArrayView<const int32> Hashes)
		{
			return (int32)CityHash32((const char*)Hashes.GetData(), Hashes.Num() * Hashes.GetTypeSize());
		}

		static inline int32 HashElements(
			const FSchema& Schema,
			const TArrayView<const FName> SchemaElementNames,
			const int32 Salt)
		{
			// Note: Here we xor all entries together. 
			// This makes the hash in invariant to the ordering of names which is actually what we want 
			// since this array is representing a set-like structure and it is fine to pass elements in a different order.

			int32 Hash = 0x5592716a;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaElementNames.Num(); SchemaElementIdx++)
			{
				Hash ^= HashFNameStable(SchemaElementNames[SchemaElementIdx]);
			}

			return Hash;
		}

		static inline int32 HashElements(
			const FSchema& Schema,
			const TArrayView<const FName> SchemaElementNames,
			const TArrayView<const FSchemaElement> SchemaElements,
			const int32 Salt)
		{
			// Note: Here we xor all entries together. 
			// This makes the hash in invariant to the ordering of pairs of names and elements 
			// which is actually what we want since these two arrays are representing a map-like 
			// structure and it is fine to pass keys and values in a different order.

			int32 Hash = 0x5b3bbe4d;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaElements.Num(); SchemaElementIdx++)
			{
				Hash ^= HashCombine({ HashFNameStable(SchemaElementNames[SchemaElementIdx]), GetSchemaObjectsCompatibilityHash(Schema, SchemaElements[SchemaElementIdx], Salt) });
			}

			return Hash;
		}
	}

	int32 GetSchemaObjectsCompatibilityHash(
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const int32 Salt)
	{
		check(Schema.IsValid(SchemaElement));
		const EType SchemaElementType = Schema.GetType(SchemaElement);

		const int32 Hash = Private::HashCombine({ Salt, Private::HashInt((int32)SchemaElementType) });

		switch (SchemaElementType)
		{
		case EType::Null: return Hash;

		case EType::Continuous: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetContinuous(SchemaElement).Num) });

		case EType::DiscreteExclusive: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetDiscreteExclusive(SchemaElement).Num) });
		case EType::DiscreteInclusive: return Private::HashCombine({ Hash, Private::HashInt(Schema.GetDiscreteInclusive(SchemaElement).Num) });

		case EType::NamedDiscreteExclusive:
		{
			const FSchemaNamedDiscreteExclusiveParameters Parameters = Schema.GetNamedDiscreteExclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Salt) });
		}

		case EType::NamedDiscreteInclusive:
		{
			const FSchemaNamedDiscreteInclusiveParameters Parameters = Schema.GetNamedDiscreteInclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Salt) });
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Parameters.Elements, Salt) });
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashInt(Parameters.Num), GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt) });
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashInt(Parameters.MaxNum), GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt) });
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);
			return GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt);
		}

		case EType::Conv1d:
		{
			const FSchemaConv1dParameters Parameters = Schema.GetConv1d(SchemaElement);
			return Private::HashCombine({
				Hash, 
				Private::HashInt(Parameters.InputLength), 
				Private::HashInt(Parameters.InChannels), 
				Private::HashInt(Parameters.OutChannels), 
				Private::HashInt(Parameters.KernelSize),
				Private::HashInt(Parameters.Padding),
				GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt) });
		}

		case EType::Conv2d:
		{
			const FSchemaConv2dParameters Parameters = Schema.GetConv2d(SchemaElement);
			return Private::HashCombine({
				Hash,
				Private::HashInt(Parameters.InputHeight),
				Private::HashInt(Parameters.InputWidth),
				Private::HashInt(Parameters.InChannels),
				Private::HashInt(Parameters.OutChannels),
				Private::HashInt(Parameters.KernelSize),
				Private::HashInt(Parameters.Stride),
				Private::HashInt(Parameters.Padding),
				GetSchemaObjectsCompatibilityHash(Schema, Parameters.Element, Salt) });
		}

		case EType::Sparse:
		{
			return Private::HashCombine({ Hash, Private::HashInt(Schema.GetSparse(SchemaElement).Num) });
		}

		case EType::NamedSparse:
		{
			const FSchemaNamedSparseParameters Parameters = Schema.GetNamedSparse(SchemaElement);
			return Private::HashCombine({ Hash, Private::HashElements(Schema, Parameters.ElementNames, Salt) });
		}

		default:
		{
			checkNoEntry();
			return 0;
		}

		}
	}

	bool AreSchemaObjectsCompatible(
		const FSchema& SchemaA,
		const FSchemaElement SchemaElementA,
		const FSchema& SchemaB,
		const FSchemaElement SchemaElementB)
	{
		check(SchemaA.IsValid(SchemaElementA));
		check(SchemaB.IsValid(SchemaElementB));

		const EType SchemaElementTypeA = SchemaA.GetType(SchemaElementA);
		const EType SchemaElementTypeB = SchemaB.GetType(SchemaElementB);
		
		// If any element is an encoding element we forward the comparison to the sub-element since encoding elements don't affect compatibility
		if (SchemaElementTypeA == EType::Encoding) { return AreSchemaObjectsCompatible(SchemaA, SchemaA.GetEncoding(SchemaElementA).Element, SchemaB, SchemaElementB); }
		if (SchemaElementTypeB == EType::Encoding) { return AreSchemaObjectsCompatible(SchemaA, SchemaElementA, SchemaB, SchemaB.GetEncoding(SchemaElementB).Element); }
		
		// Otherwise if types don't match we immediately know elements are incompatible
		if (SchemaElementTypeA != SchemaElementTypeB) { return false; }

		// This is an early-out since if the input sizes are different we are definitely incompatible
		if (SchemaA.GetObservationVectorSize(SchemaElementA) != SchemaB.GetObservationVectorSize(SchemaElementB)) { return false; }

		switch (SchemaElementTypeA)
		{
		case EType::Null: return true;

		case EType::Continuous: return SchemaA.GetContinuous(SchemaElementA).Num == SchemaB.GetContinuous(SchemaElementB).Num;
		
		case EType::DiscreteExclusive: return SchemaA.GetDiscreteExclusive(SchemaElementA).Num == SchemaB.GetDiscreteExclusive(SchemaElementB).Num;
		case EType::DiscreteInclusive: return SchemaA.GetDiscreteInclusive(SchemaElementA).Num == SchemaB.GetDiscreteInclusive(SchemaElementB).Num;

		case EType::NamedDiscreteExclusive:
		{
			const FSchemaNamedDiscreteExclusiveParameters ParametersA = SchemaA.GetNamedDiscreteExclusive(SchemaElementA);
			const FSchemaNamedDiscreteExclusiveParameters ParametersB = SchemaB.GetNamedDiscreteExclusive(SchemaElementB);

			if (ParametersA.ElementNames.Num() != ParametersB.ElementNames.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.ElementNames.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
			}

			return true;
		}

		case EType::NamedDiscreteInclusive:
		{
			const FSchemaNamedDiscreteInclusiveParameters ParametersA = SchemaA.GetNamedDiscreteInclusive(SchemaElementA);
			const FSchemaNamedDiscreteInclusiveParameters ParametersB = SchemaB.GetNamedDiscreteInclusive(SchemaElementB);

			if (ParametersA.ElementNames.Num() != ParametersB.ElementNames.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.ElementNames.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
			}

			return true;
		}

		case EType::And:
		{
			const FSchemaAndParameters ParametersA = SchemaA.GetAnd(SchemaElementA);
			const FSchemaAndParameters ParametersB = SchemaB.GetAnd(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters ParametersA = SchemaA.GetOrExclusive(SchemaElementA);
			const FSchemaOrExclusiveParameters ParametersB = SchemaB.GetOrExclusive(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters ParametersA = SchemaA.GetOrInclusive(SchemaElementA);
			const FSchemaOrInclusiveParameters ParametersB = SchemaB.GetOrInclusive(SchemaElementB);

			if (ParametersA.Elements.Num() != ParametersB.Elements.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.Elements.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
				if (!AreSchemaObjectsCompatible(SchemaA, ParametersA.Elements[SchemaElementAIdx], SchemaB, ParametersB.Elements[SchemaElementBIdx])) { return false; }
			}

			return true;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters ParametersA = SchemaA.GetArray(SchemaElementA);
			const FSchemaArrayParameters ParametersB = SchemaB.GetArray(SchemaElementB);

			return (ParametersA.Num == ParametersB.Num) && AreSchemaObjectsCompatible(SchemaA, ParametersA.Element, SchemaB, ParametersB.Element);
		}

		case EType::Set:
		{
			const FSchemaSetParameters ParametersA = SchemaA.GetSet(SchemaElementA);
			const FSchemaSetParameters ParametersB = SchemaB.GetSet(SchemaElementB);

			return (ParametersA.MaxNum == ParametersB.MaxNum) && AreSchemaObjectsCompatible(SchemaA, ParametersA.Element, SchemaB, ParametersB.Element);
		}

		case EType::Encoding:
		{
			checkf(false, TEXT("Encoding elements should always be forwarded..."));
			return false;
		}

		case EType::Conv1d:
		{
			const FSchemaConv1dParameters ParametersA = SchemaA.GetConv1d(SchemaElementA);
			const FSchemaConv1dParameters ParametersB = SchemaB.GetConv1d(SchemaElementB);

			return (ParametersA.InputLength == ParametersB.InputLength)
				&& (ParametersA.InChannels == ParametersB.InChannels)
				&& (ParametersA.OutChannels == ParametersB.OutChannels)
				&& (ParametersA.KernelSize == ParametersB.KernelSize)
				&& (ParametersA.Padding == ParametersB.Padding)
				&& AreSchemaObjectsCompatible(SchemaA, ParametersA.Element, SchemaB, ParametersB.Element);
		}


		case EType::Conv2d:
		{
			const FSchemaConv2dParameters ParametersA = SchemaA.GetConv2d(SchemaElementA);
			const FSchemaConv2dParameters ParametersB = SchemaB.GetConv2d(SchemaElementB);

			return (ParametersA.InputHeight == ParametersB.InputHeight)
				&& (ParametersA.InputWidth == ParametersB.InputWidth)
				&& (ParametersA.InChannels == ParametersB.InChannels)
				&& (ParametersA.OutChannels == ParametersB.OutChannels)
				&& (ParametersA.KernelSize == ParametersB.KernelSize)
				&& (ParametersA.Stride == ParametersB.Stride)
				&& (ParametersA.Padding == ParametersB.Padding)
				&& AreSchemaObjectsCompatible(SchemaA, ParametersA.Element, SchemaB, ParametersB.Element);
		}

		case EType::Sparse: return SchemaA.GetSparse(SchemaElementA).Num == SchemaB.GetSparse(SchemaElementB).Num;

		case EType::NamedSparse:
		{
			const FSchemaNamedSparseParameters ParametersA = SchemaA.GetNamedSparse(SchemaElementA);
			const FSchemaNamedSparseParameters ParametersB = SchemaB.GetNamedSparse(SchemaElementB);

			if (ParametersA.ElementNames.Num() != ParametersB.ElementNames.Num()) { return false; }

			for (int32 SchemaElementAIdx = 0; SchemaElementAIdx < ParametersA.ElementNames.Num(); SchemaElementAIdx++)
			{
				const int32 SchemaElementBIdx = ParametersB.ElementNames.Find(ParametersA.ElementNames[SchemaElementAIdx]);
				if (SchemaElementBIdx == INDEX_NONE) { return false; }
			}

			return true;
		}

		default:
		{
			checkNoEntry();
			return false;
		}

		}
	}

	void MakeEncoderNetworkModelBuilderElementFromSchema(
		NNE::RuntimeBasic::FModelBuilderElement& OutElement,
		NNE::RuntimeBasic::FModelBuilder& Builder,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings)
	{
		const EType SchemaElementType = Schema.GetType(SchemaElement);

		switch (SchemaElementType)
		{
		case EType::Null:
		{
			OutElement = Builder.MakeCopy(0);
			break;
		}

		case EType::Continuous:
		{
			const int32 ValueNum = Schema.GetContinuous(SchemaElement).Num;

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::DiscreteExclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteExclusive(SchemaElement).Num;

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::DiscreteInclusive:
		{
			const int32 ValueNum = Schema.GetDiscreteInclusive(SchemaElement).Num;

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::NamedDiscreteExclusive:
		{
			const int32 ValueNum = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num();

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::NamedDiscreteInclusive:
		{
			const int32 ValueNum = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num();

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
			BuilderLayers.Reserve(Parameters.Elements.Num());
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderLayers.Emplace(BuilderSubElement);
			}

			OutElement = Builder.MakeConcat(BuilderLayers);
			break;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderSubLayers;
			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderEncoders;
			BuilderSubLayers.Reserve(Parameters.Elements.Num());
			BuilderEncoders.Reserve(Parameters.Elements.Num());
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(SubElement);
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderSubLayers.Emplace(BuilderSubElement);

				NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
				LinearLayerSettings.Type = NetworkSettings.bUseCompressedLinearLayers ?
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

				switch (NetworkSettings.WeightInitialization)
				{
				case EWeightInitialization::KaimingGaussian: LinearLayerSettings.WeightInitializationSettings.Type =
					NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
				case EWeightInitialization::KaimingUniform: LinearLayerSettings.WeightInitializationSettings.Type =
					NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
				default: checkNoEntry();
				}

				BuilderEncoders.Emplace(Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.EncodingSize, LinearLayerSettings));
			}

			OutElement = Builder.MakeAggregateOrExclusive(Parameters.EncodingSize, BuilderSubLayers, BuilderEncoders);
			break;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderSubLayers;
			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderQueryLayers;
			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderKeyLayers;
			TArray<NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderValueLayers;
			BuilderSubLayers.Reserve(Parameters.Elements.Num());
			BuilderQueryLayers.Reserve(Parameters.Elements.Num());
			BuilderValueLayers.Reserve(Parameters.Elements.Num());
			for (const FSchemaElement SubElement : Parameters.Elements)
			{
				const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(SubElement);
				NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
				MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
				BuilderSubLayers.Emplace(BuilderSubElement);

				NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
				LinearLayerSettings.Type = NetworkSettings.bUseCompressedLinearLayers ?
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
					NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

				switch (NetworkSettings.WeightInitialization)
				{
				case EWeightInitialization::KaimingGaussian: LinearLayerSettings.WeightInitializationSettings.Type =
					NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
				case EWeightInitialization::KaimingUniform: LinearLayerSettings.WeightInitializationSettings.Type =
					NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
				default: checkNoEntry();
				}

				BuilderQueryLayers.Emplace(Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.AttentionEncodingSize, LinearLayerSettings));
				BuilderKeyLayers.Emplace(Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.AttentionEncodingSize, LinearLayerSettings));
				BuilderValueLayers.Emplace(Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.ValueEncodingSize, LinearLayerSettings));
			}

			OutElement = Builder.MakeAggregateOrInclusive(
				Parameters.ValueEncodingSize,
				Parameters.AttentionEncodingSize,
				Parameters.AttentionHeadNum,
				BuilderSubLayers,
				BuilderQueryLayers,
				BuilderKeyLayers,
				BuilderValueLayers);

			break;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);
			OutElement = Builder.MakeArray(Parameters.Num, BuilderSubElement);
			break;
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);

			const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(Parameters.Element);

			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);

			NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
			LinearLayerSettings.Type = NetworkSettings.bUseCompressedLinearLayers ?
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

			switch (NetworkSettings.WeightInitialization)
			{
			case EWeightInitialization::KaimingGaussian: LinearLayerSettings.WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
			case EWeightInitialization::KaimingUniform: LinearLayerSettings.WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
			default: checkNoEntry();
			}

			OutElement = Builder.MakeAggregateSet(
				Parameters.MaxNum,
				Parameters.ValueEncodingSize,
				Parameters.AttentionEncodingSize,
				Parameters.AttentionHeadNum,
				BuilderSubElement,
				Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.AttentionEncodingSize, LinearLayerSettings),
				Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.AttentionEncodingSize, LinearLayerSettings),
				Builder.MakeLinearLayer(SubElementEncodedSize, Parameters.AttentionHeadNum * Parameters.ValueEncodingSize, LinearLayerSettings));

			break;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);

			const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(Parameters.Element);

			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);

			NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings LinearLayerSettings;
			LinearLayerSettings.Type = NetworkSettings.bUseCompressedLinearLayers ?
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Compressed :
				NNE::RuntimeBasic::FModelBuilder::ELinearLayerType::Normal;

			switch (NetworkSettings.WeightInitialization)
			{
			case EWeightInitialization::KaimingGaussian: LinearLayerSettings.WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
			case EWeightInitialization::KaimingUniform: LinearLayerSettings.WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
			default: checkNoEntry();
			}

			OutElement = Builder.MakeSequence({
				BuilderSubElement,
				Builder.MakeMLP(
					SubElementEncodedSize,
					Parameters.EncodingSize,
					Parameters.EncodingSize,
					Parameters.LayerNum + 1, // Add 1 to account for input layer
					Private::GetNNEActivationFunction(Parameters.ActivationFunction),
					true,
					LinearLayerSettings)
				});

			break;
		}

		case EType::Conv1d:
		{
			const FSchemaConv1dParameters Parameters = Schema.GetConv1d(SchemaElement);
			const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(Parameters.Element);
			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);
			
			NNE::RuntimeBasic::FModelBuilder::FWeightInitializationSettings WeightInitializationSettings;
			switch (NetworkSettings.WeightInitialization)
			{
			case EWeightInitialization::KaimingGaussian: WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
			case EWeightInitialization::KaimingUniform: WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
			default: checkNoEntry();
			}

			NNE::RuntimeBasic::FModelBuilderElement Conv1dLayer = Builder.MakeConv1d(
				Parameters.InputLength,
				Parameters.InChannels,
				Parameters.OutChannels,
				Parameters.KernelSize,
				Parameters.Padding,
				Private::GetNNEPaddingMode(Parameters.PaddingMode),
				Builder.MakeInitialWeights(Parameters.InChannels * Parameters.KernelSize, Parameters.OutChannels, WeightInitializationSettings),
				Builder.MakeInitialBiases(Parameters.OutChannels, WeightInitializationSettings),
				BuilderSubElement);
			
			OutElement = Builder.MakeSequence({
				Conv1dLayer,
				Builder.MakeActivation(Conv1dLayer.GetOutputSize(), Private::GetNNEActivationFunction(Parameters.ActivationFunction))
			});

			break;
		}

		case EType::Conv2d:
		{
			const FSchemaConv2dParameters Parameters = Schema.GetConv2d(SchemaElement);
			const int32 SubElementEncodedSize = Schema.GetEncodedVectorSize(Parameters.Element);
			NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeEncoderNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Parameters.Element, NetworkSettings);

			NNE::RuntimeBasic::FModelBuilder::FWeightInitializationSettings WeightInitializationSettings;
			switch (NetworkSettings.WeightInitialization)
			{
			case EWeightInitialization::KaimingGaussian: WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingGaussian; break;
			case EWeightInitialization::KaimingUniform: WeightInitializationSettings.Type =
				NNE::RuntimeBasic::FModelBuilder::EWeightInitializationType::KaimingUniform; break;
			default: checkNoEntry();
			}

			NNE::RuntimeBasic::FModelBuilderElement Conv2dLayer = Builder.MakeConv2d(
				Parameters.InputHeight,
				Parameters.InputWidth,
				Parameters.InChannels,
				Parameters.OutChannels,
				Parameters.KernelSize,
				Parameters.Stride,
				Parameters.Padding,
				Private::GetNNEPaddingMode(Parameters.PaddingMode),
				Builder.MakeInitialWeights(Parameters.InChannels * Parameters.KernelSize * Parameters.KernelSize, Parameters.OutChannels, WeightInitializationSettings),
				Builder.MakeInitialBiases(Parameters.OutChannels, WeightInitializationSettings),
				BuilderSubElement);

			OutElement = Builder.MakeSequence({
				Conv2dLayer,
				Builder.MakeActivation(Conv2dLayer.GetOutputSize(), Private::GetNNEActivationFunction(Parameters.ActivationFunction))
			});

			break;
		}

		case EType::Sparse:
		{
			const int32 ValueNum = Schema.GetSparse(SchemaElement).Num;

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		case EType::NamedSparse:
		{
			const int32 ValueNum = Schema.GetNamedSparse(SchemaElement).ElementNames.Num();

			OutElement = Builder.MakeDenormalize(
				ValueNum,
				Builder.MakeValuesZero(ValueNum),
				Builder.MakeValuesOne(ValueNum));
			break;
		}

		default:
		{
			checkNoEntry();
		}
		}

		checkf(OutElement.GetInputSize() == Schema.GetObservationVectorSize(SchemaElement),
			TEXT("Encoder Network Input unexpected size for %s. Got %i, expected %i according to Schema."),
			*Schema.GetTag(SchemaElement).ToString(), OutElement.GetInputSize(), Schema.GetObservationVectorSize(SchemaElement));

		checkf(OutElement.GetOutputSize() == Schema.GetEncodedVectorSize(SchemaElement),
			TEXT("Encoder Network Output unexpected size for %s. Got %i, expected %i according to Schema."),
			*Schema.GetTag(SchemaElement).ToString(), OutElement.GetOutputSize(), Schema.GetEncodedVectorSize(SchemaElement));
	}

	void GenerateEncoderNetworkFileDataFromSchema(
		TArray<uint8>& OutFileData,
		uint32& OutInputSize,
		uint32& OutOutputSize,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings,
		const uint32 Seed)
	{
		check(Schema.IsValid(SchemaElement));

		NNE::RuntimeBasic::FModelBuilder Builder(Seed);
		NNE::RuntimeBasic::FModelBuilderElement Element;
		MakeEncoderNetworkModelBuilderElementFromSchema(Element, Builder, Schema, SchemaElement, NetworkSettings);
		Builder.WriteFileDataAndReset(OutFileData, OutInputSize, OutOutputSize, Element);
	}

	void SetVectorFromObject(
		TLearningArrayView<1, float> OutObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FObject& Object,
		const FObjectElement ObjectElement)
	{
		check(Schema.IsValid(SchemaElement));
		check(Object.IsValid(ObjectElement));
		check(OutObservationVector.Num() == Schema.GetObservationVectorSize(SchemaElement));

		// Check that the types match

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const EType ObjectElementType = Object.GetType(ObjectElement);
		check(ObjectElementType == SchemaElementType);

		// Zero Observation Vector

		Array::Zero(OutObservationVector);

		// Logic for each specific element type

		switch (SchemaElementType)
		{

		case EType::Null: return;

		case EType::Continuous:
		{
			// Check the input sizes match

			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);
			const FObjectContinuousParameters ObjectParameters = Object.GetContinuous(ObjectElement);

			const TArrayView<const float> ObservationValues = ObjectParameters.Values;
			check(Schema.GetObservationVectorSize(SchemaElement) == ObservationValues.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == SchemaParameters.Num);

			// Copy in and scale the values from the observation object

			const int32 ValueNum = SchemaParameters.Num;
			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				float Value = ObservationValues[ValueIdx];

				if (SchemaParameters.NormalizationOption == ENormalization::Manual)
				{
					Value -= SchemaParameters.Offsets[ValueIdx];
					const float StdDev = FMath::Max(SchemaParameters.Scales[ValueIdx], UE_SMALL_NUMBER);
					Value /= StdDev;
				}
				
				OutObservationVector[ValueIdx] = Value;
			}
			return;
		}

		case EType::DiscreteExclusive:
		{
			const int32 ObservationValue = Object.GetDiscreteExclusive(ObjectElement).DiscreteIndex;
			check(Schema.GetObservationVectorSize(SchemaElement) > ObservationValue && ObservationValue >= 0);
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set the single value in the observation vector

			OutObservationVector[ObservationValue] = 1.0f;
			return;
		}

		case EType::DiscreteInclusive:
		{
			const TArrayView<const int32> ObservationValues = Object.GetDiscreteInclusive(ObjectElement).DiscreteIndices;
			check(Schema.GetObservationVectorSize(SchemaElement) >= ObservationValues.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set values in the observation vector

			for (int32 ObservationValueIdx = 0; ObservationValueIdx < ObservationValues.Num(); ObservationValueIdx++)
			{
				check(Schema.GetObservationVectorSize(SchemaElement) > ObservationValues[ObservationValueIdx] && ObservationValues[ObservationValueIdx] >= 0);
				OutObservationVector[ObservationValues[ObservationValueIdx]] = 1.0f;
			}

			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			const FName ObservationValue = Object.GetNamedDiscreteExclusive(ObjectElement).ElementName;
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set the single value in the observation vector
			const int32 ObservationIndex = SchemaNames.Find(ObservationValue);
			check(ObservationIndex != INDEX_NONE);
			OutObservationVector[ObservationIndex] = 1.0f;
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			const TArrayView<const FName> ObservationValues = Object.GetNamedDiscreteInclusive(ObjectElement).ElementNames;
			check(Schema.GetObservationVectorSize(SchemaElement) >= ObservationValues.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set values in the observation vector
			for (int32 ObservationValueIdx = 0; ObservationValueIdx < ObservationValues.Num(); ObservationValueIdx++)
			{
				const int32 ObservationIndex = SchemaNames.Find(ObservationValues[ObservationValueIdx]);
				check(ObservationIndex != INDEX_NONE);
				OutObservationVector[ObservationIndex] = 1.0f;
			}
			return;
		}

		case EType::And:
		{
			// Check the number of sub-elements match

			const FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const FObjectAndParameters ObjectParameters = Object.GetAnd(ObjectElement);
			check(SchemaParameters.Elements.Num() == ObjectParameters.Elements.Num());

			// Update Sub-elements

			int32 SubElementOffset = 0;

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 ObjectElementIndex = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);
				check(ObjectElementIndex != INDEX_NONE);

				const int32 SubElementSize = Schema.GetObservationVectorSize(SchemaParameters.Elements[SchemaElementIdx]);

				SetVectorFromObject(
					OutObservationVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIndex]);

				SubElementOffset += SubElementSize;
			}

			check(SubElementOffset == OutObservationVector.Num());
			return;
		}

		case EType::OrExclusive:
		{
			// Check only one sub-element is given and index is valid

			const FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const FObjectOrExclusiveParameters ObjectParameters = Object.GetOrExclusive(ObjectElement);

			const int32 SchemaElementIndex = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);
			check(SchemaElementIndex != INDEX_NONE);

			// Update sub-element

			const int32 SubElementSize = Schema.GetObservationVectorSize(SchemaParameters.Elements[SchemaElementIndex]);

			SetVectorFromObject(
				OutObservationVector.Slice(0, SubElementSize),
				Schema,
				SchemaParameters.Elements[SchemaElementIndex],
				Object,
				ObjectParameters.Element);

			// Set Mask

			const int32 MaxSubElementSize = Private::GetMaxObservationVectorSize(Schema, SchemaParameters.Elements);

			OutObservationVector[MaxSubElementSize + SchemaElementIndex] = 1.0f;

			check(OutObservationVector.Num() == MaxSubElementSize + SchemaParameters.Elements.Num());
			return;
		}

		case EType::OrInclusive:
		{
			// Check all indices are in range

			const FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const FObjectOrInclusiveParameters ObjectParameters = Object.GetOrInclusive(ObjectElement);
			check(ObjectParameters.Elements.Num() <= SchemaParameters.Elements.Num());

			// Update sub-elements

			int32 SubElementOffset = 0;

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(SchemaParameters.Elements[SchemaElementIdx]);
				const int32 ObjectElementIdx = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);

				if (ObjectElementIdx != INDEX_NONE)
				{
					SetVectorFromObject(
						OutObservationVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						SchemaParameters.Elements[SchemaElementIdx],
						Object,
						ObjectParameters.Elements[ObjectElementIdx]);
				}

				SubElementOffset += SubElementSize;
			}

			// Set Mask

			check(SubElementOffset + SchemaParameters.Elements.Num() == OutObservationVector.Num());

			for (int32 ObjectElementIdx = 0; ObjectElementIdx < ObjectParameters.Elements.Num(); ObjectElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectElementIdx]);
				check(SchemaElementIdx != INDEX_NONE);

				OutObservationVector[SubElementOffset + SchemaElementIdx] = 1.0f;
			}

			return;
		}

		case EType::Array:
		{
			// Check number of array elements is correct

			const FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const FObjectArrayParameters ObjectParameters = Object.GetArray(ObjectElement);
			check(SchemaParameters.Num == ObjectParameters.Elements.Num());

			// Update sub-elements

			const int32 SubElementSize = Schema.GetObservationVectorSize(SchemaParameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < SchemaParameters.Num; ElementIdx++)
			{
				SetVectorFromObject(
					OutObservationVector.Slice(ElementIdx * SubElementSize, SubElementSize),
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]);
			}

			return;
		}

		case EType::Set:
		{
			// Check number of set elements is correct

			const FSchemaSetParameters SchemaParameters = Schema.GetSet(SchemaElement);
			const FObjectSetParameters ObjectParameters = Object.GetSet(ObjectElement);
			check(SchemaParameters.MaxNum >= ObjectParameters.Elements.Num());

			// Update sub-elements

			int32 SubElementOffset = 0;

			const int32 SubElementSize = Schema.GetObservationVectorSize(SchemaParameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				SetVectorFromObject(
					OutObservationVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]);

				SubElementOffset += SubElementSize;
			}

			SubElementOffset = SubElementSize * SchemaParameters.MaxNum;

			// Set Mask

			Array::Set(OutObservationVector.Slice(SubElementOffset, ObjectParameters.Elements.Num()), 1.0f);

			check(SubElementOffset + SchemaParameters.MaxNum == OutObservationVector.Num());
			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const FObjectEncodingParameters ObjectParameters = Object.GetEncoding(ObjectElement);

			SetVectorFromObject(
				OutObservationVector,
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);

			return;
		}

		case EType::Conv1d:
		{
			const FSchemaConv1dParameters SchemaParameters = Schema.GetConv1d(SchemaElement);
			const FObjectConv1dParameters ObjectParameters = Object.GetConv1d(ObjectElement);

			SetVectorFromObject(
				OutObservationVector,
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);

			return;
		}

		case EType::Conv2d:
		{
			const FSchemaConv2dParameters SchemaParameters = Schema.GetConv2d(SchemaElement);
			const FObjectConv2dParameters ObjectParameters = Object.GetConv2d(ObjectElement);

			SetVectorFromObject(
				OutObservationVector,
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);

			return;
		}


		case EType::Sparse:
		{
			const TArrayView<const int32> ObservationValues = Object.GetSparse(ObjectElement).DiscreteIndices;
			const TArrayView<const float> ObservationWeights = Object.GetSparse(ObjectElement).Values;
			check(ObservationValues.Num() == ObservationWeights.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) >= ObservationValues.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) >= ObservationWeights.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set values in the observation vector

			for (int32 ObservationValueIdx = 0; ObservationValueIdx < ObservationValues.Num(); ObservationValueIdx++)
			{
				check(Schema.GetObservationVectorSize(SchemaElement) > ObservationValues[ObservationValueIdx] && ObservationValues[ObservationValueIdx] >= 0);
				OutObservationVector[ObservationValues[ObservationValueIdx]] = ObservationWeights[ObservationValueIdx];
			}

			return;
		}

		case EType::NamedSparse:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedSparse(SchemaElement).ElementNames;
			const TArrayView<const FName> ObservationValues = Object.GetNamedSparse(ObjectElement).ElementNames;
			const TArrayView<const float> ObservationWeights = Object.GetNamedSparse(ObjectElement).Values;
			check(ObservationValues.Num() == ObservationWeights.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) >= ObservationValues.Num());
			check(Schema.GetObservationVectorSize(SchemaElement) == OutObservationVector.Num());

			// Set values in the observation vector
			for (int32 ObservationValueIdx = 0; ObservationValueIdx < ObservationValues.Num(); ObservationValueIdx++)
			{
				const int32 ObservationIndex = SchemaNames.Find(ObservationValues[ObservationValueIdx]);
				check(ObservationIndex != INDEX_NONE);
				OutObservationVector[ObservationIndex] = ObservationWeights[ObservationValueIdx];
			}
			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}

		}
	}

	void GetObjectFromVector(
		FObject& OutObject,
		FObjectElement& OutObjectElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ObservationVector)
	{
		check(Schema.IsValid(SchemaElement));

		// Check that the types match

		const EType SchemaElementType = Schema.GetType(SchemaElement);
		const FName SchemaElementTag = Schema.GetTag(SchemaElement);

		// Get Observation Vector Size

		const int32 ObservationVectorSize = ObservationVector.Num();
		check(ObservationVectorSize == Schema.GetObservationVectorSize(SchemaElement));

		// Logic for each specific element type

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			OutObjectElement = OutObject.CreateNull(SchemaElementTag);
			return;
		}

		case EType::Continuous:
		{
			const FSchemaContinuousParameters SchemaParameters = Schema.GetContinuous(SchemaElement);
			check(ObservationVectorSize == SchemaParameters.Num);

			const int32 ValueNum = SchemaParameters.Num;

			TLearningArray<1, float, TInlineAllocator<32>> ObservationValues;
			ObservationValues.SetNumUninitialized({ ValueNum });

			for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				float Value = ObservationVector[ValueIdx];

				if (SchemaParameters.NormalizationOption == ENormalization::Manual)
				{
					const float StdDev = FMath::Max(SchemaParameters.Scales[ValueIdx], UE_SMALL_NUMBER);
					Value *= StdDev;
					Value += SchemaParameters.Offsets[ValueIdx];
				}
				ObservationValues[ValueIdx] = Value;
			}

			OutObjectElement = OutObject.CreateContinuous({ MakeArrayView(ObservationValues.GetData(), ObservationValues.Num()) }, SchemaElementTag);
			return;
		}

		case EType::DiscreteExclusive:
		{
			check(ObservationVectorSize == Schema.GetDiscreteExclusive(SchemaElement).Num);

			// Find Index
			int32 ExclusiveIndex = INDEX_NONE;
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				check(ObservationVector[Idx] == 0.0f || ObservationVector[Idx] == 1.0f);
				if (ObservationVector[Idx])
				{
					ExclusiveIndex = Idx;
					break;
				}
			}
			check(ExclusiveIndex != INDEX_NONE);

			OutObjectElement = OutObject.CreateDiscreteExclusive({ ExclusiveIndex }, SchemaElementTag);
			return;
		}

		case EType::DiscreteInclusive:
		{
			check(ObservationVectorSize == Schema.GetDiscreteInclusive(SchemaElement).Num);

			// Find Indices
			TArray<int32, TInlineAllocator<8>> InclusiveIndices;
			InclusiveIndices.Reserve(ObservationVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				check(ObservationVector[Idx] == 0.0f || ObservationVector[Idx] == 1.0f);
				if (ObservationVector[Idx])
				{
					InclusiveIndices.Add(Idx);
				}
			}

			OutObjectElement = OutObject.CreateDiscreteInclusive({ InclusiveIndices }, SchemaElementTag);
			return;
		}


		case EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			check(ObservationVectorSize == Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num());

			// Find Name
			FName ExclusiveName = NAME_None;
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				check(ObservationVector[Idx] == 0.0f || ObservationVector[Idx] == 1.0f);
				if (ObservationVector[Idx])
				{
					ExclusiveName = SchemaNames[Idx];
					break;
				}
			}
			check(ExclusiveName != NAME_None);

			OutObjectElement = OutObject.CreateNamedDiscreteExclusive({ ExclusiveName }, SchemaElementTag);
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			check(ObservationVectorSize == Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num());

			// Find Names
			TArray<FName, TInlineAllocator<8>> InclusiveNames;
			InclusiveNames.Reserve(ObservationVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				check(ObservationVector[Idx] == 0.0f || ObservationVector[Idx] == 1.0f);
				if (ObservationVector[Idx])
				{
					InclusiveNames.Add(SchemaNames[Idx]);
				}
			}

			OutObjectElement = OutObject.CreateNamedDiscreteInclusive({ InclusiveNames }, SchemaElementTag);
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			// Create Sub-elements

			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Elements.Num());

			int32 SubElementOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetObjectFromVector(
					OutObject,
					SubElements[SchemaElementIdx],
					Schema,
					Parameters.Elements[SchemaElementIdx],
					ObservationVector.Slice(SubElementOffset, SubElementSize));

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ObservationVectorSize);

			OutObjectElement = OutObject.CreateAnd({ Parameters.ElementNames, SubElements }, SchemaElementTag);
			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

			// Find active element

			const int32 MaxSubElementSize = Private::GetMaxObservationVectorSize(Schema, Parameters.Elements);

			int32 SchemaElementIndex = INDEX_NONE;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				check(ObservationVector[MaxSubElementSize + SubElementIdx] == 0.0f || ObservationVector[MaxSubElementSize + SubElementIdx] == 1.0f);
				if (ObservationVector[MaxSubElementSize + SubElementIdx])
				{
					SchemaElementIndex = SubElementIdx;
					break;
				}
			}
			check(SchemaElementIndex != INDEX_NONE);

			// Create sub-element

			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIndex]);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Elements[SchemaElementIndex],
				ObservationVector.Slice(0, SubElementSize));

			OutObjectElement = OutObject.CreateOrExclusive({ Parameters.ElementNames[SchemaElementIndex], SubElement }, SchemaElementTag);
			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

			// Find total sub-element size

			const int32 TotalSubElementSize = Private::GetTotalObservationVectorSize(Schema, Parameters.Elements);

			// Create sub-elements

			TArray<FName, TInlineAllocator<8>> SubElementNames;
			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElementNames.Reserve(Parameters.Elements.Num());
			SubElements.Reserve(Parameters.Elements.Num());

			int32 SubElementOffset = 0;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SubElementIdx]);

				check(
					ObservationVector[TotalSubElementSize + SubElementIdx] == 0.0f || 
					ObservationVector[TotalSubElementSize + SubElementIdx] == 1.0f);

				if (ObservationVector[TotalSubElementSize + SubElementIdx] == 1.0f)
				{
					FObjectElement SubElement;
					GetObjectFromVector(
						OutObject,
						SubElement,
						Schema,
						Parameters.Elements[SubElementIdx],
						ObservationVector.Slice(SubElementOffset, SubElementSize));

					SubElementNames.Add(Parameters.ElementNames[SubElementIdx]);
					SubElements.Add(SubElement);
				}

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset + Parameters.Elements.Num() == ObservationVectorSize);

			OutObjectElement = OutObject.CreateOrInclusive({ SubElementNames, SubElements }, SchemaElementTag);
			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);

			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElements.SetNumUninitialized(Parameters.Num);

			// Create sub-elements

			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				GetObjectFromVector(
					OutObject,
					SubElements[ElementIdx],
					Schema,
					Parameters.Element,
					ObservationVector.Slice(ElementIdx * SubElementSize, SubElementSize));
			}

			OutObjectElement = OutObject.CreateArray({ SubElements }, SchemaElementTag);
			return;
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);

			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			// Create sub-elements

			TArray<FObjectElement, TInlineAllocator<8>> SubElements;
			SubElements.Reserve(Parameters.MaxNum);

			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.MaxNum; SubElementIdx++)
			{
				check(
					ObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f || 
					ObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 1.0f);

				if (ObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f)
				{
					break;
				}

				FObjectElement SubElement;

				GetObjectFromVector(
					OutObject,
					SubElement,
					Schema,
					Parameters.Element,
					ObservationVector.Slice(SubElementIdx * SubElementSize, SubElementSize));

				SubElements.Add(SubElement);
			}

			OutObjectElement = OutObject.CreateSet({ SubElements }, SchemaElementTag);
			return;
		}

		case EType::Encoding:
		{
			const FSchemaEncodingParameters Parameters = Schema.GetEncoding(SchemaElement);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Element,
				ObservationVector);

			OutObjectElement = OutObject.CreateEncoding({ SubElement }, SchemaElementTag);
			return;
		}

		case EType::Conv1d:
		{
			const FSchemaConv1dParameters Parameters = Schema.GetConv1d(SchemaElement);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Element,
				ObservationVector);

			OutObjectElement = OutObject.CreateConv1d({ SubElement }, SchemaElementTag);
			return;
		}

		case EType::Conv2d:
		{
			const FSchemaConv2dParameters Parameters = Schema.GetConv2d(SchemaElement);

			FObjectElement SubElement;
			GetObjectFromVector(
				OutObject,
				SubElement,
				Schema,
				Parameters.Element,
				ObservationVector);

			OutObjectElement = OutObject.CreateConv2d({ SubElement }, SchemaElementTag);
			return;
		}


		case EType::Sparse:
		{
			check(ObservationVectorSize == Schema.GetSparse(SchemaElement).Num);

			// Find Indices
			TArray<int32, TInlineAllocator<8>> InclusiveIndices;
			TArray<float, TInlineAllocator<8>> InclusiveValues;
			InclusiveIndices.Reserve(ObservationVectorSize);
			InclusiveValues.Reserve(ObservationVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				if (ObservationVector[Idx] != 0.0f)
				{
					InclusiveIndices.Add(Idx);
					InclusiveValues.Add(ObservationVector[Idx]);
				}
			}

			OutObjectElement = OutObject.CreateSparse({ InclusiveIndices, InclusiveValues }, SchemaElementTag);
			return;
		}

		case EType::NamedSparse:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedSparse(SchemaElement).ElementNames;
			check(ObservationVectorSize == Schema.GetNamedSparse(SchemaElement).ElementNames.Num());

			// Find Names
			TArray<FName, TInlineAllocator<8>> InclusiveNames;
			TArray<float, TInlineAllocator<8>> InclusiveValues;
			InclusiveNames.Reserve(ObservationVectorSize);
			InclusiveValues.Reserve(ObservationVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				if (ObservationVector[Idx] != 0.0f)
				{
					InclusiveNames.Add(SchemaNames[Idx]);
					InclusiveValues.Add(ObservationVector[Idx]);
				}
			}

			OutObjectElement = OutObject.CreateNamedSparse({ InclusiveNames, InclusiveValues }, SchemaElementTag);
			return;
		}

		default:
		{
			checkNoEntry();
			OutObjectElement = FObjectElement();
			return;
		}
		}
	}

	void GetContinuousObservationVectorMask(
		TLearningArrayView<1, bool> OutObservationVectorMask,
		const TLearningArrayView<1, const float> InObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement)
	{
		check(Schema.IsValid(SchemaElement));

		const EType SchemaElementType = Schema.GetType(SchemaElement);

		const int32 ObservationVectorSize = OutObservationVectorMask.Num();
		check(ObservationVectorSize == Schema.GetObservationVectorSize(SchemaElement));

		Array::Zero(OutObservationVectorMask);

		switch (SchemaElementType)
		{

		case EType::Null:
		{
			return;
		}

		case EType::Continuous:
		{
			check(ObservationVectorSize == Schema.GetContinuous(SchemaElement).Num);
			Array::Set(OutObservationVectorMask, true);
			return;
		}

		case EType::DiscreteExclusive:
		{
			return;
		}

		case EType::DiscreteInclusive:
		{
			return;
		}

		case EType::NamedDiscreteExclusive:
		{
			return;
		}

		case EType::NamedDiscreteInclusive:
		{
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			int32 SubElementOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);

				GetContinuousObservationVectorMask(
					OutObservationVectorMask.Slice(SubElementOffset, SubElementSize),
					InObservationVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					Parameters.Elements[SchemaElementIdx]);

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset == ObservationVectorSize);

			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			const int32 MaxSubElementSize = Private::GetMaxObservationVectorSize(Schema, Parameters.Elements);

			int32 SchemaElementIndex = INDEX_NONE;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				check(InObservationVector[MaxSubElementSize + SubElementIdx] == 0.0f || InObservationVector[MaxSubElementSize + SubElementIdx] == 1.0f);
				if (InObservationVector[MaxSubElementSize + SubElementIdx])
				{
					SchemaElementIndex = SubElementIdx;
					break;
				}
			}
			check(SchemaElementIndex != INDEX_NONE);

			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIndex]);

			GetContinuousObservationVectorMask(
				OutObservationVectorMask.Slice(0, SubElementSize),
				InObservationVector.Slice(0, SubElementSize),
				Schema,
				Parameters.Elements[SchemaElementIndex]);

			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			const int32 TotalSubElementSize = Private::GetTotalObservationVectorSize(Schema, Parameters.Elements);

			int32 SubElementOffset = 0;
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SubElementIdx]);

				check(
					InObservationVector[TotalSubElementSize + SubElementIdx] == 0.0f ||
					InObservationVector[TotalSubElementSize + SubElementIdx] == 1.0f);

				if (InObservationVector[TotalSubElementSize + SubElementIdx] == 1.0f)
				{
					GetContinuousObservationVectorMask(
						OutObservationVectorMask.Slice(SubElementOffset, SubElementSize),
						InObservationVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						Parameters.Elements[SubElementIdx]);
				}

				SubElementOffset += SubElementSize;
			}
			check(SubElementOffset + Parameters.Elements.Num() == ObservationVectorSize);

			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				GetContinuousObservationVectorMask(
					OutObservationVectorMask.Slice(ElementIdx * SubElementSize, SubElementSize),
					InObservationVector.Slice(ElementIdx * SubElementSize, SubElementSize),
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.MaxNum; SubElementIdx++)
			{
				check(
					InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f ||
					InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 1.0f);

				if (InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f)
				{
					break;
				}

				GetContinuousObservationVectorMask(
					OutObservationVectorMask.Slice(SubElementIdx* SubElementSize, SubElementSize),
					InObservationVector.Slice(SubElementIdx* SubElementSize, SubElementSize),
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Encoding:
		{
			GetContinuousObservationVectorMask(
				OutObservationVectorMask,
				InObservationVector,
				Schema,
				Schema.GetEncoding(SchemaElement).Element);

			return;
		}

		case EType::Conv1d:
		{
			GetContinuousObservationVectorMask(
				OutObservationVectorMask,
				InObservationVector,
				Schema,
				Schema.GetConv1d(SchemaElement).Element);

			return;
		}

		case EType::Conv2d:
		{
			GetContinuousObservationVectorMask(
				OutObservationVectorMask,
				InObservationVector,
				Schema,
				Schema.GetConv2d(SchemaElement).Element);

			return;
		}

		case EType::Sparse:
		{
			check(ObservationVectorSize == Schema.GetSparse(SchemaElement).Num);
			Array::Set(OutObservationVectorMask, true);
			return;
		}

		case EType::NamedSparse:
		{
			check(ObservationVectorSize == Schema.GetNamedSparse(SchemaElement).ElementNames.Num());
			Array::Set(OutObservationVectorMask, true);
			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}
		}
	}

	void InitObservationDistributionMinMax(
		TLearningArrayView<1, float> OutObservationDistributionMin,
		TLearningArrayView<1, float> OutObservationDistributionMax)
	{
		Array::Set(OutObservationDistributionMin, +UE_MAX_FLT);
		Array::Set(OutObservationDistributionMax, -UE_MAX_FLT);
	}

	void FitObservationDistributionMinMax(
		TLearningArrayView<1, float> InOutObservationDistributionMin,
		TLearningArrayView<1, float> InOutObservationDistributionMax,
		const TLearningArrayView<1, const float> InObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement)
	{
		check(Schema.IsValid(SchemaElement));

		const EType SchemaElementType = Schema.GetType(SchemaElement);

		const int32 ObservationVectorSize = InObservationVector.Num();
		const int32 ObservationDistributionVectorSize = InOutObservationDistributionMin.Num();
		check(ObservationVectorSize == Schema.GetObservationVectorSize(SchemaElement));
		check(ObservationDistributionVectorSize == Schema.GetObservationDistributionVectorSize(SchemaElement));

		switch (SchemaElementType)
		{

		case EType::Null:
		case EType::DiscreteExclusive:
		case EType::DiscreteInclusive:
		case EType::NamedDiscreteExclusive:
		case EType::NamedDiscreteInclusive:
		{
			return;
		}

		case EType::Continuous:
		{
			check(ObservationVectorSize == Schema.GetContinuous(SchemaElement).Num);
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationDistributionMin[Idx] = FMath::Min(InOutObservationDistributionMin[Idx], InObservationVector[Idx]);
				InOutObservationDistributionMax[Idx] = FMath::Max(InOutObservationDistributionMax[Idx], InObservationVector[Idx]);
			}
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			int32 SubElementOffset = 0;
			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				FitObservationDistributionMinMax(
					InOutObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
					InOutObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
					InObservationVector.Slice(SubElementOffset, SubElementSize),
					Schema,
					Parameters.Elements[SchemaElementIdx]);

				SubElementOffset += SubElementSize;
				SubElementDistributionOffset += SubElementDistributionSize;
			}
			check(SubElementOffset == ObservationVectorSize);
			check(SubElementDistributionOffset == ObservationDistributionVectorSize);

			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			const int32 MaxSubElementSize = Private::GetMaxObservationVectorSize(Schema, Parameters.Elements);

			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				check(InObservationVector[MaxSubElementSize + SchemaElementIdx] == 0.0f || InObservationVector[MaxSubElementSize + SchemaElementIdx] == 1.0f);
				if (InObservationVector[MaxSubElementSize + SchemaElementIdx])
				{
					const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);

					FitObservationDistributionMinMax(
						InOutObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						InOutObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						InObservationVector.Slice(0, SubElementSize),
						Schema,
						Parameters.Elements[SchemaElementIdx]);

					break;
				}

				SubElementDistributionOffset += SubElementDistributionSize;
			}

			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			const int32 TotalSubElementSize = Private::GetTotalObservationVectorSize(Schema, Parameters.Elements);

			int32 SubElementOffset = 0;
			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				check(
					InObservationVector[TotalSubElementSize + SchemaElementIdx] == 0.0f ||
					InObservationVector[TotalSubElementSize + SchemaElementIdx] == 1.0f);

				if (InObservationVector[TotalSubElementSize + SchemaElementIdx] == 1.0f)
				{
					FitObservationDistributionMinMax(
						InOutObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						InOutObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						InObservationVector.Slice(SubElementOffset, SubElementSize),
						Schema,
						Parameters.Elements[SchemaElementIdx]);
				}

				SubElementOffset += SubElementSize;
				SubElementDistributionOffset += SubElementDistributionSize;
			}
			check(SubElementOffset + Parameters.Elements.Num() == ObservationVectorSize);
			check(SubElementDistributionOffset + Parameters.Elements.Num() == ObservationDistributionVectorSize);

			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);
			const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				FitObservationDistributionMinMax(
					InOutObservationDistributionMin.Slice(ElementIdx * SubElementDistributionSize, SubElementDistributionSize),
					InOutObservationDistributionMax.Slice(ElementIdx * SubElementDistributionSize, SubElementDistributionSize),
					InObservationVector.Slice(ElementIdx * SubElementSize, SubElementSize),
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.MaxNum; SubElementIdx++)
			{
				check(
					InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f ||
					InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 1.0f);

				if (InObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f)
				{
					break;
				}

				FitObservationDistributionMinMax(
					InOutObservationDistributionMin,
					InOutObservationDistributionMax,
					InObservationVector.Slice(SubElementIdx * SubElementSize, SubElementSize),
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Encoding:
		{
			FitObservationDistributionMinMax(
				InOutObservationDistributionMin,
				InOutObservationDistributionMax,
				InObservationVector,
				Schema,
				Schema.GetEncoding(SchemaElement).Element);

			return;
		}

		case EType::Conv1d:
		{
			FitObservationDistributionMinMax(
				InOutObservationDistributionMin,
				InOutObservationDistributionMax,
				InObservationVector,
				Schema,
				Schema.GetConv1d(SchemaElement).Element);

			return;
		}

		case EType::Conv2d:
		{
			FitObservationDistributionMinMax(
				InOutObservationDistributionMin,
				InOutObservationDistributionMax,
				InObservationVector,
				Schema,
				Schema.GetConv2d(SchemaElement).Element);

			return;
		}

		case EType::Sparse:
		{
			check(ObservationVectorSize == Schema.GetSparse(SchemaElement).Num);
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationDistributionMin[Idx] = FMath::Min(InOutObservationDistributionMin[Idx], InObservationVector[Idx]);
				InOutObservationDistributionMax[Idx] = FMath::Max(InOutObservationDistributionMax[Idx], InObservationVector[Idx]);
			}
			return;
		}

		case EType::NamedSparse:
		{
			check(ObservationVectorSize == Schema.GetNamedSparse(SchemaElement).ElementNames.Num());
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationDistributionMin[Idx] = FMath::Min(InOutObservationDistributionMin[Idx], InObservationVector[Idx]);
				InOutObservationDistributionMax[Idx] = FMath::Max(InOutObservationDistributionMax[Idx], InObservationVector[Idx]);
			}
			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}
		}
	}

	void ClampToObservationDistributionMinMax(
		TLearningArrayView<1, float> InOutObservationVector,
		const TLearningArrayView<1, const float> ObservationDistributionMin,
		const TLearningArrayView<1, const float> ObservationDistributionMax,
		const FSchema& Schema,
		const FSchemaElement SchemaElement)
	{
		check(Schema.IsValid(SchemaElement));

		const EType SchemaElementType = Schema.GetType(SchemaElement);

		const int32 ObservationVectorSize = InOutObservationVector.Num();
		const int32 ObservationDistributionVectorSize = ObservationDistributionMin.Num();
		check(ObservationVectorSize == Schema.GetObservationVectorSize(SchemaElement));
		check(ObservationDistributionVectorSize == Schema.GetObservationDistributionVectorSize(SchemaElement));

		switch (SchemaElementType)
		{

		case EType::Null:
		case EType::DiscreteExclusive:
		case EType::DiscreteInclusive:
		case EType::NamedDiscreteExclusive:
		case EType::NamedDiscreteInclusive:
		{
			return;
		}

		case EType::Continuous:
		{
			check(ObservationVectorSize == Schema.GetContinuous(SchemaElement).Num);
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationVector[Idx] = FMath::Clamp(
					InOutObservationVector[Idx], 
					ObservationDistributionMin[Idx], 
					ObservationDistributionMax[Idx]);
			}
			return;
		}

		case EType::And:
		{
			const FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

			int32 SubElementOffset = 0;
			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				ClampToObservationDistributionMinMax(
					InOutObservationVector.Slice(SubElementOffset, SubElementSize),
					ObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
					ObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
					Schema,
					Parameters.Elements[SchemaElementIdx]);

				SubElementOffset += SubElementSize;
				SubElementDistributionOffset += SubElementDistributionSize;
			}
			check(SubElementOffset == ObservationVectorSize);
			check(SubElementDistributionOffset == ObservationDistributionVectorSize);

			return;
		}

		case EType::OrExclusive:
		{
			const FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);
			const int32 MaxSubElementSize = Private::GetMaxObservationVectorSize(Schema, Parameters.Elements);

			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				check(InOutObservationVector[MaxSubElementSize + SchemaElementIdx] == 0.0f || InOutObservationVector[MaxSubElementSize + SchemaElementIdx] == 1.0f);
				if (InOutObservationVector[MaxSubElementSize + SchemaElementIdx])
				{
					const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);

					ClampToObservationDistributionMinMax(
						InOutObservationVector.Slice(0, SubElementSize),
						ObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						ObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						Schema,
						Parameters.Elements[SchemaElementIdx]);

					break;
				}

				SubElementDistributionOffset += SubElementDistributionSize;
			}

			return;
		}

		case EType::OrInclusive:
		{
			const FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);
			const int32 TotalSubElementSize = Private::GetTotalObservationVectorSize(Schema, Parameters.Elements);

			int32 SubElementOffset = 0;
			int32 SubElementDistributionOffset = 0;
			for (int32 SchemaElementIdx = 0; SchemaElementIdx < Parameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Elements[SchemaElementIdx]);
				const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Elements[SchemaElementIdx]);

				check(
					InOutObservationVector[TotalSubElementSize + SchemaElementIdx] == 0.0f ||
					InOutObservationVector[TotalSubElementSize + SchemaElementIdx] == 1.0f);

				if (InOutObservationVector[TotalSubElementSize + SchemaElementIdx] == 1.0f)
				{
					ClampToObservationDistributionMinMax(
						InOutObservationVector.Slice(SubElementOffset, SubElementSize),
						ObservationDistributionMin.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						ObservationDistributionMax.Slice(SubElementDistributionOffset, SubElementDistributionSize),
						Schema,
						Parameters.Elements[SchemaElementIdx]);
				}

				SubElementOffset += SubElementSize;
				SubElementDistributionOffset += SubElementDistributionSize;
			}
			check(SubElementOffset + Parameters.Elements.Num() == ObservationVectorSize);
			check(SubElementDistributionOffset + Parameters.Elements.Num() == ObservationDistributionVectorSize);

			return;
		}

		case EType::Array:
		{
			const FSchemaArrayParameters Parameters = Schema.GetArray(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);
			const int32 SubElementDistributionSize = Schema.GetObservationDistributionVectorSize(Parameters.Element);

			for (int32 ElementIdx = 0; ElementIdx < Parameters.Num; ElementIdx++)
			{
				ClampToObservationDistributionMinMax(
					InOutObservationVector.Slice(ElementIdx * SubElementSize, SubElementSize),
					ObservationDistributionMin.Slice(ElementIdx * SubElementDistributionSize, SubElementDistributionSize),
					ObservationDistributionMax.Slice(ElementIdx * SubElementDistributionSize, SubElementDistributionSize),
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Set:
		{
			const FSchemaSetParameters Parameters = Schema.GetSet(SchemaElement);
			const int32 SubElementSize = Schema.GetObservationVectorSize(Parameters.Element);

			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.MaxNum; SubElementIdx++)
			{
				check(
					InOutObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f ||
					InOutObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 1.0f);

				if (InOutObservationVector[SubElementSize * Parameters.MaxNum + SubElementIdx] == 0.0f)
				{
					break;
				}

				ClampToObservationDistributionMinMax(
					InOutObservationVector.Slice(SubElementIdx * SubElementSize, SubElementSize),
					ObservationDistributionMin,
					ObservationDistributionMax,
					Schema,
					Parameters.Element);
			}

			return;
		}

		case EType::Encoding:
		{
			ClampToObservationDistributionMinMax(
				InOutObservationVector,
				ObservationDistributionMin,
				ObservationDistributionMax,
				Schema,
				Schema.GetEncoding(SchemaElement).Element);

			return;
		}

		case EType::Conv1d:
		{
			ClampToObservationDistributionMinMax(
				InOutObservationVector,
				ObservationDistributionMin,
				ObservationDistributionMax,
				Schema,
				Schema.GetConv1d(SchemaElement).Element);

			return;
		}

		case EType::Conv2d:
		{
			ClampToObservationDistributionMinMax(
				InOutObservationVector,
				ObservationDistributionMin,
				ObservationDistributionMax,
				Schema,
				Schema.GetConv2d(SchemaElement).Element);

			return;
		}

		case EType::Sparse:
		{
			check(ObservationVectorSize == Schema.GetSparse(SchemaElement).Num);
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationVector[Idx] = FMath::Clamp(
					InOutObservationVector[Idx],
					ObservationDistributionMin[Idx],
					ObservationDistributionMax[Idx]);
			}
			return;
		}

		case EType::NamedSparse:
		{
			check(ObservationVectorSize == Schema.GetNamedSparse(SchemaElement).ElementNames.Num());
			check(ObservationVectorSize == ObservationDistributionVectorSize);
			for (int32 Idx = 0; Idx < ObservationVectorSize; Idx++)
			{
				InOutObservationVector[Idx] = FMath::Clamp(
					InOutObservationVector[Idx],
					ObservationDistributionMin[Idx],
					ObservationDistributionMax[Idx]);
			}
			return;
		}

		default:
		{
			checkNoEntry();
			return;
		}
		}
	}

}

FArchive& operator<<(FArchive& Ar, UE::Learning::Observation::FSchemaElement& SchemaElement)
{
	Ar << SchemaElement.Generation;
	Ar << SchemaElement.Index;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::Learning::Observation::FObjectElement& ObjectElement)
{
	Ar << ObjectElement.Generation;
	Ar << ObjectElement.Index;
	return Ar;
}
bool operator==(const UE::Learning::Observation::FSchemaElement& Lhs, const UE::Learning::Observation::FSchemaElement& Rhs)
{
	return Lhs.Generation == Rhs.Generation && Lhs.Index == Rhs.Index;
}

bool operator==(const UE::Learning::Observation::FObjectElement& Lhs, const UE::Learning::Observation::FObjectElement& Rhs)
{
	return Lhs.Generation == Rhs.Generation && Lhs.Index == Rhs.Index;
}

uint32 GetTypeHash(const UE::Learning::Observation::FSchemaElement& SchemaElement)
{
	return GetTypeHash(TPair<int32, int32>{ SchemaElement.Generation, SchemaElement.Index });
}

uint32 GetTypeHash(const UE::Learning::Observation::FObjectElement& ObjectElement)
{
	return GetTypeHash(TPair<int32, int32>{ ObjectElement.Generation, ObjectElement.Index });
}