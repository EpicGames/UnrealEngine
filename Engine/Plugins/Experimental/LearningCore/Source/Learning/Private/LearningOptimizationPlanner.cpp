// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningOptimizationPlanner.h"

#include "LearningLog.h"
#include "LearningOptimizer.h"
#include "LearningProgress.h"

#include "Internationalization/Internationalization.h"
#define LOCTEXT_NAMESPACE "LearningOptimizationPlanner"

namespace UE::Learning
{
	void FOptimizationPlannerBuffer::Resize(
		const int32 SampleNum,
		const int32 StepNum,
		const int32 ActionVectorDimensionNum)
	{
		Samples.SetNumUninitialized({ SampleNum, StepNum, ActionVectorDimensionNum });
		Losses.SetNumUninitialized({ SampleNum });
	}

	namespace OptimizationPlanner
	{
		void RunPlan(
			TLearningArrayView<2, float> ActionVectorBuffer,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TLearningArrayView<3, const float> ActionVectors,
			const FIndexSet Instances)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::OptimizationPlanner::RunPlan);

			const int32 StepNum = ActionVectors.Num<0>();

			// Reset Environment

			ResetFunction(Instances);

			// Step

			for (int32 StepIdx = 0; StepIdx < StepNum; StepIdx++)
			{
				// Decode Actions

				Array::Copy(ActionVectorBuffer, ActionVectors[StepIdx], Instances);
				Array::Check(ActionVectorBuffer, Instances);

				ActionFunction(Instances);

				// Update Environment

				UpdateFunction(Instances);
			}
		}

		static inline void UpdateOptimizerBest(
			TLearningArrayView<1, float> InOutActionVector,
			float& InOutBestLoss,
			const TLearningArrayView<2, const float> ActionVectors,
			const TLearningArrayView<1, const float> Losses,
			FRWLock* InOutActionVectorLock,
			TAtomic<bool>* bActionVectorsUpdatedFlag)
		{
			const int32 LossesNum = Losses.Num();

			int32 BestIndex = INDEX_NONE;

			for (int32 LossIdx = 0; LossIdx < LossesNum; LossIdx++)
			{
				if (Losses[LossIdx] < InOutBestLoss)
				{
					InOutBestLoss = Losses[LossIdx];
					BestIndex = LossIdx;
				}
			}

			if (BestIndex != INDEX_NONE)
			{
				FScopeNullableWriteLock ScopeLock(InOutActionVectorLock);

				Array::Copy(InOutActionVector, ActionVectors[BestIndex]);

				if (bActionVectorsUpdatedFlag)
				{
					*bActionVectorsUpdatedFlag = true;
				}
			}
		}

		static inline void ComputePlanLosses(
			FOptimizationPlannerBuffer& OptimizationPlannerBuffer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			const int32 StepNum,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const FIndexSet SampleInstances)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::OptimizationPlanner::ComputePlanLosses);

			// Zero Losses

			Array::Zero(OptimizationPlannerBuffer.Losses);

			// For each step

			for (int32 StepIdx = 0; StepIdx < StepNum; StepIdx++)
			{
				// Copy action vectors from the samples

				int32 SampleIdx = 0;
				for (const int32 InstanceIdx : SampleInstances)
				{
					Array::Copy(
						ActionVectorBuffer[InstanceIdx],
						OptimizationPlannerBuffer.Samples[SampleIdx][StepIdx]);

					SampleIdx++;
				}

				// Decode Actions

				Array::Check(ActionVectorBuffer, SampleInstances);
				ActionFunction(SampleInstances);

				// Update Environment

				UpdateFunction(SampleInstances);

				// Compute Rewards

				RewardFunction(SampleInstances);

				// Update Loss (use negative reward)

				SampleIdx = 0;
				for (const int32 InstanceIdx : SampleInstances)
				{
					OptimizationPlannerBuffer.Losses[SampleIdx] -= RewardBuffer[InstanceIdx] / StepNum;
					SampleIdx++;
				}
			}
		}

		void Plan(
			TLearningArrayView<2, float> InOutActionVectors,
			FOptimizationPlannerBuffer& OptimizationPlannerBuffer,
			IOptimizer& Optimizer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			const int32 IterationNum,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const FIndexSet SampleInstances,
			const ELogSetting LogSettings,
			FProgress* Progress,
			FRWLock* ActionVectorsLock,
			TAtomic<bool>* bActionVectorsUpdatedFlag)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::OptimizationPlanner::Plan);

			checkf(OptimizationPlannerBuffer.Samples.Num<0>() == SampleInstances.Num(),
				TEXT("Number of samples in OptimizationPlannerBuffer (%lli) must match the number of SampleInstances (%i)."),
				OptimizationPlannerBuffer.Samples.Num<0>(), SampleInstances.Num());

			checkf(OptimizationPlannerBuffer.Samples.Num<1>() == InOutActionVectors.Num<0>(),
				TEXT("Number of steps in OptimizationPlannerBuffer (%lli) must match the number of steps in InOutActionVectors (%lli)."),
				OptimizationPlannerBuffer.Samples.Num<1>(), InOutActionVectors.Num<0>());

			const int32 StepNum = OptimizationPlannerBuffer.Samples.Num<1>();

			if (Progress)
			{
				Progress->SetMessage(LOCTEXT("LearningOptimizationPlannerProgressMessage", "Learning: Running Optimization Planner..."));
				Progress->SetProgress(IterationNum);
			}

			float BestLoss = MAX_flt;

			Optimizer.Reset(
				OptimizationPlannerBuffer.Samples.Flatten<1>(),
				InOutActionVectors.Flatten());

			for (int32 IterationIdx = 0; IterationIdx < IterationNum; IterationIdx++)
			{
				// Reset Environment

				ResetFunction(SampleInstances);

				// Generate Plan

				ComputePlanLosses(
					OptimizationPlannerBuffer,
					ActionVectorBuffer,
					RewardBuffer,
					StepNum,
					ActionFunction,
					UpdateFunction,
					RewardFunction,
					SampleInstances);

				// Update Optimizer

				UpdateOptimizerBest(
					InOutActionVectors.Flatten(),
					BestLoss,
					OptimizationPlannerBuffer.Samples.Flatten<1>(),
					OptimizationPlannerBuffer.Losses,
					ActionVectorsLock,
					bActionVectorsUpdatedFlag);

				Optimizer.Update(
					OptimizationPlannerBuffer.Samples.Flatten<1>(),
					OptimizationPlannerBuffer.Losses,
					LogSettings);

				if (Progress) { Progress->Decrement(); }
			}
		}

		void PlanWindowed(
			TLearningArrayView<2, float> InOutActionVectors,
			FOptimizationPlannerBuffer& OptimizationPlannerBuffer,
			IOptimizer& Optimizer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			const int32 InitialIterationsNum,
			const int32 FinalIterationsNum,
			const int32 WindowIterationsNum,
			const int32 WindowStepNum,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances, const int32 Instance)> ResetFromInstanceFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const int32 PlanInstance,
			const FIndexSet SampleInstances,
			const ELogSetting LogSettings,
			FProgress* Progress,
			FRWLock* ActionVectorsLock,
			TAtomic<bool>* bActionVectorsUpdatedFlag)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::OptimizationPlanner::PlanWindowed);

			checkf(WindowStepNum <= InOutActionVectors.Num<0>(),
				TEXT("WindowStepNum (%i) must be less than or equal to the number of steps in the action vectors (%lli)"),
				WindowStepNum, InOutActionVectors.Num<0>());

			checkf(OptimizationPlannerBuffer.Losses.Num() == SampleInstances.Num(),
				TEXT("Number of samples in OptimizationPlannerBuffer (%lli) must match the number of SampleInstances (%i)."),
				OptimizationPlannerBuffer.Losses.Num(), SampleInstances.Num());

			checkf(OptimizationPlannerBuffer.Samples.Num<1>() == WindowStepNum,
				TEXT("Number of steps in OptimizationPlannerBuffer (%lli) must match WindowStepNum (%i)."),
				OptimizationPlannerBuffer.Samples.Num<1>(), WindowStepNum);

			check(!SampleInstances.Contains(PlanInstance));

			const int32 TotalStepNum = InOutActionVectors.Num<0>();

			if (Progress)
			{
				const int32 TotalIterationNum =
					WindowStepNum == TotalStepNum ? InitialIterationsNum :
					WindowStepNum == TotalStepNum - 1 ? InitialIterationsNum + FinalIterationsNum :
					InitialIterationsNum + FinalIterationsNum + (TotalStepNum - WindowStepNum - 1) * WindowIterationsNum;

				Progress->SetMessage(LOCTEXT("LearningOptimizationPlannerProgressMessage", "Learning: Running Optimization Planner..."));
				Progress->SetProgress(TotalIterationNum);
			}

			// Reset Plan Instance

			ResetFunction(PlanInstance);

			// Begin Sliding Window

			int32 StepIdx = 0;

			while (StepIdx + WindowStepNum <= TotalStepNum)
			{
				float BestLoss = MAX_flt;

				Optimizer.Reset(
					OptimizationPlannerBuffer.Samples.Flatten<1>(),
					InOutActionVectors.Slice(StepIdx, WindowStepNum).Flatten());

				const bool bInitialIteration = StepIdx == 0;
				const bool bFinalIteration = StepIdx + WindowStepNum == TotalStepNum;

				const int32 IterationNum =
					bInitialIteration ? InitialIterationsNum :
					bFinalIteration ? FinalIterationsNum :
					WindowIterationsNum;

				for (int32 IterationIdx = 0; IterationIdx < IterationNum; IterationIdx++)
				{
					// Reset Environment from current plan

					ResetFromInstanceFunction(SampleInstances, PlanInstance);

					// Generate Plan

					ComputePlanLosses(
						OptimizationPlannerBuffer,
						ActionVectorBuffer,
						RewardBuffer,
						WindowStepNum,
						ActionFunction,
						UpdateFunction,
						RewardFunction,
						SampleInstances);

					// Update Optimizer

					UpdateOptimizerBest(
						InOutActionVectors.Slice(StepIdx, WindowStepNum).Flatten(),
						BestLoss,
						OptimizationPlannerBuffer.Samples.Flatten<1>(),
						OptimizationPlannerBuffer.Losses,
						ActionVectorsLock,
						bActionVectorsUpdatedFlag);

					Optimizer.Update(
						OptimizationPlannerBuffer.Samples.Flatten<1>(),
						OptimizationPlannerBuffer.Losses,
						LogSettings);

					if (Progress) { Progress->Decrement(); }
				}

				const int32 PlanStepNum = bFinalIteration ? WindowStepNum : 1;

				for (int32 PlanStepIdx = 0; PlanStepIdx < PlanStepNum; PlanStepIdx++)
				{
					// Copy action vectors for this step

					Array::Copy(ActionVectorBuffer[PlanInstance], InOutActionVectors[StepIdx]);

					// Decode Actions

					Array::Check(ActionVectorBuffer, PlanInstance);
					ActionFunction(PlanInstance);

					// Update Environment

					UpdateFunction(PlanInstance);
				}

				// Slide window forward

				StepIdx += PlanStepNum;
			}
		}

	}
}

#undef LOCTEXT_NAMESPACE
