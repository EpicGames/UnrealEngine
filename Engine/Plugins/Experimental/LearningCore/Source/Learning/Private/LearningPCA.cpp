// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningPCA.h"

#include "LearningLog.h"
#include "LearningEigen.h"
#include "LearningProgress.h"

#include "Internationalization/Internationalization.h"
#define LOCTEXT_NAMESPACE "LearningPCA"

namespace UE::Learning
{
	void FPCAEncoder::Empty()
	{
		Matrix.Empty();
	}

	bool FPCAEncoder::IsEmpty() const
	{
		return Matrix.IsEmpty();
	}

	bool FPCAEncoder::Serialize(FArchive& Ar)
	{
		Array::Serialize(Ar, Matrix);

		return true;
	}

	int32 FPCAEncoder::DimensionNum() const
	{
		return Matrix.Num<0>();
	}

	int32 FPCAEncoder::FeatureNum() const
	{
		return Matrix.Num<1>();
	}

	FPCAResult FPCAEncoder::Fit(
		const TLearningArrayView<2, const float> Data,
		const FPCASettings& Settings,
		const ELogSetting LogSettings,
		FProgress* Progress)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPCAEncoder::Fit);

		Array::Check(Data);

		const int32 FramesNum = Data.Num<0>();
		const int32 DimensionNum = Data.Num<1>();
		const int32 SubsampleRate = Settings.Subsample;
		const int32 SubsampleNum = FramesNum / SubsampleRate;

		if (Progress)
		{
			Progress->SetMessage(LOCTEXT("FitProgressMessage", "Learning: Fitting PCA..."));

			// Right now it is impossible to get the process from the PCA computation
			// so we have to settle with just setting 1 and then 0 when it is done.
			Progress->SetProgress(1);
		}

		// Compute PCA

		Eigen::ComputationInfo Info;
		Eigen::VectorXf VarianceRatio;
		Eigen::MatrixXf MatrixV;

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Computing PCA...");
		}

		if (Settings.bStableComputation)
		{
			Eigen::JacobiSVD<Eigen::MatrixXf> SVDSolver(
				InEigenMatrix(Data)(Eigen::seqN(0, SubsampleNum, SubsampleRate), Eigen::all),
				Eigen::ComputeThinV);

			Info = SVDSolver.info();
			VarianceRatio = SVDSolver.singularValues() / SVDSolver.singularValues().sum();
			MatrixV = SVDSolver.matrixV();
		}
		else
		{
			Eigen::BDCSVD<Eigen::MatrixXf> SVDSolver(
				InEigenMatrix(Data)(Eigen::seqN(0, SubsampleNum, SubsampleRate), Eigen::all),
				Eigen::ComputeThinV);

			Info = SVDSolver.info();
			VarianceRatio = SVDSolver.singularValues() / SVDSolver.singularValues().sum();
			MatrixV = SVDSolver.matrixV();
		}

		// If successful

		int32 ComponentsToKeepNum = 0;
		float KeptVarianceRatio = 0.0f;

		if (ensure(Info == Eigen::ComputationInfo::Success))
		{
			// Find number of components to keep

			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Display, "--- PCA Cumulative Variance ---");
			}

			for (int32 ValueIndex = 0; ValueIndex < FMath::Min((int32)VarianceRatio.size(), Settings.MaximumDimensions); ValueIndex++)
			{
				ComponentsToKeepNum = ValueIndex + 1;
				KeptVarianceRatio += FMath::Abs(VarianceRatio(ValueIndex));

				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Display, "%4i: %6.4f", ValueIndex, KeptVarianceRatio);
				}

				if (KeptVarianceRatio > Settings.MaximumVarianceRatio)
				{
					break;
				}
			}

			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Display, "--------------------");
			}

			check(ComponentsToKeepNum > 0);

			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Display, "Kept %i of %i Components", ComponentsToKeepNum, DimensionNum);
			}

			// Copy Transform

			Matrix.SetNumUninitialized({ DimensionNum, ComponentsToKeepNum });
			OutEigenMatrix(Matrix).noalias() = MatrixV.leftCols(ComponentsToKeepNum);
			Array::Check(Matrix);
		}
		else
		{
			UE_LOGF(LogLearning, Warning, "PCA Computation Failed: %ls", Eigen::ComputationInfoString(Info));
			Empty();
		}

		if (Progress)
		{
			Progress->Done();
		}

		// Result

		FPCAResult Result;
		Result.bSuccess = Info == Eigen::ComputationInfo::Success;
		Result.DimensionNum = ComponentsToKeepNum;
		Result.VarianceRatioPreserved = KeptVarianceRatio;

		return Result;
	}

	void FPCAEncoder::Transform(
		TLearningArrayView<2, float> OutData,
		const TLearningArrayView<2, const float> Data,
		FProgress* Progress) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPCAEncoder::Transform);

		if (Progress)
		{
			Progress->SetMessage(LOCTEXT("TransformProgressMessage", "Learning: PCA Transform..."));
			Progress->SetProgress(Data.Num<0>());
		}

		Array::Check(Data);

		SlicedParallelFor(Data.Num<0>(), 128, [&](const int32 RowStart, const int32 RowSliceNum)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::PCA::Transform::Batch);

			OutEigenMatrix(OutData.Slice(RowStart, RowSliceNum)).noalias() =
				(InEigenMatrix(Matrix).transpose() * InEigenMatrix(Data.Slice(RowStart, RowSliceNum)).transpose()).transpose();

			if (Progress)
			{
				Progress->Decrement(RowSliceNum);
			}
		});

		Array::Check(OutData);
	}

	void FPCAEncoder::Transform(
		TLearningArrayView<1, float> OutData,
		const TLearningArrayView<1, const float> Data) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPCAEncoder::Transform);
		check(Data.Num() == DimensionNum());
		check(OutData.Num() == FeatureNum());

		Array::Check(Data);

		OutEigenVector(OutData).noalias() = InEigenMatrix(Matrix).transpose() * InEigenVector(Data);

		Array::Check(OutData);
	}

	void FPCAEncoder::InverseTransform(
		TLearningArrayView<1, float> OutData,
		const TLearningArrayView<1, const float> Data) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPCAEncoder::InverseTransform);
		check(Data.Num() == FeatureNum());
		check(OutData.Num() == DimensionNum());

		Array::Check(Data);

		OutEigenVector(OutData).noalias() = InEigenMatrix(Matrix) * InEigenVector(Data);

		Array::Check(OutData);
	}

}

#undef LOCTEXT_NAMESPACE
