// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningPSOOptimizer.h"

#include "LearningRandom.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning
{
	FPSOOptimizer::FPSOOptimizer(const uint32 InSeed, const FPSOOptimizerSettings& InSettings) : Seed(InSeed), Settings(InSettings) { }

	void FPSOOptimizer::Resize(const int32 SampleNum, const int32 DimensionsNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPSOOptimizer::Resize);

		LocalBestPositions.SetNumUninitialized({ SampleNum, DimensionsNum });
		LocalBestLoss.SetNumUninitialized({ SampleNum });
		GlobalBestPosition.SetNumUninitialized({ DimensionsNum });
		Velocities.SetNumUninitialized({ SampleNum, DimensionsNum });
		LocalUniformSamples.SetNumUninitialized({ SampleNum, DimensionsNum });
		GlobalUniformSamples.SetNumUninitialized({ SampleNum, DimensionsNum });
	}

	void FPSOOptimizer::Reset(
		TLearningArrayView<2, float> OutSamples,
		const TLearningArrayView<1, const float> InitialGuess)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPSOOptimizer::Reset);

		const int32 SampleNum = OutSamples.Num<0>();
		const int32 DimensionNum = OutSamples.Num<1>();

		Iterations = 0;

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			Array::Copy(LocalBestPositions[SampleIdx], InitialGuess);
		}
		Array::Set(LocalBestLoss, MAX_flt);

		Array::Copy(GlobalBestPosition, InitialGuess);
		GlobalBestLoss = MAX_flt;

		Random::SampleUniformArray(Velocities.Flatten(), Seed, -1.0f, 1.0f);
		Random::SampleUniformArray(OutSamples.Flatten(), Seed, -1.0f, 1.0f);

		for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			for (int32 DimensionIdx = 0; DimensionIdx < DimensionNum; DimensionIdx++)
			{
				OutSamples[SampleIdx][DimensionIdx] += InitialGuess[DimensionIdx];
			}
		}
	}

	void FPSOOptimizer::Update(
		TLearningArrayView<2, float> InOutSamples,
		const TLearningArrayView<1, const float> Losses,
		const ELogSetting LogSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPSOOptimizer::Update);

		const int32 SampleNum = InOutSamples.Num<0>();
		const int32 DimNum = InOutSamples.Num<1>();

		Array::Check(InOutSamples);
		Array::Check(Losses);

		Random::SampleFloatArray(
			LocalUniformSamples.Flatten(),
			Seed);

		Random::SampleFloatArray(
			GlobalUniformSamples.Flatten(),
			Seed);

		// Update Iteration Count

		Iterations++;

		// I've done a slight re-ordering of the classical algorithm. Here I update 
		// the bests first to avoid additional fitness function evaluations and to 
		// make the usage of the algorithm resemble a bit more CMA in terms of API.
		// I don't think it should have any impact since these two steps are performed
		// in a loop anyway, so it is just like taking the end of one loop iteration, and 
		// performing it instead at the beginning of the next loop iteration.

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPSOOptimizer::Update::UpdateBest);

#if UE_LEARNING_ISPC
			ispc::LearningUpdatePSOBest(
				GlobalBestLoss,
				GlobalBestPosition.GetData(),
				LocalBestLoss.GetData(),
				LocalBestPositions.GetData(),
				Losses.GetData(),
				InOutSamples.GetData(),
				SampleNum,
				DimNum);
#else
			for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				if (Losses[SampleIdx] < LocalBestLoss[SampleIdx])
				{
					LocalBestLoss[SampleIdx] = Losses[SampleIdx];
					Array::Copy(LocalBestPositions[SampleIdx], InOutSamples[SampleIdx]);

					if (Losses[SampleIdx] < GlobalBestLoss)
					{
						GlobalBestLoss = Losses[SampleIdx];
						Array::Copy(GlobalBestPosition, InOutSamples[SampleIdx]);
					}
				}
			}
#endif
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FPSOOptimizer::Update::UpdateVelocitiesSamples);

#if UE_LEARNING_ISPC
			ispc::LearningUpdatePSOVelocitiesSamples(
				Velocities.GetData(),
				InOutSamples.GetData(),
				LocalUniformSamples.GetData(),
				GlobalUniformSamples.GetData(),
				LocalBestPositions.GetData(),
				GlobalBestPosition.GetData(),
				SampleNum,
				DimNum,
				Settings.Momentum,
				Settings.LocalGain,
				Settings.GlobalGain);
#else
			for (int32 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					Velocities[SampleIdx][DimIdx] = (
						Settings.Momentum * Velocities[SampleIdx][DimIdx] +
						Settings.LocalGain * LocalUniformSamples[SampleIdx][DimIdx] * (LocalBestPositions[SampleIdx][DimIdx] - InOutSamples[SampleIdx][DimIdx]) +
						Settings.GlobalGain * GlobalUniformSamples[SampleIdx][DimIdx] * (GlobalBestPosition[DimIdx] - InOutSamples[SampleIdx][DimIdx]));
					
					InOutSamples[SampleIdx][DimIdx] += Velocities[SampleIdx][DimIdx];
				}
			}
#endif
		}

		if (LogSettings != ELogSetting::Silent)
		{
			float AvgLoss = 0.0f;
			for (int32 LossIdx = 0; LossIdx < SampleNum; LossIdx++)
			{
				AvgLoss += Losses[LossIdx] / SampleNum;
			}

			UE_LOGF(LogLearning, Display, "PSO Iteration %6i | Mean Loss: %5.3f | Best Loss: %5.3f",
				Iterations, AvgLoss, GlobalBestLoss);
		}
	}




}

