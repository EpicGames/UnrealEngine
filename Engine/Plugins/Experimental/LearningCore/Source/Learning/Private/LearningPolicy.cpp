// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningPolicy.h"

namespace UE::Learning
{
	FNeuralNetworkPolicy::FNeuralNetworkPolicy(
		const int32 InMaxInstanceNum,
		const int32 InObservationEncodedNum,
		const int32 InActionEncodedNum,
		const int32 InMemoryStateNum,
		const TSharedPtr<FNeuralNetwork>& InNeuralNetwork,
		const FNeuralNetworkInferenceSettings& InInferenceSettings)
		: ObservationEncodedNum(InObservationEncodedNum)
		, ActionEncodedNum(InActionEncodedNum)
		, MemoryStateNum(InMemoryStateNum)
	{
		check(InNeuralNetwork->GetInputSize() == ObservationEncodedNum + MemoryStateNum);
		check(InNeuralNetwork->GetOutputSize() == ActionEncodedNum + MemoryStateNum);

		Input.SetNumUninitialized({ InMaxInstanceNum, ObservationEncodedNum + MemoryStateNum });
		Output.SetNumUninitialized({ InMaxInstanceNum, ActionEncodedNum + MemoryStateNum });

		Array::Zero(Input);
		Array::Zero(Output);

		NeuralNetworkFunction = MakeShared<FNeuralNetworkFunction>(InMaxInstanceNum, InNeuralNetwork, InInferenceSettings);
	}

	void FNeuralNetworkPolicy::Evaluate(
		TLearningArrayView<2, float> OutputActionVectorsEncoded,
		TLearningArrayView<2, float> OutputMemoryState,
		const TLearningArrayView<2, const float> InputObservationVectorsEncoded,
		const TLearningArrayView<2, const float> InputMemoryState, 
		const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FNeuralNetworkPolicy::Evaluate);

		const int32 InstanceNum = Instances.Num();

		Array::Check(InputObservationVectorsEncoded, Instances);
		Array::Check(InputMemoryState, Instances);

		// Copy in Observation and Memory State into network input

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(Input[InstanceIdx].Slice(0, ObservationEncodedNum), InputObservationVectorsEncoded[InstanceIdx]);
			Array::Copy(Input[InstanceIdx].Slice(ObservationEncodedNum, MemoryStateNum), InputMemoryState[InstanceIdx]);
		}

		NeuralNetworkFunction->Evaluate(Output, Input, Instances);

		// Copy Out Memory State

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(OutputActionVectorsEncoded[InstanceIdx], Output[InstanceIdx].Slice(0, ActionEncodedNum));
			Array::Copy(OutputMemoryState[InstanceIdx], Output[InstanceIdx].Slice(ActionEncodedNum, MemoryStateNum));
		}

		Array::Check(OutputActionVectorsEncoded, Instances);
		Array::Check(OutputMemoryState, Instances);
	}

	void FNeuralNetworkPolicy::UpdateNeuralNetwork(const TSharedPtr<FNeuralNetwork>& NewNeuralNetwork)
	{
		NeuralNetworkFunction->UpdateNeuralNetwork(NewNeuralNetwork);
	}

	const TSharedPtr<FNeuralNetwork>& FNeuralNetworkPolicy::GetNeuralNetwork() const
	{
		return NeuralNetworkFunction->GetNeuralNetwork();
	}
}
