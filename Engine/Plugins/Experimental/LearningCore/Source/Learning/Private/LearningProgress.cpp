// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningProgress.h"

namespace UE::Learning
{
	int32 FProgress::GetProgress() const
	{
		return Progress;
	}

	void FProgress::GetMessage(FText& OutMessage)
	{
		Lock.ReadLock();
		OutMessage = Message;
		Lock.ReadUnlock();
	}

	void FProgress::SetProgress(const int32 InProgress)
	{
		Progress = InProgress;
	}

	void FProgress::SetMessage(const FText& InMessage)
	{
		Lock.WriteLock();
		Message = InMessage;
		Lock.WriteUnlock();
	}

	void FProgress::Decrement()
	{
		Progress--;
	}

	void FProgress::Decrement(const int32 Num)
	{
		Progress -= Num;
	}

	void FProgress::Done()
	{
		Progress = 0;
	}

	FScopeNullableReadLock::FScopeNullableReadLock(FRWLock* InLock) : Lock(InLock)
	{
		if (Lock)
		{
			Lock->ReadLock();
		}
	}

	FScopeNullableReadLock::~FScopeNullableReadLock()
	{
		if (Lock)
		{
			Lock->ReadUnlock();
			Lock = nullptr;
		}
	}

	FScopeNullableWriteLock::FScopeNullableWriteLock(FRWLock* InLock) : Lock(InLock)
	{
		if (Lock)
		{
			Lock->WriteLock();
		}
	}

	FScopeNullableWriteLock::~FScopeNullableWriteLock()
	{
		if (Lock)
		{
			Lock->WriteUnlock();
			Lock = nullptr;
		}
	}

}