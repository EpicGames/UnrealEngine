// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningRandom.h"

#include "LearningFrameSet.h"
#include "LearningFrameRangeSet.h"

#if UE_LEARNING_ISPC
#include "Learning.ispc.generated.h"
#endif

namespace UE::Learning::Random
{
	static inline float UniformToGaussian(const float R1, const float R2)
	{
		return FMath::Sqrt(-2.0f * FMath::Loge(FMath::Max(R1, UE_SMALL_NUMBER))) * FMath::Cos(R2 * UE_TWO_PI);
	}

	static inline float Sigmoid(const float X)
	{
		return 1.0f / (1.0f + FMath::Exp(-X));
	}

	uint32 Int(const uint32 State)
	{
		// Here we use a simple xor and shift based 
		// hashing algorithm which is appropriate for 
		// vectorization using ISPC
		uint32 X = State ^ 0xb74eaecf;
		X = ((X >> 16) ^ X) * 0x45d9f3b;
		X = ((X >> 16) ^ X) * 0x45d9f3b;
		return (X >> 16) ^ X;
	}

	float Float(const uint32 State)
	{
		// Same approach as used in FRandomStream
		float Output;
		*((uint32*)(&Output)) = 0x3F800000U | (Int(State ^ 0x1c89a74a) >> 9);
		return Output - 1.0f;
	}

	int32 IntInRange(const uint32 State, const int32 Min, const int32 Max)
	{
		const int32 Range = (Max - Min) + 1;
		return Min + ((Range > 0) ? FMath::TruncToInt(Float(State ^ 0x7d3b208a) * (float)(Range)) : 0);
	}

	float Uniform(
		const uint32 State,
		const float Min,
		const float Max)
	{
		return (Max - Min) * Float(State ^ 0x72404541) + Min;
	}

	float Gaussian(
		const uint32 State,
		const float Mean,
		const float Std)
	{
		return Std * UniformToGaussian(
			Float(State ^ 0x4855e88f),
			Float(State ^ 0x0eedb850)) + Mean;
	}

	FVector VectorGaussian(
		const uint32 State,
		const FVector Mean,
		const FVector Std)
	{
		return FVector(
			Gaussian(State ^ 0x0c7eaaf3, Mean.X, Std.X),
			Gaussian(State ^ 0x109be32d, Mean.Y, Std.Y),
			Gaussian(State ^ 0xed839d93, Mean.Z, Std.Z));
	}

	float ClippedGaussian(
		const uint32 State,
		const float Mean,
		const float Std,
		const float Clip)
	{
		return FMath::Clamp(Gaussian(State ^ 0xf3ce904a, Mean, Std), Mean - Std * Clip, Mean + Std * Clip);
	}

	FVector PlanarUniform(
		const uint32 State,
		const float Min,
		const float Max,
		const FVector Axis0,
		const FVector Axis1)
	{
		return 
			Uniform(Int(State ^ 0x0fd71b1d), Min, Max) * Axis0 +
			Uniform(Int(State ^ 0x0a25cffa), Min, Max) * Axis1;
	}

	FVector PlanarGaussian(
		const uint32 State,
		const float Mean,
		const float Std,
		const FVector Axis0,
		const FVector Axis1)
	{
		return 
			Gaussian(Int(State ^ 0x371f49fd), Mean, Std) * Axis0 +
			Gaussian(Int(State ^ 0xcf35268a), Mean, Std) * Axis1;
	}

	FVector PlanarClippedGaussian(
		const uint32 State,
		const float Mean,
		const float Std,
		const float Clip,
		const FVector Axis0,
		const FVector Axis1)
	{
		return 
			ClippedGaussian(Int(State ^ 0xa29f129d), Mean, Std, Clip) * Axis0 +
			ClippedGaussian(Int(State ^ 0x8facd15b), Mean, Std, Clip) * Axis1;
	}

	FVector PlanarDirection(
		const uint32 State,
		const FVector Axis0,
		const FVector Axis1)
	{
		return PlanarGaussian(State ^ 0x46b8aa96, 0.0f, 1.0f, Axis0, Axis1).GetSafeNormal(UE_SMALL_NUMBER, Axis0);
	}

	FQuat Rotation(const uint32 State)
	{
		FQuat Rotation = FQuat(
			Gaussian(Int(State ^ 0x21ed962e)),
			Gaussian(Int(State ^ 0xeac13a67)),
			Gaussian(Int(State ^ 0xb6f2db89)),
			Gaussian(Int(State ^ 0xc10cd5e4))).GetNormalized();

		Rotation.EnforceShortestArcWith(FQuat::Identity);
		return Rotation;
	}

	void IntArray(
		TLearningArrayView<1, uint32> Output,
		const uint32 State)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::IntArray);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomIntArray(
			Output.GetData(),
			ElementNum,
			State);
#else
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = Int(State ^ 0xbed25b30 ^ Int(ElementIdx ^ 0xb521a8d9));
		}
#endif
	}

	LEARNING_API void IntArray(
		TLearningArrayView<1, uint32> Output,
		const uint32 State,
		const FIndexSet Indices)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::IntArray);

		if (Indices.IsSlice() && Indices.Num() > 4)
		{
			IntArray(
				Output.Slice(Indices.GetSliceStart(), Indices.GetSliceNum()),
				State);
		}
		else
		{
			for (const int32 ElementIdx : Indices)
			{
				Output[ElementIdx] = Int(State ^ 0xbed25b30 ^ Int(ElementIdx ^ 0xb521a8d9));
			}
		}
	}

	void FloatArray(
		TLearningArrayView<1, float> Output,
		const uint32 State)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::FloatArray);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomFloatArray(
			Output.GetData(),
			ElementNum,
			State);
#else
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = Float(State ^ 0xf955fac7 ^ Int(ElementIdx ^ 0xcd989d6f));
		}
#endif
	}

	void UniformArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Min,
		const float Max)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::UniformArray);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomUniformArray(
			Output.GetData(),
			ElementNum,
			State,
			Min,
			Max);
#else
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = Uniform(State ^ 0x5f15554c ^ Int(ElementIdx ^ 0x242735e0), Min, Max);
		}
#endif
	}

	void GaussianArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Mean,
		const float Std)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::GaussianArray);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomGaussianArray(
			Output.GetData(),
			ElementNum,
			State,
			Mean,
			Std);
#else
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = Gaussian(State ^ 0x7b5d5f62 ^ Int(ElementIdx ^ 0x546ab965), Mean, Std);
		}
#endif
	}

	void ClippedGaussianArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Mean,
		const float Std,
		const float Clip)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::ClippedGaussianArray);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomClippedGaussianArray(
			Output.GetData(),
			ElementNum,
			State,
			Mean,
			Std,
			Clip);
#else
		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = ClippedGaussian(State ^ 0x11a760f7 ^ Int(ElementIdx ^ 0x1dfc9b77), Mean, Std, Clip);
		}
#endif
	}

	void PlanarClippedGaussianArray(
		TLearningArrayView<1, FVector> Output,
		const uint32 State,
		const float Mean,
		const float Std,
		const float Clip,
		const FVector Axis0,
		const FVector Axis1)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::PlanarClippedGaussianArray);

		const int32 ElementNum = Output.Num();

		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = PlanarClippedGaussian(State ^ 0x5eeac916 ^ Int(ElementIdx ^ 0x8527618d), Mean, Std, Clip, Axis0, Axis1);
		}
	}

	void PlanarDirectionArray(
		TLearningArrayView<1, FVector> Output,
		const uint32 State,
		const FVector Axis0,
		const FVector Axis1)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::PlanarDirectionArray);

		const int32 ElementNum = Output.Num();

		for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
		{
			Output[ElementIdx] = PlanarDirection(State ^ 0xd80bd375 ^ Int(ElementIdx ^ 0x50d8c207), Axis0, Axis1);
		}
	}

	void DistributionIndependantNormal(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionIndependantNormal);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomDistributionIndependantNormal(
			Output.GetData(),
			Mean.GetData(),
			LogStd.GetData(),
			ElementNum,
			State,
			Scale);
#else
		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Mean[ElementIdx];
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Gaussian(State ^ 0x7db0e4e9 ^ Int(ElementIdx ^ 0xf4976a00), Mean[ElementIdx], Scale * FMath::Exp(FMath::Min(LogStd[ElementIdx], 10.0f)));
			}
		}
#endif
	}

	void DistributionIndependantNormalMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const TLearningArrayView<1, const bool> Masked,
		const TLearningArrayView<1, const float> MaskedValues,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionIndependantNormalMasked);

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomDistributionIndependantNormalMasked(
			Output.GetData(),
			Mean.GetData(),
			LogStd.GetData(),
			Masked.GetData(),
			MaskedValues.GetData(),
			ElementNum,
			State,
			Scale);
#else
		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Masked[ElementIdx] ? MaskedValues[ElementIdx] : Mean[ElementIdx];
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Masked[ElementIdx] ? MaskedValues[ElementIdx] : Gaussian(State ^ 0x0a006c6b ^ Int(ElementIdx ^ 0x6fe1d121), Mean[ElementIdx], Scale * FMath::Exp(FMath::Min(LogStd[ElementIdx], 10.0f)));
			}
		}
#endif
	}

	void DistributionMultinoulli(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionMultinoulli);

		check(Output.Num() == Logits.Num());

		const int32 ElementNum = Output.Num();
		if (ElementNum == 0)
		{
			return;
		}

		float MaxValue = -UE_MAX_FLT;
		uint32 MaxIdx = INDEX_NONE;

		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				if (Logits[ElementIdx] > MaxValue)
				{
					MaxValue = Logits[ElementIdx];
					MaxIdx = ElementIdx;
				}
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				const float ElementValue = Logits[ElementIdx] / Scale - FMath::Loge(-FMath::Loge(Uniform(State ^ 0x7a156b37 ^ Int(ElementIdx ^ 0x0c71bebb))));

				if (ElementValue > MaxValue)
				{
					MaxValue = ElementValue;
					MaxIdx = ElementIdx;
				}
			}
		}

		check(MaxIdx != INDEX_NONE);

		Array::Zero(Output);
		Output[MaxIdx] = 1.0f;
	}

	void DistributionMultinoulliMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionMultinoulliMasked);

		check(Output.Num() == Logits.Num());
		check(Logits.Num() == Masked.Num());

		const int32 ElementNum = Output.Num();
		if (ElementNum == 0)
		{
			return;
		}

		float MaxValue = -UE_MAX_FLT;
		uint32 MaxIdx = INDEX_NONE;

		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				if (!Masked[ElementIdx] && Logits[ElementIdx] > MaxValue)
				{
					MaxValue = Logits[ElementIdx];
					MaxIdx = ElementIdx;
				}
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				if (!Masked[ElementIdx])
				{
					const float ElementValue = Logits[ElementIdx] / Scale - FMath::Loge(-FMath::Loge(Uniform(State ^ 0x93575b09 ^ Int(ElementIdx ^ 0xa0c6ac79))));

					if (ElementValue > MaxValue)
					{
						MaxValue = ElementValue;
						MaxIdx = ElementIdx;
					}
				}
			}
		}

		check(MaxIdx != INDEX_NONE);

		Array::Zero(Output);
		Output[MaxIdx] = 1.0f;
	}

	void DistributionBernoulli(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionBernoulli);

		check(Output.Num() == Logits.Num());

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomDistributionBernoulli(
			Output.GetData(),
			Logits.GetData(),
			ElementNum,
			State,
			Scale);
#else
		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Logits[ElementIdx] > 0.0f ? 1.0f : 0.0f;
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = Sigmoid(Logits[ElementIdx] / Scale) > Uniform(State ^ 0xf4021a46 ^ Int(ElementIdx ^ 0x7a8cc64e)) ? 1.0f : 0.0f;
			}
		}
#endif
	}


	void DistributionBernoulliMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Random::DistributionBernoulliMasked);

		check(Output.Num() == Logits.Num());
		check(Logits.Num() == Masked.Num());

		const int32 ElementNum = Output.Num();

#if UE_LEARNING_ISPC
		ispc::LearningRandomDistributionBernoulliMasked(
			Output.GetData(),
			Logits.GetData(),
			Masked.GetData(),
			ElementNum,
			State,
			Scale);
#else
		if (Scale < UE_SMALL_NUMBER)
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = !Masked[ElementIdx] && Logits[ElementIdx] > 0.0f ? 1.0f : 0.0f;
			}
		}
		else
		{
			for (int32 ElementIdx = 0; ElementIdx < ElementNum; ElementIdx++)
			{
				Output[ElementIdx] = !Masked[ElementIdx] && Sigmoid(Logits[ElementIdx] / Scale) > Uniform(State ^ 0x8c992b7f ^ Int(ElementIdx ^ 0x1953d0fc)) ? 1.0f : 0.0f;
			}
		}
#endif
	}

	void FrameInFrameSet(
		int32& OutEntryIdx,
		int32& OutFrameIdx,
		const FFrameSet& FrameSet,
		const uint32 State)
	{
		const int32 TotalFrameNum = FrameSet.GetTotalFrameNum();
		const int32 FrameOffset = IntInRange(0x26810f69 ^ State, 0, TotalFrameNum - 1);
		const bool bFound = FrameSet.FindOffset(OutEntryIdx, OutFrameIdx, FrameOffset);
		check(bFound);
	}

	void FrameInFrameRangeSet(
		int32& OutEntryIdx,
		int32& OutRangeIdx,
		int32& OutRangeFrame,
		const FFrameRangeSet& FrameRangeSet,
		const uint32 State)
	{
		const int32 TotalFrameNum = FrameRangeSet.GetTotalFrameNum();
		const int32 FrameOffset = IntInRange(0x76ad14bb ^ State, 0, TotalFrameNum - 1);
		const bool bFound = FrameRangeSet.FindOffset(OutEntryIdx, OutRangeIdx, OutRangeFrame, FrameOffset);
		check(bFound);
	}

	////////////

	uint32 SampleInt(uint32& State)
	{
		State = Int(State ^ 0xba3030e4);
		return Int(State ^ 0xfdb4f6bf);
	}

	float SampleFloat(uint32& State)
	{
		State = Int(State ^ 0x2b056265);
		return Float(State ^ 0xd29c58ed);
	}

	int32 SampleIntInRange(uint32& State, const int32 Min, const int32 Max)
	{
		State = Int(State ^ 0xf732b4b4);
		return IntInRange(State ^ 0x755e1fe4, Min, Max);
	}

	float SampleUniform(
		uint32& State,
		const float Min,
		const float Max)
	{
		State = Int(State ^ 0x462b86af);
		return Uniform(State ^ 0x0c9be5a2, Min, Max);
	}

	float SampleGaussian(
		uint32& State,
		const float Mean,
		const float Std)
	{
		State = Int(State ^ 0xca0ae9bd);
		return Gaussian(State ^ 0x5df36815, Mean, Std);
	}

	FVector SampleVectorGaussian(
		uint32& State,
		const FVector Mean,
		const FVector Std)
	{
		State = Int(State ^ 0x9efa29b7);
		return VectorGaussian(State ^ 0x57167b59, Mean, Std);
	}

	float SampleClippedGaussian(
		uint32& State,
		const float Mean,
		const float Std,
		const float Clip)
	{
		State = Int(State ^ 0xcc10710e);
		return ClippedGaussian(State ^ 0xcfc12df0, Mean, Std, Clip);
	}

	FVector SamplePlanarUniform(
		uint32& State,
		const float Min,
		const float Max,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0x4d3d153f);
		return PlanarUniform(State ^ 0x28b92167, Min, Max, Axis0, Axis1);
	}

	FVector SamplePlanarGaussian(
		uint32& State,
		const float Mean,
		const float Std,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0xafce90a5);
		return PlanarGaussian(State ^ 0x8a6579c3, Mean, Std, Axis0, Axis1);
	}

	FVector SamplePlanarClippedGaussian(
		uint32& State,
		const float Mean,
		const float Std,
		const float Clip,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0xf4af224c);
		return PlanarClippedGaussian(State ^ 0x45ccf24f, Mean, Std, Clip, Axis0, Axis1);
	}

	FVector SamplePlanarDirection(
		uint32& State,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0xc763d831);
		return PlanarDirection(State ^ 0x2898da60, Axis0);
	}

	FQuat SampleRotation(uint32& State)
	{
		State = Int(State ^ 0xa8a56b0c);
		return Rotation(State ^ 0xeb4ff4d2);
	}

	void SampleIntArray(
		TLearningArrayView<1, uint32> Output,
		uint32& State)
	{
		State = Int(State ^ 0xab9c9ee3);
		IntArray(Output, State ^ 0x6adcdb41);
	}

	void SampleIntArray(
		TLearningArrayView<1, uint32> Output,
		uint32& State,
		const FIndexSet Indices)
	{
		State = Int(State ^ 0xab9c9ee3);
		IntArray(Output, State ^ 0x6adcdb41, Indices);
	}

	void SampleFloatArray(
		TLearningArrayView<1, float> Output,
		uint32& State)
	{
		State = Int(State ^ 0x543e2434);
		FloatArray(Output, State ^ 0x8a0b0503);
	}

	void SampleUniformArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Min,
		const float Max)
	{
		State = Int(State ^ 0x8f086d42);
		UniformArray(Output, State ^ 0x2f3ca619, Min, Max);
	}

	void SampleGaussianArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Mean,
		const float Std)
	{
		State = Int(State ^ 0x82a69d18);
		GaussianArray(Output, State ^ 0xf2381309, Mean, Std);
	}

	void SampleClippedGaussianArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Mean,
		const float Std,
		const float Clip)
	{
		State = Int(State ^ 0xb96fa3cf);
		ClippedGaussianArray(Output, State ^ 0xfc1bc4a9, Mean, Std, Clip);
	}

	void SamplePlanarClippedGaussianArray(
		TLearningArrayView<1, FVector> Output,
		uint32& State,
		const float Mean,
		const float Std,
		const float Clip,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0xb538a8b5);
		PlanarClippedGaussianArray(Output, State ^ 0x73a55e65, Mean, Std, Clip, Axis0, Axis1);
	}

	void SamplePlanarDirectionArray(
		TLearningArrayView<1, FVector> Output,
		uint32& State,
		const FVector Axis0,
		const FVector Axis1)
	{
		State = Int(State ^ 0x3219c5db);
		PlanarDirectionArray(Output, State ^ 0x6bfd3e6a, Axis0, Axis1);
	}

	void SampleDistributionIndependantNormal(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const float Scale)
	{
		State = Int(State ^ 0x984c4d5f);
		DistributionIndependantNormal(
			Output,
			State,
			Mean,
			LogStd,
			Scale);
	}

	void SampleDistributionIndependantNormalMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const TLearningArrayView<1, const bool> Masked,
		const TLearningArrayView<1, const float> MaskedValues,
		const float Scale)
	{
		State = Int(State ^ 0xc58f03a8);
		DistributionIndependantNormalMasked(
			Output,
			State,
			Mean,
			LogStd,
			Masked,
			MaskedValues,
			Scale);
	}

	void SampleDistributionMultinoulli(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale)
	{
		State = Int(State ^ 0xc9261d09);
		DistributionMultinoulli(
			Output,
			State,
			Logits,
			Scale);
	}

	void SampleDistributionMultinoulliMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale)
	{
		State = Int(State ^ 0xb744ea38);
		DistributionMultinoulliMasked(
			Output,
			State,
			Logits,
			Masked,
			Scale);
	}

	void SampleDistributionBernoulli(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale)
	{
		State = Int(State ^ 0xfeba3d63);
		DistributionBernoulli(
			Output,
			State,
			Logits,
			Scale);
	}

	void SampleDistributionBernoulliMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale)
	{
		State = Int(State ^ 0x21610d4a);
		DistributionBernoulliMasked(
			Output,
			State,
			Logits,
			Masked,
			Scale);
	}


	void SampleFrameInFrameSet(
		int32& OutEntryIdx,
		int32& OutFrameIdx,
		uint32& State,
		const FFrameSet& FrameSet)
	{
		State = Int(State ^ 0xfbfed01a);
		FrameInFrameSet(
			OutEntryIdx,
			OutFrameIdx,
			FrameSet,
			State ^ 0xef6c39f5);
	}

	void SampleFrameInFrameRangeSet(
		int32& OutEntryIdx,
		int32& OutRangeIdx,
		int32& OutRangeFrame,
		uint32& State,
		const FFrameRangeSet& FrameRangeSet)
	{
		State = Int(State ^ 0x54f54418);
		FrameInFrameRangeSet(
			OutEntryIdx,
			OutRangeIdx,
			OutRangeFrame,
			FrameRangeSet,
			State ^ 0xec5ae556);
	}
}