// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "UObject/NameTypes.h"

#define UE_API LEARNING_API

namespace UE::NNE::RuntimeBasic
{
	class FModelBuilderElement;
	class FModelBuilder;
}

namespace UE::Learning::Action
{
	/**
	 * Action Type
	 * 
	 * The core type of an action object for which different loss functions and behaviors are defined.
	 */
	enum class EType : uint8
	{
		// Empty Action
		Null				= 0,	

		// Vector of continuous float actions
		Continuous			= 1,

		// Set of exclusive discrete actions
		DiscreteExclusive	= 2,

		// Set of inclusive discrete actions
		DiscreteInclusive	= 3,

		// Set of named exclusive discrete actions
		NamedDiscreteExclusive = 4,

		// Set of named inclusive discrete actions
		NamedDiscreteInclusive = 5,

		// Combination of multiple actions
		And					= 6,

		// Exclusive choice from a set of actions
		OrExclusive			= 7,

		// Inclusive choice from a set of actions
		OrInclusive			= 8,

		// Fixed sized array of actions
		Array				= 9,

		// Encoding of another action
		Encoding			= 10,
	};

	enum class ENormalization : uint8
	{
		Manual = 0,
		AutoShared = 1,
		AutoPerDimension = 2,
	};

	static inline FString GetNormalizationOptionAsString(const ENormalization NormalizationOption)
	{
		switch (NormalizationOption)
		{
		case UE::Learning::Action::ENormalization::Manual: return TEXT("Manual");
		case UE::Learning::Action::ENormalization::AutoShared: return TEXT("AutoShared");
		case UE::Learning::Action::ENormalization::AutoPerDimension: return TEXT("AutoPerDimension");
		default: checkNoEntry(); return TEXT("");
		}
	}

	/**
	 * Action Schema Element
	 * 
	 * A single element in the action schema representing a part of an action. Internally this consists of a index used by the schema to look up 
	 * the associated action data and a generation id which can be used to check when this index is no longer valid.
	 */
	struct FSchemaElement { int32 Index = INDEX_NONE; uint32 Generation = INDEX_NONE; };

	struct FSchemaContinuousParameters
	{
		// Number of values in the continuous action
		int32 Num = 0;

		// Per-element standard deviation values for normalization. If empty, identity scaling is applied. *Only used when normalization option is set to manual.
		TArrayView<const float> Scales;

		// Per-element mean values for normalization. If empty, zero bias is applied. *Only used when normalization option is set to manual.
		TArrayView<const float> Offsets;

		// The normalization option to use.
		ENormalization NormalizationOption = ENormalization::Manual;
	};

	struct FSchemaDiscreteExclusiveParameters
	{
		// Number of choices in the discrete action
		int32 Num = 0;

		// Prior Probabilities of the discrete actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaDiscreteInclusiveParameters
	{
		// Number of choices in the discrete action
		int32 Num = 0;

		// Prior Probabilities of the discrete actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaNamedDiscreteExclusiveParameters
	{
		// Names of the discrete actions.
		TArrayView<const FName> ElementNames;

		// Prior Probabilities of the discrete actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaNamedDiscreteInclusiveParameters
	{
		// Names of the discrete actions.
		TArrayView<const FName> ElementNames;

		// Prior Probabilities of the discrete actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaAndParameters
	{
		// Names of the sub-actions.
		TArrayView<const FName> ElementNames;

		// The associated sub-elements.
		TArrayView<const FSchemaElement> Elements;
	};

	struct FSchemaOrExclusiveParameters
	{
		// Names of the sub-actions.
		TArrayView<const FName> ElementNames;

		// The associated sub-elements.
		TArrayView<const FSchemaElement> Elements;

		// Prior Probabilities of the sub-actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaOrInclusiveParameters
	{
		// Names of the sub-actions.
		TArrayView<const FName> ElementNames;

		// The associated sub-elements.
		TArrayView<const FSchemaElement> Elements;

		// Prior Probabilities of the sub-actions.
		TArrayView<const float> PriorProbabilities;
	};

	struct FSchemaArrayParameters
	{
		// The array sub-element.
		FSchemaElement Element;

		// The number of elements in the array.
		int32 Num = 0;
	};

	/** Activation Function to use for encoding */
	enum class EEncodingActivationFunction : uint8
	{
		ELU = 0,
		ReLU = 1,
		TanH = 2,
		GELU = 3,
	};

	static inline FString GetEncodingActivationAsString(const EEncodingActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case UE::Learning::Action::EEncodingActivationFunction::ELU: return TEXT("ELU");
		case UE::Learning::Action::EEncodingActivationFunction::ReLU: return TEXT("ReLU");
		case UE::Learning::Action::EEncodingActivationFunction::TanH: return TEXT("TanH");
		case UE::Learning::Action::EEncodingActivationFunction::GELU: return TEXT("GELU");
		default: checkNoEntry(); return TEXT("");
		}
	}

	struct FSchemaEncodingParameters
	{
		// The sub-element.
		FSchemaElement Element;

		// The size at which the sub-element should be encoded.
		int32 EncodingSize = 32;

		// The number of layers in the encoding
		int32 LayerNum = 1;

		// The activation function to use for encoding
		EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ELU;
	};

	/**
	 * Action Schema
	 *
	 * This object allows you to construct a description of the kind of actions you might want to be generated by a policy. Internally this object
	 * contains a pool of individual elements. This allows them to be constructed performantly and in a cache efficient way. This object is therefore
	 * required to access any data about the individual action elements that are created.
	 */
	struct FSchema
	{
		UE_API FSchemaElement CreateNull(const FName Tag = NAME_None);
		UE_API FSchemaElement CreateContinuous(const FSchemaContinuousParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateDiscreteExclusive(const FSchemaDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateDiscreteInclusive(const FSchemaDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateNamedDiscreteExclusive(const FSchemaNamedDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateNamedDiscreteInclusive(const FSchemaNamedDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateAnd(const FSchemaAndParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateOrExclusive(const FSchemaOrExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateOrInclusive(const FSchemaOrInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateArray(const FSchemaArrayParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateEncoding(const FSchemaEncodingParameters Parameters, const FName Tag = NAME_None);

		UE_API FSchemaContinuousParameters GetContinuous(const FSchemaElement Element) const;
		UE_API FSchemaDiscreteExclusiveParameters GetDiscreteExclusive(const FSchemaElement Element) const;
		UE_API FSchemaDiscreteInclusiveParameters GetDiscreteInclusive(const FSchemaElement Element) const;
		UE_API FSchemaNamedDiscreteExclusiveParameters GetNamedDiscreteExclusive(const FSchemaElement Element) const;
		UE_API FSchemaNamedDiscreteInclusiveParameters GetNamedDiscreteInclusive(const FSchemaElement Element) const;
		UE_API FSchemaAndParameters GetAnd(const FSchemaElement Element) const;
		UE_API FSchemaOrExclusiveParameters GetOrExclusive(const FSchemaElement Element) const;
		UE_API FSchemaOrInclusiveParameters GetOrInclusive(const FSchemaElement Element) const;
		UE_API FSchemaArrayParameters GetArray(const FSchemaElement Element) const;
		UE_API FSchemaEncodingParameters GetEncoding(const FSchemaElement Element) const;

		// Checks if the given element is valid
		UE_API bool IsValid(const FSchemaElement Element) const;

		// Gets the type of the given element
		UE_API EType GetType(const FSchemaElement Element) const;

		// Gets the tag of the given element
		UE_API FName GetTag(const FSchemaElement Element) const;

		// Get the encoded vector size of the given element
		UE_API int32 GetEncodedVectorSize(const FSchemaElement Element) const;

		// Get the action vector size of the given element
		UE_API int32 GetActionVectorSize(const FSchemaElement Element) const;

		// Get the action distribution vector size of the given element
		UE_API int32 GetActionDistributionVectorSize(const FSchemaElement Element) const;

		// Get the action modifier vector size of the given element
		UE_API int32 GetActionModifierVectorSize(const FSchemaElement Element) const;

		// Set the current generation
		UE_API void SetGeneration(const uint32 InGeneration);

		// Get the current generation
		UE_API uint32 GetGeneration() const;
		
		// Checks if the given schema is empty of elements.
		UE_API bool IsEmpty() const;

		// Empty all internal buffers of elements. This invalidates all existing elements.
		UE_API void Empty();

		// Reset all internal buffers (without freeing memory). This invalidates all existing elements.
		UE_API void Reset();

		// Serializes an action schema. Returns false if serialization fails.
		UE_API bool Serialize(FArchive& Ar);

	private:

		struct FContinuousData
		{
			int32 Num = INDEX_NONE;
			int32 NormalizationIndexOffset = INDEX_NONE;
			ENormalization NormalizationOption = ENormalization::Manual;
		};

		struct FDiscreteExclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
		};

		struct FDiscreteInclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
		};

		struct FNamedDiscreteExclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FNamedDiscreteInclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FAndData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FOrExclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
		};

		struct FOrInclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
			int32 PriorProbabilitiesOffset = INDEX_NONE;
		};

		struct FArrayData
		{
			int32 Num = INDEX_NONE;
			int32 ElementIndex = INDEX_NONE;
		};

		struct FEncodingData
		{
			int32 EncodingSize = INDEX_NONE;
			int32 ElementIndex = INDEX_NONE;
			int32 LayerNum = INDEX_NONE;
			EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ELU;
		};

		uint32 Generation = 0;

		/** These have entries for each Schema Element */
		TArray<EType> Types;
		TArray<FName> Tags;
		TArray<int32> EncodedVectorSizes;
		TArray<int32> ActionVectorSizes;
		TArray<int32> ActionDistributionVectorSizes;
		TArray<int32> ActionModifierVectorSizes;
		TArray<int32> TypeDataIndices;

		/** These are indexed based on the type and the index found in TypeDataIndices */
		TArray<FContinuousData> ContinuousData;
		TArray<FDiscreteExclusiveData> DiscreteExclusiveData;
		TArray<FDiscreteInclusiveData> DiscreteInclusiveData;
		TArray<FNamedDiscreteExclusiveData> NamedDiscreteExclusiveData;
		TArray<FNamedDiscreteInclusiveData> NamedDiscreteInclusiveData;
		TArray<FAndData> AndData;
		TArray<FOrExclusiveData> OrExclusiveData;
		TArray<FOrInclusiveData> OrInclusiveData;
		TArray<FArrayData> ArrayData;
		TArray<FEncodingData> EncodingData;

		/** This is an array of all the SubElements and their names, referenced by other elements */
		TArray<FName> SubElementNames;
		TArray<FSchemaElement> SubElementObjects;

		/** This is an array of all the prior probabilities, referenced by other elements. */
		TArray<float> PriorProbabilities;

		/** These are arrays for normalization values, referenced by continuous and flow matching elements. */
		TArray<float> NormalizationScales;
		TArray<float> NormalizationOffsets;
	};

	/**
	 * Action Object Element
	 *
	 * A single element in the action object representing part of an action. Internally this consists of a index used by the object to look up
	 * the associated action data and a generation id which can be used to check when this index is no longer valid.
	 */
	struct FObjectElement { int32 Index = INDEX_NONE; uint32 Generation = INDEX_NONE; };

	struct FObjectContinuousParameters
	{
		// Continuous action values
		TArrayView<const float> Values;
	};

	struct FObjectDiscreteExclusiveParameters
	{
		// Exclusive discrete action index
		int32 DiscreteIndex = INDEX_NONE;
	};

	struct FObjectDiscreteInclusiveParameters
	{
		// Inclusive discrete action indices
		TArrayView<const int32> DiscreteIndices;
	};

	struct FObjectNamedDiscreteExclusiveParameters
	{
		// Named Exclusive discrete action name
		FName ElementName = NAME_None;
	};

	struct FObjectNamedDiscreteInclusiveParameters
	{
		// Named Inclusive discrete action name
		TArrayView<const FName> ElementNames;
	};

	struct FObjectAndParameters
	{
		// Names of the sub-actions.
		TArrayView<const FName> ElementNames;

		// The associated sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectOrExclusiveParameters
	{
		// Name of the chosen sub-action.
		FName ElementName;

		// The associated chosen sub-element.
		FObjectElement Element;
	};

	struct FObjectOrInclusiveParameters
	{
		// Names of the chosen sub-actions.
		TArrayView<const FName> ElementNames;

		// The associated chosen sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectArrayParameters
	{
		// Array of sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectEncodingParameters
	{
		// Encoded sub-element.
		FObjectElement Element;
	};

	struct FObjectFlowMatchingParameters
	{
		// Flow matching float action values
		TArrayView<const float> Values;
	};

	/**
	 * Action Object
	 *
	 * This object allows you to construct or get data from an instance of an action you might have generated from a policy. Internally this object
	 * contains a pool of individual elements. This allows them to be constructed performantly and in a cache efficient way. This object is therefore
	 * required to access any data about the individual action elements that are created.
	 */
	struct FObject
	{
		UE_API FObjectElement CreateNull(const FName Tag = NAME_None);
		UE_API FObjectElement CreateContinuous(const FObjectContinuousParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateDiscreteExclusive(const FObjectDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateDiscreteInclusive(const FObjectDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateNamedDiscreteExclusive(const FObjectNamedDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateNamedDiscreteInclusive(const FObjectNamedDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateAnd(const FObjectAndParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateOrExclusive(const FObjectOrExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateOrInclusive(const FObjectOrInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateArray(const FObjectArrayParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateEncoding(const FObjectEncodingParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateFlowMatching(const FObjectFlowMatchingParameters Parameters, const FName Tag = NAME_None);

		UE_API FObjectContinuousParameters GetContinuous(const FObjectElement Element) const;
		UE_API FObjectDiscreteExclusiveParameters GetDiscreteExclusive(const FObjectElement Element) const;
		UE_API FObjectDiscreteInclusiveParameters GetDiscreteInclusive(const FObjectElement Element) const;
		UE_API FObjectNamedDiscreteExclusiveParameters GetNamedDiscreteExclusive(const FObjectElement Element) const;
		UE_API FObjectNamedDiscreteInclusiveParameters GetNamedDiscreteInclusive(const FObjectElement Element) const;
		UE_API FObjectAndParameters GetAnd(const FObjectElement Element) const;
		UE_API FObjectOrExclusiveParameters GetOrExclusive(const FObjectElement Element) const;
		UE_API FObjectOrInclusiveParameters GetOrInclusive(const FObjectElement Element) const;
		UE_API FObjectArrayParameters GetArray(const FObjectElement Element) const;
		UE_API FObjectEncodingParameters GetEncoding(const FObjectElement Element) const;
		UE_API FObjectFlowMatchingParameters GetFlowMatching(const FObjectElement Element) const;

		// Checks if the given element is valid
		UE_API bool IsValid(const FObjectElement Element) const;

		// Gets the type of the given element
		UE_API EType GetType(const FObjectElement Element) const;

		// Gets the tag of the given element
		UE_API FName GetTag(const FObjectElement Element) const;

		// Set the current generation
		UE_API void SetGeneration(const uint32 InGeneration);

		// Get the current generation
		UE_API uint32 GetGeneration() const;

		// Checks if the given object is empty of elements.
		UE_API bool IsEmpty() const;

		// Empty all internal buffers of elements. This invalidates all existing elements.
		UE_API void Empty();

		// Reset all internal buffers (without freeing memory). This invalidates all existing elements.
		UE_API void Reset();

		// Serializes an action object. Returns false if serialization fails.
		UE_API bool Serialize(FArchive& Ar);

	private:

		uint32 Generation = 0;

		TArray<EType> Types;
		TArray<FName> Tags;
		TArray<int32> ContinuousDataOffsets;
		TArray<int32> ContinuousDataNums;
		TArray<int32> DiscreteDataOffsets;
		TArray<int32> DiscreteDataNums;
		TArray<int32> ElementDataOffsets;
		TArray<int32> ElementDataNums;

		TArray<float> ContinuousValues;
		TArray<int32> DiscreteValues;
		TArray<FName> SubElementNames;
		TArray<FObjectElement> SubElementObjects;
	};


	/**
	 * Action Modifier Element
	 *
	 * A single element representing a modification to an action performed during sampling (such as masking). Internally this consists of a index 
	 * used by the object to look up the associated action modification data and a generation id which can be used to check when this index is no 
	 * longer valid.
	 */
	struct FModifierElement { int32 Index = INDEX_NONE; uint32 Generation = INDEX_NONE; };

	struct FModifierContinuousParameters
	{
		// Indicates which dimensions are masked
		TArrayView<const bool> Masked;

		// Values used to replace masked dimensions 
		TArrayView<const float> MaskedValues;
	};

	struct FModifierDiscreteExclusiveParameters
	{
		// Disallowed indices
		TArrayView<const int32> MaskedIndices;
	};

	struct FModifierDiscreteInclusiveParameters
	{
		// Disallowed indices
		TArrayView<const int32> MaskedIndices;
	};

	struct FModifierNamedDiscreteExclusiveParameters
	{
		// Disallowed actions
		TArrayView<const FName> MaskedElementNames;
	};

	struct FModifierNamedDiscreteInclusiveParameters
	{
		// Disallowed actions
		TArrayView<const FName> MaskedElementNames;
	};

	struct FModifierAndParameters
	{
		// Names of the modified sub-actions.
		TArrayView<const FName> ElementNames;

		// The modifiers of the associated sub-elements.
		TArrayView<const FModifierElement> Elements;
	};

	struct FModifierOrExclusiveParameters
	{
		// Names of the modified sub-action.
		TArrayView<const FName> ElementNames;

		// The modifiers of the associated sub-element.
		TArrayView<const FModifierElement> Elements;

		// Disallowed sub-elements
		TArrayView<const FName> MaskedElements;
	};

	struct FModifierOrInclusiveParameters
	{
		// Names of the modified sub-actions.
		TArrayView<const FName> ElementNames;

		// The modifiers of the associated sub-elements.
		TArrayView<const FModifierElement> Elements;

		// Disallowed sub-elements
		TArrayView<const FName> MaskedElements;
	};

	struct FModifierArrayParameters
	{
		// Array of sub-element modifiers.
		TArrayView<const FModifierElement> Elements;
	};

	struct FModifierEncodingParameters
	{
		// Encoded sub-element modifier.
		FModifierElement Element;
	};

	/**
	 * Action Modifier
	 *
	 * This object allows you to describe modifications you want to apply to actions during the sampling stage. Internally this object contains a 
	 * pool of individual elements. This allows them to be constructed performantly and in a cache efficient way. This object is therefore
	 * required to access any data about the individual action modifier elements that are created.
	 */
	struct FModifier
	{
		UE_API FModifierElement CreateNull(const FName Tag = NAME_None);
		UE_API FModifierElement CreateContinuous(const FModifierContinuousParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateDiscreteExclusive(const FModifierDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateDiscreteInclusive(const FModifierDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateNamedDiscreteExclusive(const FModifierNamedDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateNamedDiscreteInclusive(const FModifierNamedDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateAnd(const FModifierAndParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateOrExclusive(const FModifierOrExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateOrInclusive(const FModifierOrInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateArray(const FModifierArrayParameters Parameters, const FName Tag = NAME_None);
		UE_API FModifierElement CreateEncoding(const FModifierEncodingParameters Parameters, const FName Tag = NAME_None);

		UE_API FModifierContinuousParameters GetContinuous(const FModifierElement Element) const;
		UE_API FModifierDiscreteExclusiveParameters GetDiscreteExclusive(const FModifierElement Element) const;
		UE_API FModifierDiscreteInclusiveParameters GetDiscreteInclusive(const FModifierElement Element) const;
		UE_API FModifierNamedDiscreteExclusiveParameters GetNamedDiscreteExclusive(const FModifierElement Element) const;
		UE_API FModifierNamedDiscreteInclusiveParameters GetNamedDiscreteInclusive(const FModifierElement Element) const;
		UE_API FModifierAndParameters GetAnd(const FModifierElement Element) const;
		UE_API FModifierOrExclusiveParameters GetOrExclusive(const FModifierElement Element) const;
		UE_API FModifierOrInclusiveParameters GetOrInclusive(const FModifierElement Element) const;
		UE_API FModifierArrayParameters GetArray(const FModifierElement Element) const;
		UE_API FModifierEncodingParameters GetEncoding(const FModifierElement Element) const;

		// Checks if the given element is valid
		UE_API bool IsValid(const FModifierElement Element) const;

		// Gets the type of the given element
		UE_API EType GetType(const FModifierElement Element) const;

		// Gets the tag of the given element
		UE_API FName GetTag(const FModifierElement Element) const;

		// Set the current generation
		UE_API void SetGeneration(const uint32 InGeneration);

		// Get the current generation
		UE_API uint32 GetGeneration() const;

		// Checks if the given object is empty of elements.
		UE_API bool IsEmpty() const;

		// Empty all internal buffers of elements. This invalidates all existing elements.
		UE_API void Empty();

		// Reset all internal buffers (without freeing memory). This invalidates all existing elements.
		UE_API void Reset();

		// Serializes an action modifier object. Returns false if serialization fails.
		UE_API bool Serialize(FArchive& Ar);

	private:

		uint32 Generation = 0;

		TArray<EType> Types;
		TArray<FName> Tags;
		TArray<int32> ContinuousDataOffsets;
		TArray<int32> ContinuousDataNums;
		TArray<int32> DiscreteDataOffsets;
		TArray<int32> DiscreteDataNums;
		TArray<int32> ElementDataOffsets;
		TArray<int32> ElementDataNums;
		TArray<int32> MaskedDataOffsets;
		TArray<int32> MaskedDataNums;

		TArray<bool> ContinuousMaskeds;
		TArray<float> ContinuousMaskedValues;
		TArray<int32> DiscreteValues;
		TArray<FName> SubElementNames;
		TArray<FModifierElement> SubElementModifiers;
		TArray<FName> MaskedElementNames;
	};

	/**
	 * Gets a hash value representing object compatibility between schemas i.e. if objects from one schema can be used by objects expecting another 
	 * schema. This is not a cryptographic hash, and so `AreSchemaObjectsCompatible` should still be used as the ultimate source of truth.
	 * This function returns an int32 so that it can be used in blueprints.
	 * 
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param Salt						Hash salt
	 */
	UE_API int32 GetSchemaObjectsCompatibilityHash(
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const int32 Salt = 0x49710e77);

	/**
	 * Test if two schemas are compatible i.e. if objects from one schema can be used by objects expecting another schema. 
	 *
	 * @param SchemaA					First Schema
	 * @param SchemaElementA			First Schema Element
	 * @param SchemaB					Second Schema
	 * @param SchemaElementB			Second Schema Element
	 * @returns							true when the schemas are compatible
	 */
	UE_API bool AreSchemaObjectsCompatible(
		const FSchema& SchemaA,
		const FSchemaElement SchemaElementA,
		const FSchema& SchemaB,
		const FSchemaElement SchemaElementB);

	/** Network weight initialization type */
	enum class EWeightInitialization : uint8
	{
		KaimingGaussian = 0,
		KaimingUniform = 1,
	};

	/**
	 * Settings for building a network from a schema.
	 */
	struct FNetworkSettings
	{
		// If to use compressed linear layers. This reduces the memory usage by half at the cost of some small impact on quality and evaluation speed.
		bool bUseCompressedLinearLayers = false;

		// Which weight initialization to use.
		EWeightInitialization WeightInitialization = EWeightInitialization::KaimingGaussian;
	};

	/**
	 * Make a NNE::RuntimeBasic::FModelBuilderElement for the given Schema. This can be used if you want to plug the Decoder generated by this 
	 * schema as a part of a larger model you are making with a NNE::RuntimeBasic::FModelBuilder.
	 *
	 * @param OutElement				Output Builder Element
	 * @param Builder					Model Builder
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param NetworkSettings			Network Generation Settings
	 */
	UE_API void MakeDecoderNetworkModelBuilderElementFromSchema(
		NNE::RuntimeBasic::FModelBuilderElement& OutElement,
		NNE::RuntimeBasic::FModelBuilder& Builder,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings = FNetworkSettings());

	/**
	 * Generate FileData for a Neural Network that can act as a Decoder for the given schema. This network will take as input a vector of size
	 * EncodedActionSize and produce a vector of size ActionDistributionVectorSize.
	 *
	 * @param OutFileData				Output File Data to write to
	 * @param OutInputSize				Size of the vector this network takes as input
	 * @param OutOutputSize				Size of the vector this network takes as output
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param NetworkSettings			Network Generation Settings
	 * @param Seed						The random seed used in initializing the network weights
	 */
	UE_API void GenerateDecoderNetworkFileDataFromSchema(
		TArray<uint8>& OutFileData,
		uint32& OutInputSize,
		uint32& OutOutputSize,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings = FNetworkSettings(),
		const uint32 Seed = 0x05dbd1f5);

	/**
	 * Sample an action vector from a vector representing the action distribution.
	 *
	 * @param InOutRandomState			Random State/Seed to use when sampling the action vector
	 * @param OutActionVector			Output action vector
	 * @param ActionDistributionVector	Input Action Distribution vector
	 * @param ActionModifierVector		Input Action Modifier vector
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param ActionNoiseScale			Scale factor for the noise. Set this to zero to always choose the mean (expected) action.
	 */
	UE_API void SampleVectorFromDistributionVector(
		uint32& InOutRandomState,
		TLearningArrayView<1, float> OutActionVector,
		const TLearningArrayView<1, const float> ActionDistributionVector,
		const TLearningArrayView<1, const float> ActionModifierVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const float ActionNoiseScale = 1.0f);

	/**
	 * Convert an action object into an action vector.
	 *
	 * @param OutActionVector			Output action vector
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param Object					Action Object
	 * @param ObjectElement				Action Object Element
	 */
	UE_API void SetVectorFromObject(
		TLearningArrayView<1, float> OutActionVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FObject& Object,
		const FObjectElement ObjectElement);

	/**
	 * Convert an action vector into an action object
	 *
	 * @param OutObject					Output Action Object
	 * @param OutObjectElement			Output Action Object Element
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param ActionVector				Input Action vector
	 */
	UE_API void GetObjectFromVector(
		FObject& OutObject,
		FObjectElement& OutObjectElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ActionVector);

	/**
	 * Convert an action modifier into an action modifier vector.
	 *
	 * @param OutActionModifierVector	Output action modifier vector
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param Modifier					Action Modifier
	 * @param ModifierElement			Action Modifier Element
	 */
	UE_API void SetVectorFromModifier(
		TLearningArrayView<1, float> OutActionModifierVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FModifier& Modifier,
		const FModifierElement ModifierElement);

	/**
	 * Convert an action modifier vector into an action modifier
	 *
	 * @param OutModifier				Output Action Modifier
	 * @param OutModifierElement		Output Action Modifier Element
	 * @param Schema					Action Schema
	 * @param SchemaElement				Action Schema Element
	 * @param ActionModifierVector		Input Action Modifier vector
	 */
	UE_API void GetModifierFromVector(
		FModifier& OutModifier,
		FModifierElement& OutModifierElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ActionModifierVector);
}

UE_API FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FSchemaElement& SchemaElement);
UE_API FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FObjectElement& ObjectElement);
UE_API FArchive& operator<<(FArchive& Ar, UE::Learning::Action::FModifierElement& ModifierElement);
UE_API bool operator==(const UE::Learning::Action::FSchemaElement& Lhs, const UE::Learning::Action::FSchemaElement& Rhs);
UE_API bool operator==(const UE::Learning::Action::FObjectElement& Lhs, const UE::Learning::Action::FObjectElement& Rhs);
UE_API bool operator==(const UE::Learning::Action::FModifierElement& Lhs, const UE::Learning::Action::FModifierElement& Rhs);
UE_API uint32 GetTypeHash(const UE::Learning::Action::FSchemaElement& SchemaElement);
UE_API uint32 GetTypeHash(const UE::Learning::Action::FObjectElement& ObjectElement);
UE_API uint32 GetTypeHash(const UE::Learning::Action::FModifierElement& ModifierElement);

#undef UE_API
