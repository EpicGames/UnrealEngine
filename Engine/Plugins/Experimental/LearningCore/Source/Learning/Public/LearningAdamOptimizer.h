// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningOptimizer.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	struct FAdamOptimizerSettings
	{

		// The size of the std for finite differencing. This may need to be 
		// increased if the optimizer does not appear to be doing anything
		float FiniteDifferenceStd = 0.3f;

		// Update learning rate. Larger values may speed up convergence at the cost of instability.
		float LearningRate = 0.1f;

		// First moment momentum ratio. Reduce if seeing large swings in the loss
		float Beta1 = 0.9f;

		// Second moment momentum ratio. Reduce if seeing large swings in the loss
		float Beta2 = 0.999f;
	};

	/**
	* Adam Optimizer
	* 
	* Estimates gradient via finite differencing and does gradient descent. 
	* Roughly based on https://arxiv.org/abs/1910.06513
	*/
	struct FAdamOptimizer : public IOptimizer
	{
		UE_API FAdamOptimizer(const uint32 Seed, const FAdamOptimizerSettings& InSettings = FAdamOptimizerSettings());

		UE_API virtual void Resize(const int32 SampleNum, const int32 DimensionsNum) override final;

		UE_API virtual void Reset(
			TLearningArrayView<2, float> OutSamples,
			const TLearningArrayView<1, const float> InitialGuess) override final;

		UE_API virtual void Update(
			TLearningArrayView<2, float> InOutSamples,
			const TLearningArrayView<1, const float> Losses,
			const ELogSetting LogSettings = ELogSetting::Normal) override final;

	private:

		UE_API void Sample(TLearningArrayView<2, float> OutSamples);

		uint32 Seed = 0;
		FAdamOptimizerSettings Settings;

		TLearningArray<1, int32> Iterations;
		TLearningArray<1, float> Estimate;
		TLearningArray<1, float> Gradient;

		TLearningArray<1, float> M0;
		TLearningArray<1, float> M1;
		TLearningArray<1, float> M1HatMax;

		TLearningArray<2, float> GaussianSamples;
	};
}

#undef UE_API
