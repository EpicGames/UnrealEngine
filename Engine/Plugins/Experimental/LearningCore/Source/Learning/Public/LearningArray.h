// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Containers/UnrealString.h"

#include "MultiArrayView.h"
#include "MultiArray.h"

#define UE_API LEARNING_API

// If to enable ISPC code in Learning
#ifndef UE_LEARNING_ISPC
#define UE_LEARNING_ISPC INTEL_ISPC
//#define UE_LEARNING_ISPC 0
#endif

// Learning Multi-Dimensional Array Types

template<uint32 DimNum>
using TLearningArrayShape = TMultiArrayShape<DimNum>;

template<uint32 DimNum, typename ElementType>
using TLearningArrayView = TMultiArrayView<DimNum, ElementType>;

template<uint32 DimNum, typename ElementType>
using TLearningConstArrayView = TConstMultiArrayView<DimNum, ElementType>;

template<uint32 DimNum, typename ElementType, typename Allocator = FDefaultAllocator64>
using TLearningArray = TMultiArray<DimNum, ElementType, Allocator>;

// Learning Array Functions

namespace UE::Learning
{
	/**
	* Iterator for FIndexSet (see below)
	*/
	struct FIndexSetIterator
	{
		FIndexSetIterator(const int32* InPtr) : bUseIndex(false), Index(0), Ptr(InPtr) {}
		FIndexSetIterator(int32 InIndex) : bUseIndex(true), Index(InIndex), Ptr(nullptr) {}

		bool operator!=(const FIndexSetIterator& Rhs) const { return bUseIndex ? Index != Rhs.Index : Ptr != Rhs.Ptr; }
		FIndexSetIterator& operator++() { if (bUseIndex) { Index++; } else { Ptr++; }; return *this; }
		int32 operator*() const { return bUseIndex ? Index : *Ptr; }

	private:
		const bool bUseIndex = true;
		int32 Index = 0;
		const int32* Ptr = nullptr;
	};

	/**
	* FIndexSet represents a set of indices - more specifically, either a slice of indices, 
	* or an ArrayView of int32 which acts as the set of indices. This type is generally used 
	* when we have allocated arrays of data but only need to iterate over or update a subset. 
	* For example, if we allocate data for 100 instances, but only 10 of those instances are 
	* active we can use an FIndexSet to indicate which ones.
	* 
	* Having this type act as either a slice or array of int32 lets us write code once, but 
	* still allows the compiler to potentially generate both code paths (or we can write 
	* both paths manually, using ispc to be more efficient when we have a slice and don't 
	* require random reads/writes).
	* 
	* We can also attempt to automatically convert index sets which contain consecutive 
	* indices from arrays of int32 to slices using the `TryMakeSlice` function for more 
	* efficient processing.
	*/
	struct FIndexSet
	{
		FIndexSet() = default;
		FIndexSet(const int32 InSingleIndex) : SliceStart(InSingleIndex), SliceNum(1) {}
		FIndexSet(const int32 InSliceStart, const int32 InSliceNum) : SliceStart(InSliceStart), SliceNum(InSliceNum) {}
		FIndexSet(const TArrayView<const int32> InIndices) : Indices(InIndices) {}
		FIndexSet(const TLearningArrayView<1, const int32> InIndices) : Indices(InIndices) {}
		FIndexSet(const TArrayView<int32> InIndices) : Indices(InIndices) {}
		FIndexSet(const TLearningArrayView<1, int32> InIndices) : Indices(InIndices) {}
		FIndexSet(std::initializer_list<int32> InIndices) : Indices(InIndices) {}
		
		template<typename InAllocatorType>
		FIndexSet(const TArray<int32, InAllocatorType>& InIndices) : Indices(InIndices) {}

		template<typename InAllocatorType>
		FIndexSet(const TLearningArray<1, int32, InAllocatorType>& InIndices) : Indices(InIndices) {}

		FIndexSet& operator=(const int32 InSingleIndex) { SliceStart = InSingleIndex; SliceNum = 1; Indices = TLearningArrayView<1, const int32>(); return *this; }
		FIndexSet& operator=(const TArrayView<const int32> InIndices) { SliceStart = 0; SliceNum = 0; Indices = InIndices; return *this; }
		FIndexSet& operator=(TLearningArrayView<1, const int32> InIndices) { SliceStart = 0; SliceNum = 0; Indices = InIndices; return *this; }

		template<typename InAllocatorType>
		FIndexSet& operator=(const TArray<int32, InAllocatorType>& InIndices) { SliceStart = 0; SliceNum = 0; Indices = InIndices; return *this; }

		template<typename InAllocatorType>
		FIndexSet& operator=(const TLearningArray<1, int32, InAllocatorType>& InIndices) { SliceStart = 0; SliceNum = 0; Indices = InIndices; return *this; }

		TArray<int32> ToArray() const
		{
			const int32 IdxNum = Num();
			TArray<int32> Out;
			Out.SetNumUninitialized(IdxNum);
			for (int32 Idx = 0; Idx < IdxNum; Idx++)
			{
				Out[Idx] = (*this)[Idx];
			}
			return Out;
		}

		/**
		* Attempts to convert this IndexSet from a view of indices into a slice for more efficient
		* processing. This works by finding the minimum and maximum elements in the array and comparing 
		* the range of this min and max to the number of elements. When indices are consecutive `Max - Min + 1 == Num`.
		* 
		* Warning: this algorithm works on the assumption that there are no duplicate indices in the view. If there
		* are duplicate indices in your set this will not give the correct result.
		* 
		* @returns true if conversion to a slice is successful
		*/
		UE_API bool TryMakeSlice();

		inline int32 operator[](const int32 Index) const
		{
			return IsSlice() ? SliceStart + Index : Indices[Index];
		}

		inline int32 Num() const
		{
			return IsSlice() ? SliceNum : Indices.Num();
		}

		inline FIndexSet Slice(const int32 InStart, const int32 InNum) const
		{
			if (IsSlice())
			{
				check(InStart >= 0);
				check(InNum >= 0);
				check(InStart + InNum <= SliceNum);
			}

			return IsSlice() ? FIndexSet(SliceStart + InStart, InNum) : Indices.Slice(InStart, InNum);
		}

		inline bool Contains(const int32 Index) const
		{
			return IsSlice() ? (Index >= SliceStart && Index < SliceStart + SliceNum) : Indices.Contains(Index);
		}

		inline int32 Find(const int32 Index) const
		{
			if (IsSlice())
			{
				if (Index >= SliceStart && Index < SliceStart + SliceNum)
				{
					return Index - SliceStart;
				}
				else
				{
					return INDEX_NONE;
				}
			}
			else
			{
				const int32 IndexNum = Indices.Num();
				for (int32 Idx = 0; Idx < IndexNum; Idx++)
				{
					if (Indices[Idx] == Index) { return Idx; }
				}
				return INDEX_NONE;
			}
		}

		inline FIndexSetIterator begin() const
		{
			return IsSlice() ?
				FIndexSetIterator(SliceStart) :
				FIndexSetIterator(Indices.GetData());
		}

		inline FIndexSetIterator end() const
		{
			return IsSlice() ?
				FIndexSetIterator(SliceStart + SliceNum) :
				FIndexSetIterator(Indices.GetData() + Indices.Num());
		}

		inline bool IsSlice() const { return Indices.GetData() == nullptr; }
		inline int32 GetSliceStart() const { check(IsSlice()); return SliceStart; }
		inline int32 GetSliceNum() const { check(IsSlice()); return SliceNum; }
		inline TLearningArrayView<1, const int32> GetIndices() const { check(!IsSlice()); return Indices; }

	private:

		int32 SliceStart = 0, SliceNum = 0;
		TLearningArrayView<1, const int32> Indices;
	};

	/**
	* Similar to `ParallelFor` but instead of providing a single index to the callback it provides a slice for the
	* callback to loop over. The advantage here is that it gives the compiler a chance of optimizing the loop code
	* in the callback such as unrolling and auto-vectorizing. This is important when the number of items to loop
	* over is large (e.g. ~1000) since otherwise this would produce an individual function call for every element
	* in the loop.
	*
	* @param Num					Number of elements to iterate over
	* @param MinSliceElementNum		Minimum number of elements to include in each call to Body
	* @param Body					Callback taking start and stop index slice of Num
	*/
	UE_API void SlicedParallelFor(const int32 Num, const int32 MinSliceElementNum, const TFunctionRef<void(const int32 Start, const int32 Length)> Body);

	/**
	* Serialize an integer to bytes
	*/
	static inline void SerializeToBytes(int64& InOutOffset, TLearningArrayView<1, uint8> Bytes, const uint8 InValue)
	{
		check(InOutOffset + (int64)sizeof(uint8) <= Bytes.Num());
		FMemory::Memcpy(&Bytes[InOutOffset], (uint8*)&InValue, sizeof(uint8));
		InOutOffset += sizeof(uint8);
	}

	/**
	* Serialize an integer to bytes
	*/
	static inline void SerializeToBytes(int64& InOutOffset, TLearningArrayView<1, uint8> Bytes, const int32 InValue)
	{
		check(InOutOffset + (int64)sizeof(int32) <= Bytes.Num());
		FMemory::Memcpy(&Bytes[InOutOffset], (uint8*)&InValue, sizeof(int32));
		InOutOffset += sizeof(int32);
	}

	/**
	* Serialize an integer to bytes
	*/
	static inline void SerializeToBytes(int64& InOutOffset, TLearningArrayView<1, uint8> Bytes, const int64 InValue)
	{
		check(InOutOffset + (int64)sizeof(int64) <= Bytes.Num());
		FMemory::Memcpy(&Bytes[InOutOffset], (uint8*)&InValue, sizeof(int64));
		InOutOffset += sizeof(int64);
	}

	/**
	* Deserialize an integer from bytes
	*/
	static inline void DeserializeFromBytes(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, uint8& OutValue)
	{
		check(InOutOffset + (int64)sizeof(uint8) <= Bytes.Num());
		FMemory::Memcpy((uint8*)&OutValue, &Bytes[InOutOffset], sizeof(uint8));
		InOutOffset += sizeof(uint8);
	}

	/**
	* Deserialize an integer from bytes
	*/
	static inline void DeserializeFromBytes(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, int32& OutValue)
	{
		check(InOutOffset + (int64)sizeof(int32) <= Bytes.Num());
		FMemory::Memcpy((uint8*)&OutValue, &Bytes[InOutOffset], sizeof(int32));
		InOutOffset += sizeof(int32);
	}

	/**
	* Deserialize an integer from bytes
	*/
	static inline void DeserializeFromBytes(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, int64& OutValue)
	{
		check(InOutOffset + (int64)sizeof(int64) <= Bytes.Num());
		FMemory::Memcpy((uint8*)&OutValue, &Bytes[InOutOffset], sizeof(int64));
		InOutOffset += sizeof(int64);
	}

	/**
	* Deserialize an integer from bytes
	*/
	static inline void DeserializeFromBytes(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, uint64& OutValue)
	{
		check(InOutOffset + (int64)sizeof(uint64) <= Bytes.Num());
		FMemory::Memcpy((uint8*)&OutValue, &Bytes[InOutOffset], sizeof(uint64));
		InOutOffset += sizeof(uint64);
	}

	/**
	* Some additional functions that act on arrays such as copy, set, zero, etc.
	* 
	* Unfortunately most of these functions need to be duplicated for Array, ArrayView and their
	* const and non-const variations as template functions don't allow for implicit casts.
	*/
	namespace Array
	{
		/**
		* Check that two array shapes are equal.
		*/
		template<uint8 InDimNum>
		inline void CheckShapesEqual(const TLearningArrayShape<InDimNum>& Lhs, const TLearningArrayShape<InDimNum>& Rhs)
		{
			for (uint8 Idx = 0; Idx < InDimNum; Idx++)
			{
				checkf(Lhs[Idx] == Rhs[Idx],
					TEXT("Array Shapes don't match on dimension %i of %i (lhs: %lld, rhs: %lld)"), Idx + 1, InDimNum, Lhs[Idx], Rhs[Idx]);
			}
		}

		/**
		* Check that an array view does not contain any NaN, Inf, -Inf
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Check(const TLearningArrayView<InDimNum, InElementType> View)
		{
			for (int64 Idx = 0; Idx < View.Num(); Idx++)
			{
				checkf(
					FMath::IsFinite(View.GetData()[Idx]),
					TEXT("Invalid value %f found at flat array index %lld"), View.GetData()[Idx], Idx);
			}
		}

		/**
		* Check that an array does not contain any NaN, Inf, -Inf
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Check(const TLearningArray<InDimNum, InElementType, Allocator>& View)
		{
			for (int64 Idx = 0; Idx < View.Num(); Idx++)
			{
				checkf(
					FMath::IsFinite(View.GetData()[Idx]),
					TEXT("Invalid value %f found at flat array index %lld"), View.GetData()[Idx], Idx);
			}
		}

		/**
		* Check that an array view does not contain any NaN, Inf, -Inf
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Check(const TLearningArrayView<InDimNum, InElementType> View, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Check(View[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					checkf(
						FMath::IsFinite(View.GetData()[Idx]) && View.GetData()[Idx],
						TEXT("Invalid value %f found at flat array index %d"), View.GetData()[Idx], Idx);
				}
			}
		}

		/**
		* Check that an array view does not contain any NaN, Inf, -Inf
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Check(const TLearningArray<InDimNum, InElementType>& View, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Check(View[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					checkf(
						FMath::IsFinite(View.GetData()[Idx]),
						TEXT("Invalid value %f found at flat array index %lld"), View.GetData()[Idx], Idx);
				}
			}
		}

		// Specializations for when array data is float and size can be 32-bit

		UE_API void FloatArrayZero32(float* Data, const int32 Num);
		UE_API void FloatArraySet32(float* Data, const int32 Num, const float Value);
		UE_API void FloatArrayCopy32(float* Dst, float* Src, const int32 Num);

		/**
		* Zero the memory of an array view
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Zero(TLearningArrayView<InDimNum, InElementType> View)
		{
			FMemory::Memzero((void*)View.GetData(), View.Num() * sizeof(InElementType));
		}

		/**
		* Zero the memory of an array
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Zero(TLearningArray<InDimNum, InElementType, Allocator>& Array)
		{
			FMemory::Memzero((void*)Array.GetData(), Array.Num() * sizeof(InElementType));
		}

		/**
		* Zero the memory of an array view
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Zero(TLearningArrayView<InDimNum, InElementType> View, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Zero(View[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					FMemory::Memzero((void*)&View[Idx], sizeof(InElementType));
				}
			}
		}

		/**
		* Zero the memory of an array view
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Zero(TLearningArray<InDimNum, InElementType, Allocator>& View, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Zero(View[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					FMemory::Memzero((void*)&View[Idx], sizeof(InElementType));
				}
			}
		}

		/**
		* Zero the memory of an array view
		*/
		template<>
		inline void Zero<1, float>(TLearningArrayView<1, float> View)
		{
			if (View.NumBytes() < MAX_int32)
			{
				FloatArrayZero32(View.GetData(), View.Num());
			}
			else
			{
				FMemory::Memzero((void*)View.GetData(), View.Num() * sizeof(float));
			}
		}

		/**
		* Zero the memory of an array view
		*/
		template<>
		inline void Zero<2, float>(TLearningArrayView<2, float> View)
		{
			if (View.NumBytes() < MAX_int32)
			{
				FloatArrayZero32(View.GetData(), View.Num());
			}
			else
			{
				FMemory::Memzero((void*)View.GetData(), View.Num() * sizeof(float));
			}
		}

		/**
		* Set each item of an array view to the given element
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Set(TLearningArrayView<InDimNum, InElementType> View, const InElementType& Element)
		{
			for (int64 Idx = 0; Idx < View.Num(); Idx++)
			{
				View.GetData()[Idx] = Element;
			}
		}

		/**
		* Set each item of an array to the given element
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Set(TLearningArray<InDimNum, InElementType, Allocator>& View, const InElementType& Element)
		{
			for (int64 Idx = 0; Idx < View.Num(); Idx++)
			{
				View.GetData()[Idx] = Element;
			}
		}

		/**
		* Set each item of an array view to the given element
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Set(TLearningArrayView<InDimNum, InElementType> View, const InElementType& Element, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Set(View[Idx], Element);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					View[Idx] = Element;
				}
			}
		}

		/**
		* Set each item of an array to the given element
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Set(TLearningArray<InDimNum, InElementType, Allocator>& View, const InElementType& Element, const FIndexSet Indices)
		{
			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Set(View[Idx], Element);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					View[Idx] = Element;
				}
			}
		}

		/**
		* Set each item of an array view to the given element
		*/
		template<>
		inline void Set<1, float>(TLearningArrayView<1, float> View, const float& Element)
		{
			if (View.NumBytes() < MAX_int32)
			{
				FloatArraySet32(View.GetData(), View.Num(), Element);
			}
			else
			{
				for (int64 Idx = 0; Idx < View.Num(); Idx++)
				{
					View.GetData()[Idx] = Element;
				}
			}
		}

		/**
		* Set each item of an array view to the given element
		*/
		template<>
		inline void Set<2, float>(TLearningArrayView<2, float> View, const float& Element)
		{
			if (View.NumBytes() < MAX_int32)
			{
				FloatArraySet32(View.GetData(), View.Num(), Element);
			}
			else
			{
				for (int64 Idx = 0; Idx < View.Num(); Idx++)
				{
					View.GetData()[Idx] = Element;
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Copy(
			TLearningArrayView<InDimNum, InElementType> Dst,
			TLearningArrayView<InDimNum, InElementType> Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Copy(
			TLearningArrayView<InDimNum, InElementType> Dst,
			const TLearningArrayView<InDimNum, const InElementType> Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array view into another array
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Copy(
			TLearningArray<InDimNum, InElementType, Allocator>& Dst,
			const TLearningArrayView<InDimNum, const InElementType> Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array view into another array
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Copy(
			TLearningArray<InDimNum, InElementType, Allocator>& Dst,
			const TLearningArrayView<InDimNum, InElementType> Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array into another array view
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Copy(
			TLearningArrayView<InDimNum, InElementType> Dst,
			const TLearningArray<InDimNum, InElementType, Allocator>& Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array into another array
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorLhs, typename AllocatorRhs>
		inline void Copy(
			TLearningArray<InDimNum, InElementType, AllocatorLhs>& Dst,
			const TLearningArray<InDimNum, InElementType, AllocatorRhs>& Src)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
			{
				Dst.GetData()[Idx] = Src.GetData()[Idx];
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void Copy(
			TLearningArrayView<InDimNum, InElementType> Dst,
			const TLearningArrayView<InDimNum, const InElementType> Src,
			const FIndexSet Indices)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Copy(Dst[Idx], Src[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					Dst[Idx] = Src[Idx];
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Copy(
			TLearningArrayView<InDimNum, InElementType> Dst,
			const TLearningArray<InDimNum, InElementType, Allocator>& Src,
			const FIndexSet Indices)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Copy(Dst[Idx], Src[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					Dst[Idx] = Src[Idx];
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorLhs, typename AllocatorRhs>
		inline void Copy(
			TLearningArray<InDimNum, InElementType, AllocatorLhs>& Dst,
			const TLearningArray<InDimNum, InElementType, AllocatorRhs>& Src,
			const FIndexSet Indices)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Copy(Dst[Idx], Src[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					Dst[Idx] = Src[Idx];
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Copy(
			TLearningArray<InDimNum, InElementType, Allocator>& Dst,
			const TLearningArrayView<InDimNum, const InElementType> Src,
			const FIndexSet Indices)
		{
			CheckShapesEqual<InDimNum>(Dst.Shape(), Src.Shape());

			if constexpr (InDimNum > 1)
			{
				for (const int32 Idx : Indices)
				{
					Copy(Dst[Idx], Src[Idx]);
				}
			}
			else
			{
				for (const int32 Idx : Indices)
				{
					Dst[Idx] = Src[Idx];
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<>
		inline void Copy<1, float>(
			TLearningArrayView<1, float> Dst,
			TLearningArrayView<1, float> Src)
		{
			CheckShapesEqual<1>(Dst.Shape(), Src.Shape());

			if (Src.NumBytes() < MAX_int32)
			{
				FloatArrayCopy32(Dst.GetData(), Src.GetData(), Dst.Num());
			}
			else
			{
				for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
				{
					Dst.GetData()[Idx] = Src.GetData()[Idx];
				}
			}
		}

		/**
		* Copy the contents of one array view into another
		*/
		template<>
		inline void Copy<2, float>(
			TLearningArrayView<2, float> Dst,
			TLearningArrayView<2, float> Src)
		{
			CheckShapesEqual<2>(Dst.Shape(), Src.Shape());

			if (Src.NumBytes() < MAX_int32)
			{
				FloatArrayCopy32(Dst.GetData(), Src.GetData(), Dst.Num());
			}
			else
			{
				for (int64 Idx = 0; Idx < Dst.Num(); Idx++)
				{
					Dst.GetData()[Idx] = Src.GetData()[Idx];
				}
			}
		}

		/**
		* Check if the shapes of two arrays are equal
		*/
		template<uint8 InDimNum>
		inline bool ShapeEqual(
			const TLearningArrayShape<InDimNum>& Lhs,
			const TLearningArrayShape<InDimNum>& Rhs)
		{
			for (uint32 Idx = 0; Idx < InDimNum; Idx++)
			{
				if (Lhs[Idx] != Rhs[Idx]) { return false; }
			}

			return true;
		}

		/**
		* Check if the contents of one array equals the contents of another.
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorLhs, typename AllocatorRhs>
		inline bool Equal(
			const TLearningArray<InDimNum, InElementType, AllocatorLhs>& Lhs,
			const TLearningArray<InDimNum, InElementType, AllocatorRhs>& Rhs)
		{
			CheckShapesEqual<InDimNum>(Lhs.Shape(), Rhs.Shape());

			for (int64 Idx = 0; Idx < Lhs.Num(); Idx++)
			{
				if (Lhs.GetData()[Idx] != Rhs.GetData()[Idx])
				{
					return false;
				}
			}

			return true;
		}

		/**
		* Check if the contents of one array equals the contents of another.
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorRhs>
		inline bool Equal(
			const TLearningArrayView<InDimNum, const InElementType> Lhs,
			const TLearningArray<InDimNum, InElementType, AllocatorRhs>& Rhs)
		{
			CheckShapesEqual<InDimNum>(Lhs.Shape(), Rhs.Shape());

			for (int64 Idx = 0; Idx < Lhs.Num(); Idx++)
			{
				if (Lhs.GetData()[Idx] != Rhs.GetData()[Idx])
				{
					return false;
				}
			}

			return true;
		}

		/**
		* Check if the contents of one array equals the contents of another.
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorRhs>
		inline bool Equal(
			const TLearningArrayView<InDimNum, InElementType> Lhs,
			const TLearningArray<InDimNum, InElementType, AllocatorRhs>& Rhs)
		{
			CheckShapesEqual<InDimNum>(Lhs.Shape(), Rhs.Shape());

			for (int64 Idx = 0; Idx < Lhs.Num(); Idx++)
			{
				if (Lhs.GetData()[Idx] != Rhs.GetData()[Idx])
				{
					return false;
				}
			}

			return true;
		}

		/**
		* Check if the contents of one array equals the contents of another.
		*/
		template<uint8 InDimNum, typename InElementType, typename AllocatorLhs>
		inline bool Equal(
			const TLearningArray<InDimNum, InElementType, AllocatorLhs>& Lhs,
			const TLearningArrayView<InDimNum, const InElementType>& Rhs)
		{
			CheckShapesEqual<InDimNum>(Lhs.Shape(), Rhs.Shape());

			for (int64 Idx = 0; Idx < Lhs.Num(); Idx++)
			{
				if (Lhs.GetData()[Idx] != Rhs.GetData()[Idx])
				{
					return false;
				}
			}

			return true;
		}

		/**
		* Check if the contents of one array equals the contents of another.
		*/
		template<uint8 InDimNum, typename InElementType>
		inline bool Equal(
			const TLearningArrayView<InDimNum, const InElementType> Lhs,
			const TLearningArrayView<InDimNum, const InElementType> Rhs)
		{
			CheckShapesEqual<InDimNum>(Lhs.Shape(), Rhs.Shape());

			for (int64 Idx = 0; Idx < Lhs.Num(); Idx++)
			{
				if (Lhs.GetData()[Idx] != Rhs.GetData()[Idx])
				{
					return false;
				}
			}

			return true;
		}

		/**
		* Shift all elements in an array to the left. Data at the end of the array will remain the same.
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void ShiftLeft(TLearningArray<InDimNum, InElementType, Allocator>& Array, const int64 ShiftNum)
		{
			for (int64 Idx = 0; Idx < (int64)Array.template Num<0>() - ShiftNum; Idx++)
			{
				Copy(Array[Idx], Array[Idx + ShiftNum]);
			}
		}

		/**
		* Shift all elements in an array to the left. Data at the end of the array will remain the same.
		*/
		template<typename InElementType, typename Allocator>
		inline void ShiftLeft(TLearningArray<1, InElementType, Allocator>& Array, const int64 ShiftNum)
		{
			for (int64 Idx = 0; Idx < (int64)Array.Num() - ShiftNum; Idx++)
			{
				Array.GetData()[Idx] = Array.GetData()[Idx + ShiftNum];
			}
		}

		/**
		* Shift all elements in an array to the left. Data at the end of the array will remain the same.
		*/
		template<uint8 InDimNum, typename InElementType>
		inline void ShiftLeft(TLearningArrayView<InDimNum, InElementType> Array, const int64 ShiftNum)
		{
			for (int64 Idx = 0; Idx < (int64)Array.template Num<0>() - ShiftNum; Idx++)
			{
				Copy(Array[Idx], Array[Idx + ShiftNum]);
			}
		}

		/**
		* Shift all elements in an array to the left. Data at the end of the array will remain the same.
		*/
		template<typename InElementType>
		inline void ShiftLeft(TLearningArrayView<1, InElementType> Array, const int64 ShiftNum)
		{
			for (int64 Idx = 0; Idx < (int64)Array.Num() - ShiftNum; Idx++)
			{
				Array.GetData()[Idx] = Array.GetData()[Idx + ShiftNum];
			}
		}

		/**
		* Serialize an array
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void Serialize(FArchive& Ar, TLearningArray<InDimNum, InElementType, Allocator>& Array)
		{
			if (Ar.IsLoading())
			{
				TLearningArrayShape<InDimNum> Shape;
				for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
				{
					Ar << Shape[ShapeIdx];
				}
				Array.SetNumUninitialized(Shape);
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(InElementType));
			}
			else if (Ar.IsSaving())
			{
				for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
				{
					int64 Num = Array.Num(ShapeIdx);
					Ar << Num;
				}
				Ar.Serialize(Array.GetData(), Array.Num() * sizeof(InElementType));
			}
		}

		/**
		* Serialization byte num
		*/
		template<uint8 InDimNum, typename InElementType>
		inline int32 SerializationByteNum32(const TLearningArrayShape<InDimNum> Shape)
		{
			return sizeof(int32) + sizeof(int32) * InDimNum + sizeof(InElementType) * Shape.Total();
		}

		/**
		* Serialize an array to bytes
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void SerializeToBytes32(int64& InOutOffset, TLearningArrayView<1, uint8> Bytes, const TMultiArrayShape<InDimNum> Shape, const TArray<InElementType, Allocator>& InArray)
		{
			Learning::SerializeToBytes(InOutOffset, Bytes, (int32)InDimNum);
			for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
			{
				Learning::SerializeToBytes(InOutOffset, Bytes, (int32)Shape[ShapeIdx]);
			}

			check(InOutOffset + (int64)sizeof(InElementType) * InArray.Num() <= Bytes.Num());
			FMemory::Memcpy(&Bytes[InOutOffset], (uint8*)InArray.GetData(), sizeof(InElementType) * InArray.Num());
			InOutOffset += sizeof(InElementType) * InArray.Num();
		}

		/**
		* Serialize an array to bytes
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void SerializeToBytes32(int64& InOutOffset, TLearningArrayView<1, uint8> Bytes, const TLearningArray<InDimNum, InElementType, Allocator>& InArray)
		{
			Learning::SerializeToBytes(InOutOffset, Bytes, (int32)InDimNum);
			for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
			{
				Learning::SerializeToBytes(InOutOffset, Bytes, (int32)InArray.Shape()[ShapeIdx]);
			}

			check(InOutOffset + (int64)sizeof(InElementType) * InArray.Num() <= Bytes.Num());
			FMemory::Memcpy(&Bytes[InOutOffset], (uint8*)InArray.GetData(), sizeof(InElementType) * InArray.Num());
			InOutOffset += sizeof(InElementType) * InArray.Num();
		}

		/**
		* Deserialize an array from bytes
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void DeserializeFromBytes32(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, TArray<InElementType, Allocator>& OutArray)
		{
			int32 DimNum = INDEX_NONE;
			Learning::DeserializeFromBytes(InOutOffset, Bytes, DimNum);
			check(DimNum == InDimNum);

			TLearningArrayShape<InDimNum> Shape;
			for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
			{
				int32 Shape32 = 0;
				Learning::DeserializeFromBytes(InOutOffset, Bytes, Shape32);
				Shape[ShapeIdx] = Shape32;
			}

			OutArray.SetNumUninitialized(Shape.Total());

			check(InOutOffset + (int64)sizeof(InElementType) * OutArray.Num() <= Bytes.Num());
			FMemory::Memcpy((uint8*)OutArray.GetData(), &Bytes[InOutOffset], sizeof(InElementType) * OutArray.Num());
			InOutOffset += sizeof(InElementType) * OutArray.Num();
		}

		/**
		* Deserialize an array from bytes
		*/
		template<uint8 InDimNum, typename InElementType, typename Allocator>
		inline void DeserializeFromBytes32(int64& InOutOffset, TLearningArrayView<1, const uint8> Bytes, TLearningArray<InDimNum, InElementType, Allocator>& OutArray)
		{
			int32 DimNum = INDEX_NONE;
			Learning::DeserializeFromBytes(InOutOffset, Bytes, DimNum);
			check(DimNum == InDimNum);

			TLearningArrayShape<InDimNum> Shape;
			for (uint8 ShapeIdx = 0; ShapeIdx < InDimNum; ShapeIdx++)
			{
				int32 Shape32 = 0;
				Learning::DeserializeFromBytes(InOutOffset, Bytes, Shape32);
				Shape[ShapeIdx] = Shape32;
			}

			OutArray.SetNumUninitialized(Shape);

			check(InOutOffset + (int64)sizeof(InElementType) * OutArray.Num() <= Bytes.Num());
			FMemory::Memcpy((uint8*)OutArray.GetData(), &Bytes[InOutOffset],  sizeof(InElementType) * OutArray.Num());
			InOutOffset += sizeof(InElementType) * OutArray.Num();
		}

		template<typename InElementType>
		inline int64 IndexOf(const TLearningArrayView<1, const InElementType> Array, const InElementType& Element)
		{
			for (int64 Idx = 0; Idx < Array.Num(); Idx++)
			{
				if (Array.GetData()[Idx] == Element)
				{
					return Idx;
				}
			}
			return INDEX_NONE;
		}

		template<typename InElementType>
		inline int64 IndexOf(const TLearningArray<1, InElementType>& Array, const InElementType& Element)
		{
			for (int64 Idx = 0; Idx < Array.Num(); Idx++)
			{
				if (Array.GetData()[Idx] == Element)
				{
					return Idx;
				}
			}
			return INDEX_NONE;
		}

		template<typename InElementType>
		FString Format(const TLearningArrayView<1, const InElementType> Array, const TFunctionRef<FString(const InElementType&)> Formatter, const int32 MaxItemNum = 16)
		{
			const int64 ItemNum = Array.Num();

			FString Output = TEXT("[");

			if (ItemNum <= MaxItemNum)
			{
				for (int64 Idx = 0; Idx < ItemNum; Idx++)
				{
					Output.Append(Formatter(Array[Idx]));
					if (Idx != ItemNum - 1) { Output.Append(TEXT(" ")); }
				}
			}
			else
			{
				for (int64 Idx = 0; Idx < MaxItemNum / 2; Idx++)
				{
					Output.Append(Formatter(Array[Idx]));
					Output.Append(TEXT(" "));
				}

				Output.Append(TEXT("... "));

				for (int64 Idx = 0; Idx < MaxItemNum / 2; Idx++)
				{
					Output.Append(Formatter(Array[ItemNum - Idx - 1]));
					if (Idx != MaxItemNum / 2 - 1) { Output.Append(TEXT(" ")); }
				}
			}

			Output.Append(TEXT("]"));

			return Output;
		}

		static inline FString FormatFloat(const TLearningArrayView<1, const float> Array, const int32 MaxItemNum = 16)
		{
			return Format<float>(Array, [](const float& Value) { return FString::Printf(TEXT("%6.3f"), Value); }, MaxItemNum);
		}

		static inline FString FormatInt32(const TLearningArrayView<1, const int32> Array, const int32 MaxItemNum = 16)
		{
			return Format<int32>(Array, [](const int32& Value) { return FString::Printf(TEXT("%i"), Value); }, MaxItemNum);
		}

		static inline FString FormatUint64(const TLearningArrayView<1, const uint64> Array, const int32 MaxItemNum = 16)
		{
			return Format<uint64>(Array, [](const uint64& Value) { return FString::Printf(TEXT("%llu"), Value); }, MaxItemNum);
		}
	}
}


#undef UE_API
