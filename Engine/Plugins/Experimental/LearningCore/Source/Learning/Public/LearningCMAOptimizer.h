// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningOptimizer.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Settings for the CMA Optimizer
	*/
	struct FCMAOptimizerSettings
	{
		// The size of the initial exploration region for CMA. 
		// Larger introduces more exploration but may have worse initial results or cause longer convergence
		float InitialStepSize = 0.3f;

		// What ratio of samples are used to update the covariance matrix and how many are discarded. 
		// Larger values can speed up convergence on flat gradients but will make things worse on complex loss landscapes
		float SurvialRatio = 0.5f;
	};

	/**
	* CMA Optimizer
	* 
	* Black box optimizer based on Covariance Matrix Adaption
	*/
	struct FCMAOptimizer : public IOptimizer
	{
		UE_API FCMAOptimizer(const uint32 Seed, const FCMAOptimizerSettings& InSettings = FCMAOptimizerSettings());

		UE_API virtual void Resize(const int32 SampleNum, const int32 DimensionsNum) override final;

		UE_API virtual void Reset(
			TLearningArrayView<2, float> OutSamples,
			const TLearningArrayView<1, const float> InitialGuess) override final;

		UE_API virtual void Update(
			TLearningArrayView<2, float> InOutSamples,
			const TLearningArrayView<1, const float> Losses,
			const ELogSetting LogSettings = ELogSetting::Normal) override final;

		static UE_API int32 DefaultSampleNum(const int32 DimensionNum);

	private:

		UE_API void Sample(TLearningArrayView<2, float> OutSamples);

		uint32 Seed = 0;
		FCMAOptimizerSettings Settings;

		int32 Iterations = 0;
		float Sigma = 0.3f;
		TLearningArray<1, float> PathSigma;
		TLearningArray<1, float> PathCovariance;
		TLearningArray<1, float> Mean;
		TLearningArray<2, float> Covariance;

		int32 Mu = 0;
		float MuEff = 0.0f;
		TLearningArray<1, float> Weights;

		TLearningArray<1, float> OldMean;
		TLearningArray<1, int32> LossRanking;
		TLearningArray<1, float> UpdateDirection;
		TLearningArray<2, float> CovarianceTransform;
		TLearningArray<2, float> CovarianceInverseSqrt;

		TLearningArray<2, float> GaussianSamples;
	};


}

#undef UE_API
