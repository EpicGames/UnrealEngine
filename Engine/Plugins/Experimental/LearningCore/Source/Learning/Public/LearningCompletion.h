// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Completion Mode
	*/
	enum class ECompletionMode : uint8
	{
		// Episode is still running
		Running = 0,

		// Episode ended early but was still in progress. Value function will be used to estimate final return.
		Truncated = 1,

		// Episode ended early and zero reward was expected for all future steps.
		Terminated = 2,
	};

	/**
	* This object stores essentially a list of instances that need to be reset.
	* It can be filled based on the completion status or manually.
	*/
	struct FResetInstanceBuffer
	{
		UE_API void Reserve(const int32 InMaxInstanceNum);

		UE_API void SetResetInstances(const FIndexSet Instances);
		UE_API void SetResetInstancesFromCompletions(const TLearningArrayView<1, const ECompletionMode> Completions, const FIndexSet Instances);
		UE_API const int32 GetResetInstanceNum() const;
		UE_API const FIndexSet GetResetInstances() const;
		UE_API const TArray<int32>& GetResetInstancesArray() const;

	private:

		FIndexSet ResetInstancesSet;
		TArray<int32> ResetInstances;
	};

	namespace Completion
	{
		/**
		* Converts a ECompletionMode into a string.
		*/
		UE_API const TCHAR* CompletionModeString(const ECompletionMode Completion);

		/**
		* Takes the logical Or of completions. More specifically, if either completion is `Terminated` that 
		* will be the result of this operator, otherwise either completion being `Truncated` takes priority 
		* over either being `Running`. Put simply: Terminated > Truncated > Running
		*/
		UE_API ECompletionMode Or(const ECompletionMode Lhs, const ECompletionMode Rhs);

		/**
		* Set completions for all instances whose episode has reached the max number of steps.
		*
		* @param OutCompletions					Output buffer to write completions to
		* @param EpisodeStepNums				Number of steps taken by each instance
		* @param EpisodeMaxStepNum				Maximum number of allowed steps
		* @param Instances						Instances to process
		*
		*/
		UE_API void EvaluateEndOfEpisodeCompletions(
			TLearningArrayView<1, ECompletionMode> OutCompletions,
			const TLearningArrayView<1, const int32> EpisodeStepNums,
			const int32 EpisodeMaxStepNum,
			const FIndexSet Instances);

	}
}


#undef UE_API
