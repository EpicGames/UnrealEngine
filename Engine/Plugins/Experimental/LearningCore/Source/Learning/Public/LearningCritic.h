// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningNeuralNetwork.h" // Included for FNeuralNetworkInferenceSettings

#include "Templates/SharedPointer.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Neural-network based critic object. Stores various settings and intermediate
	* storage required to evaluate the given network for the provided number of instances.
	*/
	struct FNeuralNetworkCritic
	{
		/**
		 * Create a new Critic Object which takes the encoded observation state, and the memory state, and outputs the expected discounted returns.
		 *
		 * @param InMaxInstanceNum				Maximum number of instances
		 * @param InObservationEncodedNum		Size of the encoded observation vector
		 * @param InMemoryStateNum				Size of the memory state
		 * @param InNeuralNetwork				Neural network object
		 * @param InInferenceSettings			Inference settings
		 */
		UE_API FNeuralNetworkCritic(
			const int32 InMaxInstanceNum,
			const int32 InObservationEncodedNum,
			const int32 InMemoryStateNum,
			const TSharedPtr<FNeuralNetwork>& InNeuralNetwork,
			const FNeuralNetworkInferenceSettings& InInferenceSettings = FNeuralNetworkInferenceSettings());

		/**
		 * Evaluate the Critic
		 * 
		 * @param OutputReturns						Output expected discounted returns of shape (MaxInstanceNum)
		 * @param InputObservationVectorsEncoded	Input encoded observation vectors of shape (MaxInstanceNum, ObservationEncodedNum)
		 * @param InputMemoryState					Input memory state of shape (MaxInstanceNum, MemoryStateNum)
		 * @param Instances							Set of instances to evaluate
		 */
		UE_API void Evaluate(
			TLearningArrayView<1, float> OutputReturns,
			const TLearningArrayView<2, const float> InputObservationVectorsEncoded,
			const TLearningArrayView<2, const float> InputMemoryState,
			const FIndexSet Instances);

		/** Sets the NeuralNetwork and re-creates the NeuralNetworkInference object */
		UE_API void UpdateNeuralNetwork(const TSharedPtr<FNeuralNetwork>& NewNeuralNetwork);

		/** Gets the NeuralNetwork associated with this Critic */
		UE_API const TSharedPtr<FNeuralNetwork>& GetNeuralNetwork() const;

	private:

		int32 ObservationEncodedNum = 0;
		int32 MemoryStateNum = 0;
		TSharedPtr<FNeuralNetworkFunction> NeuralNetworkFunction;

		TLearningArray<2, float> Input;
	};
}

#undef UE_API
