// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningFrameRangeSet.h"

#include "Math/Vector.h"
#include "Math/Quat.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	 * A FFrameAttribute represents a attribute associated with every frame in a FFrameRangeSet. An attribute is made up of multiple "channels" 
	 * such as the X, Y, Z components of a location.
	 * 
	 * The data for the attribute is stored in one large flat array of shape (ChannelNum, TotalFrameNum). This means the data is stored in a way
	 * designed for SoA access by default. Some helper functions are provided for accessing slices of this data according to various range properties.
	 * 
	 * Also provided are many "operators" which allow for batched operations on attributes to create new attributes such as adding or subtracting 
	 * attribute values. This operations are very efficient as they are computed across all frames in the attribute in batch and can be optimized 
	 * using ISPC. If you perform a binary operation on two channels with different frames in the range sets then it will construct a new channel
	 * which is the intersection of those two inputs.
	 */
	struct FFrameAttribute
	{
	public:

		/** Check if the FrameAttribute is well-formed. */
		UE_API void Check() const;

		/** True if the FrameAttribute is Empty, otherwise false */
		UE_API bool IsEmpty() const;

		/** Empties the FrameAttribute */
		UE_API void Empty();

		/** Gets the internal FrameRangeSet associated to this attribute */
		UE_API const FFrameRangeSet& GetFrameRangeSet() const;

		/** Gets the total number of frames for this attribute */
		UE_API int32 GetTotalFrameNum() const;

		/** Gets the total number of ranges for this attribute */
		UE_API int32 GetTotalRangeNum() const;

		/** Gets the number of channels in this attribute */
		UE_API int32 GetChannelNum() const;

		/** Gets a view of the complete large array of attribute data stored as (ChannelNum, TotalFrameNum) */
		UE_API TLearningArrayView<2, const float> GetAttributeData() const;

		/** Gets a view of the complete attribute data for a single channel */
		UE_API TLearningArrayView<1, const float> GetChannelAttributeData(const int32 ChannelIdx) const;

		/** Gets the attribute value for a given channel and frame index */
		UE_API const float& GetChannelAttributeDataAtFrame(const int32 ChannelIdx, const int32 FrameIdx) const;

		/** Gets the attribute data associated with a single channel, and range */
		UE_API TLearningArrayView<1, const float> GetChannelEntryRangeAttributeData(const int32 ChannelIdx, const int32 EntryIdx, const int32 RangeIdx) const;

		/** Gets the attribute data associated with range offset and length */
		UE_API TLearningArrayView<1, const float> GetChannelRangeAttributeData(const int32 ChannelIdx, const int32 RangeOffset, const int32 RangeLength) const;

		/** Gets a view of the complete large array of attribute data stored as (ChannelNum, TotalFrameNum) */
		UE_API TLearningArrayView<2, float> GetAttributeData();

		/** Gets a view of the complete attribute data for a single channel */
		UE_API TLearningArrayView<1, float> GetChannelAttributeData(const int32 ChannelIdx);

		/** Gets the attribute value for a given channel and frame index */
		UE_API float& GetChannelAttributeDataAtFrame(const int32 ChannelIdx, const int32 FrameIdx);

		/** Gets the attribute data associated with a single channel, and range */
		UE_API TLearningArrayView<1, float> GetChannelEntryRangeAttributeData(const int32 ChannelIdx, const int32 EntryIdx, const int32 RangeIdx);

		/** Gets the attribute data associated with range offset and length */
		UE_API TLearningArrayView<1, float> GetChannelRangeAttributeData(const int32 ChannelIdx, const int32 RangeOffset, const int32 RangeLength);

	public:

		/** The internal associate FrameRangeSet */
		FFrameRangeSet FrameRangeSet;

		/** The large flat array of attribute data of shape (ChannelNum, TotalFrameNum) */
		TLearningArray<2, float> AttributeData;
	};

	namespace FrameAttribute
	{
		/** Reduce op function. Takes as input a single frame attribute and a set of ranges and lengths for that attribute. */
		using ReduceOpFunction = TFunctionRef<void(
			const FFrameAttribute& In,
			const TLearningArrayView<1, const int32> RangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** Nullary op function. Produces as output a single frame attribute given a set of ranges and lengths for that attribute. */
		using NullaryOpFunction = TFunctionRef<void(
			FFrameAttribute& Out,
			const TLearningArrayView<1, const int32> RangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** Unary op function. Takes a single frame attribute as input and produces as output a single frame attribute. */
		using UnaryOpFunction = TFunctionRef<void(
			FFrameAttribute& Out,
			const FFrameAttribute& In,
			const TLearningArrayView<1, const int32> RangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** In-place op function. Takes a single frame attribute and modifies it in-place. */
		using InplaceOpFunction = TFunctionRef<void(
			FFrameAttribute& InOut,
			const TLearningArrayView<1, const int32> RangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** Binary op function. Takes a two frame attributes as input and produces as output a single frame attribute. */
		using BinaryOpFunction = TFunctionRef<void(
			FFrameAttribute& Out,
			const FFrameAttribute& Lhs,
			const FFrameAttribute& Rhs,
			const TLearningArrayView<1, const int32> OutRangeOffsets,
			const TLearningArrayView<1, const int32> LhsRangeOffsets,
			const TLearningArrayView<1, const int32> RhsRangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** Convenience type to make the definition and usage of NaryOp easier */
		using ConstFrameAttributePtr = const FFrameAttribute*;

		/** Nary op function. Takes a multiple frame attributes as input and produces as output a single frame attribute. */
		using NaryOpFunction = TFunctionRef<void(
			FFrameAttribute& Out,
			const TArrayView<const ConstFrameAttributePtr> In,
			const TLearningArrayView<1, const int32> OutRangeOffsets,
			const TArrayView<const TLearningArrayView<1, const int32>> InRangeOffsets,
			const TLearningArrayView<1, const int32> RangeLengths)>;

		/** Checks if two frame attributes are equal */
		UE_API bool Equal(const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the intersection of a frame attribute and frame range set */
		UE_API void Intersection(FFrameAttribute& OutFrameAttribute, const FFrameAttribute& FrameAttribute, const FFrameRangeSet& FrameRangeSet);

		/** Computes the frame range set where the given channel is non-zero in the frame attribute. */
		UE_API void NonZeroFrameRangeSet(FFrameRangeSet& OutFrameRangeSet, const FFrameAttribute& FrameAttribute, const int32 ChannelIdx);

		/** Concatenates the given frame attributes together. */
		UE_API void Concat(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> In);

		/** Computes a reduction on a frame attribute */
		UE_API void ReduceOp(
			const FFrameAttribute& In,
			const ReduceOpFunction Op);

		/** Creates a frame attribute from zero arguments and the given op */
		UE_API void NullaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameRangeSet& FrameRangeSet,
			const NullaryOpFunction Op);

		/** Creates a frame attribute from another and the given op */
		UE_API void UnaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameAttribute& In,
			const UnaryOpFunction Op);

		/** Modifies a frame attribute in-place using the given op */
		UE_API void InplaceOp(
			FFrameAttribute& InOut,
			const InplaceOpFunction Op);

		/** Creates a frame attribute from two others and the given op. Performs an intersection of the Lhs and Rhs if they do not match. */
		UE_API void BinaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const FFrameAttribute& Lhs,
			const FFrameAttribute& Rhs,
			const BinaryOpFunction Op);

		/** Creates a frame attribute from multiple others and the given op. */
		UE_API void NaryOp(
			FFrameAttribute& Out,
			const int32 OutChannelNum,
			const TArrayView<const ConstFrameAttributePtr> Inputs,
			const NaryOpFunction Op);


		/** Find the channel and frame with the smallest value */
		UE_API bool FindMinimum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMinimumValue, 
			const FFrameAttribute& In);

		/** Find the channel and frame with the largest value */
		UE_API bool FindMaximum(
			int32& OutChannelIdx,
			int32& OutFrameIdx,
			float& OutMaximumValue, 
			const FFrameAttribute& In);


		/** Select between True and False frame attributes based on Cond. */
		UE_API void Select(FFrameAttribute& Out, const FFrameAttribute& Cond, const FFrameAttribute& True, const FFrameAttribute& False);


		/** Create a frame attribute made up of zeros */
		UE_API void Zeros(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const int32 ChannelNum);

		/** Create a frame attribute made up of ones */
		UE_API void Ones(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const int32 ChannelNum);

		/** Create a frame attribute filled with the given value at each frame */
		UE_API void Fill(FFrameAttribute& Out, const FFrameRangeSet& FrameRangeSet, const TLearningArrayView<1, const float> Values);


		/** Add two frame attributes. Channel numbers must match. */
		UE_API void Add(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Subtract two frame attributes. Channel numbers must match. */
		UE_API void Sub(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Multiply two frame attributes. Channel numbers must match. */
		UE_API void Mul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Divide two frame attributes. Channel numbers must match. */
		UE_API void Div(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the component-wise maximum of two frame attributes. Channel numbers must match. */
		UE_API void Max(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the component-wise minimum of two frame attributes. Channel numbers must match. */
		UE_API void Min(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);


		/** Compute the dot product of two frame attributes. Channel numbers must match. */
		UE_API void Dot(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Compute the cross product of two frame attributes. Both arguments are assumed to have 3 channels. */
		UE_API void Cross(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Copy a frame attribute. */
		UE_API void Copy(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Create a frame attribute filled with the given single channel frame attribute at each frame */
		UE_API void Repeat(FFrameAttribute& Out, const int32 ChannelNum, const FFrameAttribute& In);

		/** Negate a frame attribute. */
		UE_API void Neg(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Invert a frame attribute (compute 1/x). */
		UE_API void Inv(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the absolute value of a frame attribute. */
		UE_API void Abs(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the log of a frame attribute. */
		UE_API void Log(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the exp of a frame attribute. */
		UE_API void Exp(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the sqrt of a frame attribute. */
		UE_API void Sqrt(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the sin of a frame attribute. */
		UE_API void Sin(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the cos of a frame attribute. */
		UE_API void Cos(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the atan2 of two frame attributes. */
		UE_API void Atan2(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Compute the negation of a frame attribute in place. */
		UE_API void NegInplace(FFrameAttribute& InOut);

		/** Compute 1/x of a frame attribute in place. */
		UE_API void InvInplace(FFrameAttribute& InOut);

		/** Compute the absolute value of a frame attribute in place. */
		UE_API void AbsInplace(FFrameAttribute& InOut);

		/** Compute the log of a frame attribute in place. */
		UE_API void LogInplace(FFrameAttribute& InOut);

		/** Compute the exp of a frame attribute in place. */
		UE_API void ExpInplace(FFrameAttribute& InOut);

		/** Compute the square root of a frame attribute in place. */
		UE_API void SqrtInplace(FFrameAttribute& InOut);

		/** Compute the approximate acos of a frame attribute in place. */
		UE_API void ApproxAcosInplace(FFrameAttribute& InOut);

		/** Normalize the frame attribute in place. */
		UE_API void NormalizeInplace(FFrameAttribute& InOut);

		/** Repeats the first entry in the range across the rest of the range */
		UE_API void RepeatFirstRangeEntryInplace(FFrameAttribute& InOut);


		/** Compute the length of a frame attribute over the channels dimension. */
		UE_API void Length(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Normalize a frame attribute over the channels dimension. */
		UE_API void Normalize(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Extract a single channel from frame attribute. */
		UE_API void Index(FFrameAttribute& Out, const FFrameAttribute& In, const int32 ChannelIdx);

		/** Extract a range of channels from frame attribute. */
		UE_API void Slice(FFrameAttribute& Out, const FFrameAttribute& In, const int32 StartChannelIdx, const int32 ChannelNum);

		/** Add a constant value to a frame attribute. Channel number must match the size of Rhs. */
		UE_API void AddConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Subtract a constant value to a frame attribute. Channel number must match the size of Rhs. */
		UE_API void SubConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Multiply a frame attribute by a constant value. Channel number must match the size of Rhs. */
		UE_API void MulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Divide a frame attribute by a constant value. Channel number must match the size of Rhs. */
		UE_API void DivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Compute the component-wise maximum of a constant value and a frame attribute. Channel number must match the size of Rhs. */
		UE_API void MaxConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Compute the component-wise minimum of a constant value and a frame attribute. Channel number must match the size of Rhs. */
		UE_API void MinConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);


		/** Add a constant value to a frame attribute. Channel number must match the size of Lhs. */
		UE_API void ConstantAdd(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Subtract a constant value to a frame attribute. Channel number must match the size of Lhs. */
		UE_API void ConstantSub(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Multiply a frame attribute by a constant value. Channel number must match the size of Lhs. */
		UE_API void ConstantMul(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Divide a frame attribute by a constant value. Channel number must match the size of Lhs. */
		UE_API void ConstantDiv(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Compute the component-wise maximum of a constant value and a frame attribute. Channel number must match the size of Lhs. */
		UE_API void ConstantMax(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Compute the component-wise minimum of a constant value and a frame attribute. Channel number must match the size of Lhs. */
		UE_API void ConstantMin(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);



		/** Gets the dot product with a constant value. Channel number must match the size of Lhs. */
		UE_API void ConstantDot(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Gets the squared euclidean distance to a constant value. Channel number must match the size of Lhs. */
		UE_API void ConstantSquaredDist(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Gets the euclidean distance to a constant value. Channel number must match the size of Lhs. */
		UE_API void ConstantDist(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);


		/** Add a frame attribute by a constant value in-place. Channel number must match the size of Rhs. */
		UE_API void AddConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs);

		/** Subtract a frame attribute by a constant value in-place. Channel number must match the size of Rhs. */
		UE_API void SubConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs);

		/** Multiply a frame attribute by a constant value in-place. Channel number must match the size of Rhs. */
		UE_API void MulConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs);

		/** Divide a frame attribute by a constant value in-place. Channel number must match the size of Rhs. */
		UE_API void DivConstantInplace(FFrameAttribute& InOut, const TLearningArrayView<1, const float> Rhs);


		/** Compute the sum of an array of frame attributes. Channel numbers must match. */
		UE_API void Sum(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> Inputs);

		/** Compute the product of an array of frame attributes. Channel numbers must match. */
		UE_API void Prod(FFrameAttribute& Out, const TArrayView<const ConstFrameAttributePtr> Inputs);


		/** Computes the logical and of two frame attributes. Channel numbers must match. */
		UE_API void LogicalAnd(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the logical or of two frame attributes. Channel numbers must match. */
		UE_API void LogicalOr(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the logical not of a frame attribute. */
		UE_API void LogicalNot(FFrameAttribute& Out, const FFrameAttribute& In);


		/** Computes if the values in one frame attribute are greater than another. Channel numbers must match. */
		UE_API void Gt(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in one frame attribute are greater than or equal to another. Channel numbers must match. */
		UE_API void Ge(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in one frame attribute are less than another. Channel numbers must match. */
		UE_API void Lt(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in one frame attribute are less than or equal to another. Channel numbers must match. */
		UE_API void Le(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in one frame attribute are equal to another. Channel numbers must match. */
		UE_API void Eq(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in one frame attribute are not equal to another. Channel numbers must match. */
		UE_API void Neq(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);


		/** Computes if the values in a frame attribute are greater than a constant. Channel number must match the size of Rhs. */
		UE_API void GtConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Computes if the values in a frame attribute are greater than or equal to a constant. Channel number must match the size of Rhs. */
		UE_API void GeConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Computes if the values in a frame attribute are less than a constant. Channel number must match the size of Rhs. */
		UE_API void LtConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Computes if the values in a frame attribute are greater than or equal to a constant. Channel number must match the size of Rhs. */
		UE_API void LeConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);
		
		/** Computes if the values in a frame attribute are equal to a constant. Channel number must match the size of Rhs. */
		UE_API void EqConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);

		/** Computes if the values in a frame attribute are not equal to a constant. Channel number must match the size of Rhs. */
		UE_API void NeqConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const TLearningArrayView<1, const float> Rhs);


		/** Computes if the values in a frame attribute are greater than a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantGt(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);
		
		/** Computes if the values in a frame attribute are greater than or equal to a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantGe(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);
		
		/** Computes if the values in a frame attribute are less than a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantLt(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in a frame attribute are greater than or equal to a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantLe(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);

		/** Computes if the values in a frame attribute are equal to a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantEq(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);
		
		/** Computes if the values in a frame attribute are not equal to a constant. Channel number must match the size of Lhs. */
		UE_API void ConstantNeq(FFrameAttribute& Out, const TLearningArrayView<1, const float> Lhs, const FFrameAttribute& Rhs);


		/** Applies a Gaussian smoothing filter to the ranges of values in the frame attribute. */
		UE_API void FilterGaussian(FFrameAttribute& Out, const FFrameAttribute& In, const float StdInFrames);

		/** Applies a Majority Vote filter to the ranges of values in the frame attribute. */
		UE_API void FilterMajorityVote(FFrameAttribute& Out, const FFrameAttribute& In, const int32 FilterWidthFrames);

		/** Applies a SavGol filter to the ranges of values in the frame attribute. */
		UE_API void FilterSavGol(FFrameAttribute& Out, const FFrameAttribute& In, const int32 FilterWidthFrames, const int32 PolynomialDegree, const bool bGaussianWindowed);


		/** Computes the mean across all frames. OutMean should be size of the number of channels */
		UE_API void Mean(TLearningArrayView<1, float> OutMean, const FFrameAttribute& In);

		/** Computes the mean and std across all frames. OutMean and OutStd should be size of the number of channels */
		UE_API void MeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutStd, const FFrameAttribute& In);

		/** Computes the mean and log std across all frames in the log space. OutMean and OutLogStd should be size of the number of channels */
		UE_API void LogMeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutLogStd, const FFrameAttribute& In);

		/** Computes the min and max across all frames. OutMin and OutMax should be size of the number of channels */
		UE_API void MinMax(TLearningArrayView<1, float> OutMin, TLearningArrayView<1, float> OutMax, const FFrameAttribute& In);

		/** Computes the angular mean across all frames. */
		UE_API void AngularMean(TLearningArrayView<1, float> OutMean, const FFrameAttribute& In);

		/** Computes the angular mean and std across all frames. */
		UE_API void AngularMeanStd(TLearningArrayView<1, float> OutMean, TLearningArrayView<1, float> OutStd, const FFrameAttribute& In);


		/** Computes the quaternion multiplication of two frame attributes. Both arguments are assumed to have 4 channels. */
		UE_API void QuatMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion division of two frame attributes. Both arguments are assumed to have 4 channels. */
		UE_API void QuatDiv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Compute the quaternion inverse of a frame attribute. Argument is assumed to have 4 channels. */
		UE_API void QuatInv(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute the quaternion closest to the identity quaternion for a frame attribute. Argument is assumed to have 4 channels. */
		UE_API void QuatAbs(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute a rotation vector from a quaternion. Argument is assumed to have 4 channels. */
		UE_API void QuatToRotationVector(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute a quaternion from a rotation vector. Argument is assumed to have 3 channels. */
		UE_API void QuatFromRotationVector(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Compute a quaternion from a set of basis directions. Arguments are assumed to have 3 channels. */
		UE_API void QuatFromBasisDirections(FFrameAttribute& Out, const FFrameAttribute& Forward, const FFrameAttribute& Right, const FFrameAttribute& Up);

		/** Compute a quaternion from pitch, yaw and roll. */
		UE_API void QuatFromPitchYawRoll(FFrameAttribute& Out, const FFrameAttribute& Pitch, const FFrameAttribute& Yaw, const FFrameAttribute& Roll);

		/** Computes the quaternion inverse multiplication of two frame attributes. Both arguments are assumed to have 4 channels. */
		UE_API void QuatInvMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion multiplication inverse of two frame attributes. Both arguments are assumed to have 4 channels. */
		UE_API void QuatMulInv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the rotation of the Rhs by the Lhs quaternion. Lhs is assumed to have 4 channels, and Rhs is assumed to have 3. */
		UE_API void QuatRotate(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the inverse rotation of the Rhs by the Lhs quaternion. Lhs is assumed to have 4 channels, and Rhs is assumed to have 3. */
		UE_API void QuatUnrotate(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the shortest rotation between two vectors. Both arguments are assumed to have 3 channels. */
		UE_API void QuatBetween(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion multiplication of a frame attribute by the given quaternion. Lhs is assumed to have 4 channels. */
		UE_API void QuatMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs);

		/** Computes the quaternion division of a frame attribute by the given quaternion. Lhs is assumed to have 4 channels. */
		UE_API void QuatDivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs);

		/** Computes the quaternion inverse multiplication of a frame attribute by the given quaternion. Lhs is assumed to have 4 channels. */
		UE_API void QuatInvMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs);

		/** Computes the quaternion multiplication inverse of a frame attribute by the given quaternion. Lhs is assumed to have 4 channels. */
		UE_API void QuatMulInvConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs);

		/** Computes the quaternion rotation of the given vector by a frame attribute. Lhs is assumed to have 4 channels. */
		UE_API void QuatRotateConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs);

		/** Computes the quaternion inverse rotation of the given vector by a frame attribute. Lhs is assumed to have 4 channels. */
		UE_API void QuatUnrotateConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs);

		/** Computes the quaternion between the given frame attribute and vector. Lhs is assumed to have 3 channels. */
		UE_API void QuatBetweenConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f Rhs);

		/** Computes a fast approximation of the shortest angle between the given frame attribute and the quaternion. Lhs is assumed to have 4 channels. */
		UE_API void QuatApproxShortestAngleBetweenConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FQuat4f Rhs);

		/** Computes the quaternion multiplication of a frame attribute by the given quaternion. Rhs is assumed to have 4 channels. */
		UE_API void QuatConstantMul(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion division of a frame attribute by the given quaternion. Rhs is assumed to have 4 channels. */
		UE_API void QuatConstantDiv(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);
		
		/** Computes the quaternion inverse multiplication of a frame attribute by the given quaternion. Rhs is assumed to have 4 channels. */
		UE_API void QuatConstantInvMul(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);
		
		/** Computes the quaternion multiplication inverse of a frame attribute by the given quaternion. Rhs is assumed to have 4 channels. */
		UE_API void QuatConstantMulInv(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);
		
		/** Computes the quaternion rotation of the given frame attribute. Rhs is assumed to have 3 channels. */
		UE_API void QuatConstantRotate(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion inverse rotation of the given frame attribute. Rhs is assumed to have 3 channels. */
		UE_API void QuatConstantUnrotate(FFrameAttribute& Out, const FQuat4f Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion between the given frame attribute and vector. Rhs is assumed to have 3 channels. */
		UE_API void QuatConstantBetween(FFrameAttribute& Out, const FVector3f Lhs, const FFrameAttribute& Rhs);

		/** Computes the quaternion mean and std across all frames. */
		UE_API void QuatMeanStd(FQuat4f& OutMean, FVector3f& OutStd, const FFrameAttribute& In);

		/** Computes the pitch component of a quaternion rotation. Argument is assumed to have 4 channels. */
		UE_API void QuatPitch(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Computes the yaw component of a quaternion rotation. Argument is assumed to have 4 channels. */
		UE_API void QuatYaw(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Computes the roll component of a quaternion rotation. Argument is assumed to have 4 channels. */
		UE_API void QuatRoll(FFrameAttribute& Out, const FFrameAttribute& In);


		/** Wraps a frame attribute in-place into the range -UE_PI, UE_PI */
		UE_API void WrapAngleInplace(FFrameAttribute& InOut);

		/** Computes a 3D direction from a scalar angle, leaving the Z component as zero */
		UE_API void AngleToDirection3D(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Computes an angle from a 3D direction ignoring the Z component */
		UE_API void Direction3DToAngle(FFrameAttribute& Out, const FFrameAttribute& In);


		/** Make a transform attribute from a location, rotation, and scale */
		UE_API void TransformMake(FFrameAttribute& Out, const FFrameAttribute& Location, const FFrameAttribute& Rotation, const FFrameAttribute& Scale);

		/** Make a transform attribute from a location and rotation */
		UE_API void TransformMake(FFrameAttribute& Out, const FFrameAttribute& Location, const FFrameAttribute& Rotation);

		/** Computes the transform multiplication of two frame attributes. Both arguments are assumed to have 10 channels. */
		UE_API void TransformMul(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the transform division of two frame attributes. Both arguments are assumed to have 10 channels. */
		UE_API void TransformDiv(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FFrameAttribute& Rhs);

		/** Computes the transform multiplication of a frame attribute and a constant value. Argument is assumed to have 10 channels. */
		UE_API void TransformMulConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FTransform3f& Rhs);

		/** Computes the transform division of a frame attribute and a constant value. Argument is assumed to have 10 channels. */
		UE_API void TransformDivConstant(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FTransform3f& Rhs);

		/** Computes the transform multiplication of a frame attribute and a constant value. Argument is assumed to have 10 channels. */
		UE_API void TransformConstantMul(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs);

		/** Computes the transform division of a frame attribute and a constant value. Argument is assumed to have 10 channels. */
		UE_API void TransformConstantDiv(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs);

		/** Compute the transform inverse of a frame attribute. Argument is assumed to have 10 channels. */
		UE_API void TransformInv(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Gets the location part of a transform. Argument is assumed to have 10 channels. */
		UE_API void TransformLocation(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Gets the rotation part of a transform. Argument is assumed to have 10 channels. */
		UE_API void TransformRotation(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Gets the scale part of a transform. Argument is assumed to have 10 channels. */
		UE_API void TransformScale(FFrameAttribute& Out, const FFrameAttribute& In);

		/** Applies the transform to a location */
		UE_API void TransformApplyConstantLocation(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f& Rhs);

		/** Applies the transform to a direction */
		UE_API void TransformApplyConstantDirection(FFrameAttribute& Out, const FFrameAttribute& Lhs, const FVector3f& Rhs);

		/** Applies the transform to a vector */
		UE_API void TransformConstantApply(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs);

		/** Applies the transform to a vector without translation (i.e. rotation and scale only) */
		UE_API void TransformConstantApplyNoTranslation(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs);

		/** Applies the transform to a vector without translation or scale (i.e. rotation only) */
		UE_API void TransformConstantApplyNoTranslationScale(FFrameAttribute& Out, const FTransform3f& Lhs, const FFrameAttribute& Rhs);


		/** Inertialization distance for a cubic inertialization function. See: https://theorangeduck.com/page/inertialization-transition-cost */
		UE_API void InertializationDistanceConstant(
			FFrameAttribute& Out, 
			const FFrameAttribute& LhsPos, 
			const FFrameAttribute& LhsVel,
			const TArrayView<const float> RhsLoc,
			const TArrayView<const float> RhsVel,
			const float BlendTime,
			const float LocWeight,
			const float VelWeight);

		/** Gets distance traveled of each range in a location frame attribute */
		UE_API void LocationDistanceTraveled(TArrayView<float> OutDistancesTraveled, const FFrameAttribute& In);

		/** Gets angle traveled of each range in a quat frame attribute */
		UE_API void QuatAngleTraveled(TArrayView<float> OutAnglesTraveled, const FFrameAttribute& In);

		/* Mode for handling edges in phase extraction */
		enum class EPhaseExtrapolationMode : uint8
		{
			// Repeats the phase value at the start and end frames 
			Repeat = 0,

			// Extrapolates the phase value using the nearest pair of frames
			Extrapolate = 1,
		};

		/** Extract the phase for a given set of frame ranges, using the labeled events */
		UE_API void ExtractPhase(
			FFrameAttribute& Out, 
			const FFrameRangeSet& FrameRangeSet, 
			const FFrameSet& ZeroPhaseFrames, 
			const FFrameSet& HalfPhaseFrames,
			const EPhaseExtrapolationMode Extrapolation = EPhaseExtrapolationMode::Repeat);
	}
}

#undef UE_API
