// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	struct FFrameSet;

	/**
	 * A FFrameRangeSet represents a set of ranges within a set of sequences. This can be useful for encoding things like a collection of tags or 
	 * labels for different parts of some database of animations, replays, or recordings.
	 *
	 * This data structure stores that information in a way that makes performing operations such as union, intersection, difference etc, efficient.
	 * 
	 * The way it works is by storing a sorted array of "entries", where each entry has a corresponding sequence, the number of ranges in the set for
	 * that sequence, and an index into three large arrays of range starts, lengths, and offsets. The sub-ranges of the large arrays of range 
	 * starts and lengths are also sorted, while the offsets arrays stores the offset that would be used if you had to take a slice of a large flat 
	 * array of frame data.
	 *
	 * Having the entries sorted by sequence, and the ranges sorted, allows for efficient set operations via tape-merge algorithms.
	 *
	 * This data-structure is related to the FrameSet data-structure, which essentially acts the same way but stores individual frames instead of
	 * frame ranges.
	 */
	struct FFrameRangeSet
	{
	public:

		/** Check if the FrameRangeSet is well-formed (i.e. correctly sorted without duplicate entries) */
		UE_API void Check() const;

		/**
		 * Adds the given ranges associated with the given sequence to the set. Assumes this sequence (and no sequences with a larger index)
		 * are already added to the set.
		 */
		UE_API void AddEntry(
			const int32 InSequence,
			const TLearningArrayView<1, const int32> InStarts,
			const TLearningArrayView<1, const int32> InLengths);

		/** True if the FrameRangeSet is Empty, otherwise false */
		UE_API bool IsEmpty() const;

		/** Empties the FrameRangeSet */
		UE_API void Empty();

		/** Gets the number of entries in the FrameRangeSet */
		UE_API int32 GetEntryNum() const;

		/** Returns an array of sequences for each entry */
		UE_API TLearningArrayView<1, const int32> GetEntrySequences() const;

		/** Returns an array for the number of ranges in each entry */
		UE_API TLearningArrayView<1, const int32> GetEntryRangeNums() const;

		/** Gets the sequence associated with a given entry */
		UE_API int32 GetEntrySequence(const int32 EntryIdx) const;

		/** Gets the number of ranges for a given entry */
		UE_API int32 GetEntryRangeNum(const int32 EntryIdx) const;

		/** Computes the total number of frames given by all ranges in an entry */
		UE_API int32 GetEntryTotalFrameNum(const int32 EntryIdx) const;

		/** Gets all the range starts associated with a given entry */
		UE_API TLearningArrayView<1, const int32> GetEntryRangeStarts(const int32 EntryIdx) const;

		/** Gets all the range lengths associated with a given entry */
		UE_API TLearningArrayView<1, const int32> GetEntryRangeLengths(const int32 EntryIdx) const;
		
		/** Gets all the range offsets associated with a given entry */
		UE_API TLearningArrayView<1, const int32> GetEntryRangeOffsets(const int32 EntryIdx) const;

		/** Gets the range start associated with a given entry and range index */
		UE_API int32 GetEntryRangeStart(const int32 EntryIdx, const int32 RangeIdx) const;

		/** Gets the range length associated with a given entry and range index */
		UE_API int32 GetEntryRangeLength(const int32 EntryIdx, const int32 RangeIdx) const;

		/** Gets the range stop associated with a given entry and range index */
		UE_API int32 GetEntryRangeStop(const int32 EntryIdx, const int32 RangeIdx) const;

		/** Computes the offset associated with a given entry and range index */
		UE_API int32 GetEntryRangeOffset(const int32 EntryIdx, const int32 RangeIdx) const;

		/** Gets the range start time associated with a given entry and range index */
		UE_API float GetEntryRangeStartTime(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const;

		/** Gets the range end time associated with a given entry and range index */
		UE_API float GetEntryRangeEndTime(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const;

		/** Gets the range duration associated with a given entry and range index */
		UE_API float GetEntryRangeDuration(const int32 EntryIdx, const int32 RangeIdx, const float FrameDeltaTime) const;

		/** Computes the total number of ranges across all entries */
		UE_API int32 GetTotalRangeNum() const;

		/** Gets an array of the range starts for all ranges */
		UE_API TLearningArrayView<1, const int32> GetAllRangeStarts() const;

		/** Gets an array of the range lengths for all ranges */
		UE_API TLearningArrayView<1, const int32> GetAllRangeLengths() const;

		/** Gets an array of the range offsets for all ranges */
		UE_API TLearningArrayView<1, const int32> GetAllRangeOffsets() const;

		/** Computes the total number of frames across all entries and ranges */
		UE_API int32 GetTotalFrameNum() const;

		/** Computes the average number of frames across all entries and ranges */
		UE_API float GetAverageFrameNum() const;

		/** Gets the largest number of frames across all entries and ranges */
		UE_API int32 GetMaxFrameNum() const;

		/** Gets the smallest number of frames across all entries and ranges */
		UE_API int32 GetMinFrameNum() const;

		/** Checks if this FrameRangeSet contains a given sequence */
		UE_API bool ContainsSequence(const int32 Sequence) const;

		/** Checks if this FrameRangeSet contains a given sequence and frame in that sequence */
		UE_API bool Contains(const int32 Sequence, const int32 Frame) const;

		/** Checks if the given Entry of this FrameRangeSet contains a frame */
		UE_API bool EntryContains(const int32 EntryIdx, const int32 Frame) const;

		/** Checks if this FrameRangeSet intersects a sequence and any part of a range in that sequence */
		UE_API bool IntersectsRange(const int32 Sequence, const int32 Start, const int32 Length) const;

		/** Checks if the given Entry of this FrameRangeSet fully contains a range */
		UE_API bool EntryContainsRange(const int32 EntryIdx, const int32 Start, const int32 Length) const;

		/** Checks if the given Entry of this FrameRangeSet intersects any part of a range */
		UE_API bool EntryIntersectsRange(const int32 EntryIdx, const int32 Start, const int32 Length) const;

		/** Checks if this FrameRangeSet contains a given sequence and time in that sequence */
		UE_API bool ContainsTime(const int32 Sequence, const float Time, const float FrameDeltaTime, const float Tolerance = UE_KINDA_SMALL_NUMBER) const;

		/** Finds the entry index associated with a given sequence */
		UE_API int32 FindSequenceEntry(const int32 Sequence) const;

		/** Finds the entry index and range index associated with a given sequence and frame in that sequence */
		UE_API bool Find(int32& OutEntryIdx, int32& OutRangeIdx, int32& OutRangeFrame, const int32 Sequence, const int32 Frame) const;

		/** Finds the entry index and range index associated with a range index into all ranges */
		UE_API bool FindTotalRange(int32& OutEntryIdx, int32& OutRangeIdx, const int32 TotalRangeIdx) const;

		/** Finds the entry index and range index associated with a given sequence and time in that sequence */
		UE_API bool FindTime(int32& OutEntryIdx, int32& OutRangeIdx, float& OutRangeTime, const int32 Sequence, const float Time, const float FrameDeltaTime, const float Tolerance = UE_KINDA_SMALL_NUMBER) const;

		/** Finds entry index, frame index, and frame in that range, associated with some offset  */
		UE_API bool FindOffset(int32& OutEntryIdx, int32& OutRangeIdx, int32& OutRangeFrame, const int32 Offset) const;

	public:

		/** Array of sequences associated with each entry */
		TLearningArray<1, int32> EntrySequences;

		/** Array of offsets into the RangeStarts, RangeLengths, RangeOffsets arrays associated with each entry */
		TLearningArray<1, int32> EntryRangeOffsets;

		/** Array of the number of ranges associated with each entry */
		TLearningArray<1, int32> EntryRangeNums;

		/** Large array of all range starts for all entries, indexed using offsets from EntryRangeOffsets */
		TLearningArray<1, int32> RangeStarts;

		/** Large array of all range lengths for all entries, indexed using offsets from EntryRangeOffsets */
		TLearningArray<1, int32> RangeLengths;

		/** Large array of all range frame offsets for all entries, indexed using offsets from EntryRangeOffsets */
		TLearningArray<1, int32> RangeOffsets;
	};

	namespace FrameRangeSet
	{
		/** Checks if two frame range sets are equal */
		UE_API bool Equal(const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs);

		/** Computes the union of a frame set and frame range set */
		UE_API void Union(FFrameRangeSet& OutFrameRangeSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet);

		/** Computes the intersection of a frame set and frame range set */
		UE_API void Intersection(FFrameSet& OutFrameSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet);

		/** Computes the difference of a frame set and frame range set */
		UE_API void Difference(FFrameSet& OutFrameSet, const FFrameSet& FrameSet, const FFrameRangeSet& FrameRangeSet);

		/** Computes the difference of a frame range set and a frame set */
		UE_API void Difference(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet);

		/** Computes the union of two frame range sets */
		UE_API void Union(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs);

		/** Computes the intersection of two frame range sets */
		UE_API void Intersection(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs);

		/** Computes the difference of two frame range sets */
		UE_API void Difference(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs);

		/**
		 * Computes the intersection of two frame range sets while recording the flat frame offsets in the lhs and rhs, associated with each range
		 * that gets added to the output. The OutLhsOffsets and OutRhsOffsets outputs should be pre-allocated to at least the maximum of the
		 * number of ranges in Lhs and Rhs. Returns the number of ranges in the added to the output frame range set.
		 */
		UE_API int32 IntersectionWithOffsets(
			FFrameRangeSet& Out, 
			TLearningArrayView<1, int32> OutLhsOffsets,
			TLearningArrayView<1, int32> OutRhsOffsets,
			const FFrameRangeSet& Lhs,
			const FFrameRangeSet& Rhs);

		/** Computes the bounding ranges of a given frame range set */
		UE_API void Bounds(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet);

		/** Computes the bounding ranges of a given frame set */
		UE_API void Bounds(FFrameRangeSet& Out, const FFrameSet& FrameSet);

		/** Computes all of the ranges in the Lhs which contain any part of the ranges in the Rhs */
		UE_API void Intersects(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameRangeSet& Rhs);

		/** Computes all of the ranges in the Lhs which contain any part of the frames in the Rhs */
		UE_API void Intersects(FFrameRangeSet& Out, const FFrameRangeSet& Lhs, const FFrameSet& Rhs);
		
		/** Creates a new frame range set with the start of all ranges trimmed by the given number of frames */
		UE_API void TrimStart(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimFrameNum);

		/** Creates a new frame range set with the end of all ranges trimmed by the given number of frames */
		UE_API void TrimEnd(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimFrameNum);

		/** Creates a new frame range set with both the start and the end of all ranges trimmed by the given numbers of frames */
		UE_API void Trim(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 TrimStartFrameNum, const int32 TrimEndFrameNum);

		/** Creates a new frame range set with the start of all ranges padded by the given number of frames */
		UE_API void PadStart(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadFrameNum);

		/** Creates a new frame range set with the end of all ranges padded by the given number of frames */
		UE_API void PadEnd(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadFrameNum);

		/** Creates a new frame range set with both the start and the end of all ranges padded by the given numbers of frames */
		UE_API void Pad(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const int32 PadStartFrameNum, int32 PadEndFrameNum);

		/** Creates a new frame range set with just the ranges given by the provided indices */
		UE_API void GatherRanges(FFrameRangeSet& Out, const FFrameRangeSet& FrameRangeSet, const FIndexSet Indices);

		/** Makes a Frame Range Set consisting of single-frame ranges from a frame set */
		UE_API void MakeFromFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameSet& FrameSet);

		/** Create a frame set from all of the range starts of a given frame range set */
		UE_API void MakeFrameSetFromRangeStarts(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet);

		/** Create a frame set from all of the range ends of a given frame range set */
		UE_API void MakeFrameSetFromRangeEnds(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet);

		/** Create a frame set from a given frame range set, sampling frames at a regular interval */
		UE_API void MakeFrameSetFromRangeInterval(FFrameSet& OutFrameSet, const FFrameRangeSet& FrameRangeSet, const int32 IntervalFrames = 10);

		/** Trim a frame range set to just the period of time before the first occurrence of a frame from a given frame set */
		UE_API void RangesBeforeFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet);

		/** Trim a frame range set to just the period of time after the first occurrence of a frame from a given frame set */
		UE_API void RangesAfterFrameSet(FFrameRangeSet& OutFrameRangeSet, const FFrameRangeSet& FrameRangeSet, const FFrameSet& FrameSet);

		/** Computes the entry indices associated with every range in the set */
		UE_API void AllRangeEntries(
			TLearningArrayView<1, int32> OutRangeEntries,
			const FFrameRangeSet& FrameRangeSet);

		/** Computes the range indices associated with every range in the set */
		UE_API void AllRangeIndices(
			TLearningArrayView<1, int32> OutRangeIndices,
			const FFrameRangeSet& FrameRangeSet);

		/** Computes the sequences associated with every range in the set */
		UE_API void AllRangeSequences(
			TLearningArrayView<1, int32> OutRangeSequences,
			const FFrameRangeSet& FrameRangeSet);

		/** Computes the start times associated with every range in the set */
		UE_API void AllRangeStartTimes(
			TLearningArrayView<1, float> OutRangeStartTimes,
			const FFrameRangeSet& FrameRangeSet,
			const float FrameDeltaTime);

		/** Computes the end times associated with every range in the set */
		UE_API void AllRangeEndTimes(
			TLearningArrayView<1, float> OutRangeEndTimes,
			const FFrameRangeSet& FrameRangeSet,
			const float FrameDeltaTime);

		/** Computes the durations associated with every range in the set */
		UE_API void AllRangeDurations(
			TLearningArrayView<1, float> OutRangeDurations,
			const FFrameRangeSet& FrameRangeSet,
			const float FrameDeltaTime);

		/** Iterates over every range in the set and calls the provided callback */
		UE_API void ForEachRange(
			const FFrameRangeSet& FrameRangeSet,
			const TFunctionRef<void(
				const int32 TotalRangeIdx,
				const int32 EntryIdx,
				const int32 RangeIdx)> Body);

		/** Iterates over every range in the set in parallel and calls the provided callback */
		UE_API void ParallelForEachRange(
			const FFrameRangeSet& FrameRangeSet,
			const TFunctionRef<void(
				const int32 TotalRangeIdx,
				const int32 EntryIdx,
				const int32 RangeIdx)> Body);
	}
}

#undef UE_API
