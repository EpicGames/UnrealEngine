// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	 * A FrameSet represents a set of frames within a set of sequences. This can be useful for encoding things like a collection of single-frame 
	 * events.
	 * 
	 * This data structure stores that information in a way that makes performing operations such as union, intersection, difference etc, efficient. 
	 * 
	 * The way it works is by storing a sorted array of "entries", where each entry has a corresponding sequence, the number of frames in the set for 
	 * that sequence, and an offset into a large array of frames. The sub-ranges of the large array of frames corresponding to each entry is also 
	 * sorted.
	 * 
	 * Having the entries sorted by sequence, and the frames sorted, allows for efficient set operations via tape-merge algorithms.
	 * 
	 * Also provided are some algorithms for getting "offsets", which correspond to the index associated with a particular thing if you were to
	 * flatten this data-structure into one large array of frames.
	 * 
	 * This data-structure is related to the FrameRangeSet data-structure, which essentially acts the same way but stores frame ranges instead of 
	 * individual frames.
	 */
	struct FFrameSet
	{
	public:

		/** Check if the FrameSet is well-formed (i.e. correctly sorted without duplicate entries) */
		UE_API void Check() const;

		/**
		 * Adds the given frames associated with the given sequence to the set. Assumes this sequence (and no sequences with a larger index)
		 * are already added to the set.
		 */
		UE_API void AddEntry(
			const int32 InSequence,
			const TLearningArrayView<1, const int32> InFrames);

		/** True if the FrameSet is Empty, otherwise false */
		UE_API bool IsEmpty() const;

		/** Empties the FrameSet */
		UE_API void Empty();

		/** Gets the number of entries in the FrameSet */
		UE_API int32 GetEntryNum() const;

		/** Returns and array of sequences corresponding to each entry */
		UE_API TLearningArrayView<1, const int32> GetEntrySequences() const;

		/** Returns an array containing the number of frames corresponding in each entry */
		UE_API TLearningArrayView<1, const int32> GetEntryFrameNums() const;

		/** Gets the sequence associated with a given entry */
		UE_API int32 GetEntrySequence(const int32 EntryIdx) const;

		/** Gets the number of frames for a given entry */
		UE_API int32 GetEntryFrameNum(const int32 EntryIdx) const;

		/** Gets the array of frames associated with a given entry */
		UE_API TLearningArrayView<1, const int32> GetEntryFrames(const int32 EntryIdx) const;

		/** Gets the frame number for an entry and entry frame index */
		UE_API int32 GetEntryFrame(const int32 EntryIdx, const int32 FrameIdx) const;

		/** Gets the frame time associated with an entry and entry frame index */
		UE_API float GetEntryFrameTime(const int32 EntryIdx, const int32 FrameIdx, const float FrameDeltaTime) const;

		/** Gets the flat offset associated with a particular entry if you were to flatten this data structure */
		UE_API int32 GetEntryOffset(const int32 EntryIdx) const;

		/** Gets an array of the frames */
		UE_API TLearningArrayView<1, const int32> GetAllFrames() const;

		/** Gets the total number of frames if you were to flatten this data structure */
		UE_API int32 GetTotalFrameNum() const;

		/** Checks if this FrameSet contains a given sequence */
		UE_API bool ContainsSequence(const int32 Sequence) const;

		/** Checks if this FrameSet contains a given sequence and frame in that sequence */
		UE_API bool Contains(const int32 Sequence, const int32 Frame) const;

		/** Finds the entry index associated with a given sequence */
		UE_API int32 FindSequenceEntry(const int32 Sequence) const;

		/** Finds the entry index and frame index associated with a given sequence and frame in that sequence */
		UE_API bool Find(int32& OutEntryIdx, int32& OutFrameIdx, const int32 Sequence, const int32 Frame) const;

		/** Finds the nearest entry index, frame index, and the frame difference, for some given sequence and frame in that sequence */
		UE_API bool FindNearest(int32& OutEntryIdx, int32& OutFrameIdx, int32& OutFrameDifference, const int32 Sequence, const int32 Frame) const;

		/** 
		 * Finds the nearest entry index, frame index, and the frame difference, for some given sequence and frame in that sequence, 
		 * limited to a given range
		 */
		UE_API bool FindNearestInRange(
			int32& OutEntryIdx, int32& OutFrameIdx, int32& OutFrameDifference, 
			const int32 Sequence, const int32 Frame, const int32 RangeStart, const int32 RangeLength) const;

		/** Finds entry index and frame index associated with some flat offset */
		UE_API bool FindOffset(int32& OutEntryIdx, int32& OutFrameIdx, const int32 Offset) const;

		/** Finds all the frames within the given range, returning the first frame index and the number of frames */
		UE_API bool FindFramesInRange(int32& OutStartFrameIdx, int32& OutFrameNum, const int32 EntryIdx, const int32 RangeStart, const int32 RangeLength) const;

	public:

		/** Array of sequences associated with each entry */
		TLearningArray<1, int32> EntrySequences;

		/** Array of offsets into the Frames array associated with each entry */
		TLearningArray<1, int32> EntryFrameOffsets;

		/** Array of the number of frames associated with each entry */
		TLearningArray<1, int32> EntryFrameNums;

		/** Large array of all frames for all entries, indexed using offsets from EntryFrameOffsets */
		TLearningArray<1, int32> Frames;
	};

	namespace FrameSet
	{
		/** Checks if two frame sets are equal */
		UE_API bool Equal(const FFrameSet& Lhs, const FFrameSet& Rhs);

		/** Computes the union of two frame sets */
		UE_API void Union(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs);

		/** Computes the intersection of two frame sets */
		UE_API void Intersection(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs);

		/** Computes the difference of two frame sets */
		UE_API void Difference(FFrameSet& Out, const FFrameSet& Lhs, const FFrameSet& Rhs);

		/** Computes the entry indices associated with every frame in the set */
		UE_API void AllFrameEntries(
			TLearningArrayView<1, int32> OutFrameEntries,
			const FFrameSet& FrameSet);

		/** Computes the frame indices associated with every frame in the set */
		UE_API void AllFrameIndices(
			TLearningArrayView<1, int32> OutFrameIndices,
			const FFrameSet& FrameSet);

		/** Computes the sequences associated with every frame in the set */
		UE_API void AllFrameSequences(
			TLearningArrayView<1, int32> OutFrameSequences,
			const FFrameSet& FrameSet);

		/** Computes the times associated with every frame in the set */
		UE_API void AllFrameTimes(
			TLearningArrayView<1, float> OutFrameTimes,
			const FFrameSet& FrameSet,
			const float FrameDeltaTime);

		/** Iterates over every frame in the set and calls the provided callback */
		UE_API void ForEachFrame(
			const FFrameSet& FrameSet,
			const TFunctionRef<void(
				const int32 TotalFrameIdx,
				const int32 EntryIdx,
				const int32 FrameIdx)> Body);

		/** Iterates over every frame in the set in parallel and calls the provided callback */
		UE_API void ParallelForEachFrame(
			const FFrameSet& FrameSet,
			const TFunctionRef<void(
				const int32 TotalFrameIdx,
				const int32 EntryIdx,
				const int32 RangeIdx)> Body);
	}
}

#undef UE_API
