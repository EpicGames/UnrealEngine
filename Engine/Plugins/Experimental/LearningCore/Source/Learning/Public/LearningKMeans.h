// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#define UE_API LEARNING_API

namespace UE::Learning::KMeans
{

	/**
	* Initialize a set of KMean Cluster centers by taking random points 
	* from a set of samples
	*
	* @param Centers			Output cluster centers of shape (ClusterNum, DimensionNum)
	* @param Samples			Samples to be clustered of shape (SamplesNum, DimensionNum)
	* @param Seed				Random Seed
	*/
	UE_API void InitCenters(
		TLearningArrayView<2, float> OutCenters,
		const TLearningArrayView<2, const float> Samples,
		const uint32 Seed);

	/**
	* Updates the assignments of each sample to each cluster
	*
	* @param OutAssignments			For each sample, the index of the cluster it is assigned to - shape (SamplesNum)
	* @param Centers				Cluster centers of shape (ClusterNum, DimensionNum)
	* @param Samples				Samples to be clustered of shape (SamplesNum, DimensionNum)
	* @param bParallelEvaluation	If to allow for multi-threaded evaluation
	* @param MinParallelBatchSize	Minimum batch size to use for multi-threaded evaluation
	*/
	UE_API void UpdateAssignmentsFromCenters(
		TLearningArrayView<1, int32> OutAssignments,
		const TLearningArrayView<2, const float> Centers,
		const TLearningArrayView<2, const float> Samples,
		const bool bParallelEvaluation = true,
		const uint16 MinParallelBatchSize = 16);

	/**
	* Counters the number of assignments in each cluster
	*
	* @param OutAssignmentCounts	Number of samples assigned to each cluster - shape (ClusterNum)
	* @param Assignments			Index of the cluster assigned to each sample - shape (SamplesNum
	*/
	UE_API void CountClusterAssignments(
		TLearningArrayView<1, int32> OutAssignmentCounts,
		const TLearningArrayView<1, const int32> Assignments);

	/**
	* Updates the cluster centers based on the samples and their assignments
	*
	* @param OutCenters			New cluster centers of shape (ClusterNum, DimensionNum)
	* @param Assignments		Index of the cluster assigned to each sample - shape (SamplesNum)
	* @param AssignmentCounts	Number of samples assigned to each cluster - shape (ClusterNum)
	* @param Samples			Samples to be clustered of shape (SamplesNum, DimensionNum)
	*/
	UE_API void UpdateCenters(
		TLearningArrayView<2, float> OutCenters,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<1, const int32> AssignmentCounts,
		const TLearningArrayView<2, const float> Samples);

	/**
	* Initialize a set of KMean Cluster bounds by taking random points
	* from a set of samples
	*
	* @param OutMins			Output cluster mins of shape (ClusterNum, DimensionNum)
	* @param OutMaxs			Output cluster maxs of shape (ClusterNum, DimensionNum)
	* @param Samples			Samples to be clustered of shape (SamplesNum, DimensionNum)
	* @param Seed				Random Seed
	*/
	UE_API void InitBounds(
		TLearningArrayView<2, float> OutMins,
		TLearningArrayView<2, float> OutMaxs,
		const TLearningArrayView<2, const float> Samples,
		const uint32 Seed);

	/**
	* Updates the assignments of each sample to each cluster
	*
	* @param OutAssignments			For each sample, the index of the cluster it is assigned to - shape (SamplesNum)
	* @param Mins					Cluster mins of shape (ClusterNum, DimensionNum)
	* @param Maxs					Cluster maxs of shape (ClusterNum, DimensionNum)
	* @param Samples				Samples to be clustered of shape (SamplesNum, DimensionNum)
	* @param bParallelEvaluation	If to allow for multi-threaded evaluation
	* @param MinParallelBatchSize	Minimum batch size to use for multi-threaded evaluation
	*/
	UE_API void UpdateAssignmentsFromBounds(
		TLearningArrayView<1, int32> OutAssignments,
		const TLearningArrayView<2, const float> Mins,
		const TLearningArrayView<2, const float> Maxs,
		const TLearningArrayView<2, const float> Samples,
		const bool bParallelEvaluation = true,
		const uint16 MinParallelBatchSize = 16);

	/**
	* Updates the cluster bounds based on the samples and their assignments
	*
	* @param OutMins			New cluster mins of shape (ClusterNum, DimensionNum)
	* @param OutMaxs			New cluster maxs of shape (ClusterNum, DimensionNum)
	* @param Assignments		Index of the cluster assigned to each sample - shape (SamplesNum)
	* @param AssignmentCounts	Number of samples assigned to each cluster - shape (ClusterNum)
	* @param Samples			Samples to be clustered of shape (SamplesNum, DimensionNum)
	*/
	UE_API void UpdateBounds(
		TLearningArrayView<2, float> OutMins,
		TLearningArrayView<2, float> OutMaxs,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<2, const float> Samples);

	/**
	* Generates an array of samples points, re-ordered and grouped into their clusters
	*
	* @param OutClusteredSamples		Array of samples, grouped into their clusters of shape (SamplesNum, DimensionNum)
	* @param OutClusterStarts			Start index into OutClusteredSamples for each cluster of shape (ClusterNum)
	* @param OutClusterLengths			Number of samples in each cluster of shape (ClusterNum)
	* @param OutSampleMapping			Index mapping into OutClusteredSamples for each sample in the original array - shape (SamplesNum)
	* @param OutInverseSampleMapping	Index mapping from OutClusteredSamples into the original array for each sample - shape (SamplesNum)
	* @param Assignments				Index of the cluster assigned to each sample - shape (SamplesNum)
	* @param AssignmentCounts			Number of samples assigned to each cluster - shape (ClusterNum)
	* @param Samples					Samples to be clustered of shape (SamplesNum, DimensionNum)
	*/
	UE_API void ComputeClusteredIndex(
		TLearningArrayView<2, float> OutClusteredSamples,
		TLearningArrayView<1, int32> OutClusterStarts,
		TLearningArrayView<1, int32> OutClusterLengths,
		TLearningArrayView<1, int32> OutSampleMapping,
		TLearningArrayView<1, int32> OutInverseSampleMapping,
		const TLearningArrayView<1, const int32> Assignments,
		const TLearningArrayView<1, const int32> AssignmentCounts,
		const TLearningArrayView<2, const float> Samples);
}

#undef UE_API