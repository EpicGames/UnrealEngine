// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Log Settings
	*/
	enum class ELogSetting : uint8
	{
		// Logs basic information
		Normal,

		// No logging
		Silent,
	};
}

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogLearning, Log, All);

#undef UE_API