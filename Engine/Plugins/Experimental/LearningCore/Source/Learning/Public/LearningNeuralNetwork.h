// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Templates/SharedPointer.h"

#include "LearningNeuralNetwork.generated.h"

#define UE_API LEARNING_API

class UNNEModelData;

namespace UE::NNE
{
	class IModelCPU;
	class IModelInstanceCPU;
}

namespace UE::Learning
{
	struct FNeuralNetwork;
	struct FNeuralNetworkInference;
}

/**
 * Neural Network Data Object
 * 
 * This is the UObject which contains the actual data used by a Neural Network. It stores the raw FileData used to construct the network using NNE,
 * as well as the input and output sizes and a compatibility hash that can be used to quickly check if two networks may be compatible in terms of 
 * inputs and outputs.
 * 
 * Internally this also stores the various things required to map between NNE style inference and the style of inference used in Learning via 
 * FNeuralNetwork and FNeuralNetworkInference.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningNeuralNetworkData : public UObject
{
	GENERATED_BODY()

public:

	// Get the FNeuralNetwork object that can be used to do inference.
	UE_API TSharedPtr<UE::Learning::FNeuralNetwork>& GetNetwork();

	/**
	 * Initialize the ULearningNeuralNetworkData object given some input and output sizes as well as a compatibility hash and FileData.
	 * 
	 * @param InInputSize			Network Input Size
	 * @param InOutputSize			Network Output Size
	 * @param InCompatibilityHash	Compatibility Hash
	 * @param InFileData			Network FileData
	 */
	UE_API void Init(
		const int32 InInputSize, 
		const int32 InOutputSize, 
		const int32 InCompatibilityHash, 
		const TArrayView<const uint8> InFileData);

	// Initialize this network from another ULearningNeuralNetworkData object
	UE_API void InitFrom(const ULearningNeuralNetworkData* OtherNetworkData);

	// Load a snapshot of this network from the given array of bytes
	UE_API bool LoadFromSnapshot(const TArrayView<const uint8> InBytes);

	// Save a snapshot of this network to the given array of bytes
	UE_API void SaveToSnapshot(TArrayView<uint8> OutBytes) const;

	// Number of bytes required to save or load a snapshot of this network
	UE_API int32 GetSnapshotByteNum() const;

	// If this network is empty or not
	UE_API bool IsEmpty() const;

	// Gets the network input size
	UE_API int32 GetInputSize() const;
	
	// Gets the network output size
	UE_API int32 GetOutputSize() const;
	
	// Gets the compatibility hash
	UE_API int32 GetCompatibilityHash() const;

	// Gets the content hash
	UE_API int32 GetContentHash() const;

	// Gets the update number which is incremented each time the network is loaded from a snapshot
	UE_API int32 GetUpdateNumber() const;

private:

	// Calls UpdateNetwork to update the internal representation.
	UE_API virtual void PostLoad() override final;

	// Uploads the FileData to NNE and updates the internal in-memory representation of the network used for inference.
	UE_API void UpdateNetwork();

	// Size of the inputs expected by this network
	UPROPERTY(VisibleAnywhere, Category = "Network Properties")
	int32 InputSize = 0;

	// Size of the output produced by this network
	UPROPERTY(VisibleAnywhere, Category = "Network Properties")
	int32 OutputSize = 0;

	// Compatibility hash used for testing if inputs to one network are compatible with another
	UPROPERTY(VisibleAnywhere, Category = "Network Properties")
	int32 CompatibilityHash = 0;

	// Content hash which can be used to see when the network data has changed
	UPROPERTY(VisibleAnywhere, Category = "Network Properties")
	int32 ContentHash = 0;

	// File data used by NNE
	UPROPERTY()
	TArray<uint8> FileData;

	// Number which is incremented each time the network is loaded from a snapshot
	UPROPERTY(VisibleAnywhere, Category = "Network Properties")
	int32 UpdateNumber = 0;

	// Model Data used by NNE
	UPROPERTY()
	TObjectPtr<UNNEModelData> ModelData;

	// Internal in-memory network representation
	TSharedPtr<UE::Learning::FNeuralNetwork> Network;
};

namespace UE::Learning
{
	/**
	* Settings object for a neural network instance
	*/
	struct FNeuralNetworkInferenceSettings
	{
		// If to allow for multi-threaded evaluation
		bool bParallelEvaluation = true;

		// Minimum batch size to use for multi-threaded evaluation
		uint16 MinParallelBatchSize = 16;
	};

	/**
	* Neural Network Object
	*/
	struct FNeuralNetwork
	{
		/** Create a new inference object for this network with the given maximum batch size and inference settings. */
		UE_API TSharedRef<FNeuralNetworkInference> CreateInferenceObject(
			const int32 MaxBatchSize,
			const FNeuralNetworkInferenceSettings& Settings = FNeuralNetworkInferenceSettings());

		UE_API bool IsEmpty() const;
		UE_API int32 GetInputSize() const;
		UE_API int32 GetOutputSize() const;

		UE_API void UpdateModel(const TSharedPtr<NNE::IModelCPU>& InModel, const int32 InInputSize, const int32 InOutputSize);

	private:

		int32 InputSize = 0;
		int32 OutputSize = 0;
		TSharedPtr<NNE::IModelCPU> Model;
		TArray<TWeakPtr<FNeuralNetworkInference>, TInlineAllocator<64>> InferenceObjects;
		FRWLock Lock;
	};

	/**
	* Neural Network Inference Object
	*/
	struct FNeuralNetworkInference
	{
		/**
		 * Constructs a new network inference object. Generally this should not be called directly and FNeuralNetwork::CreateInferenceObject should 
		 * be used instead so that created instances are tracked.
		 * 
		 * @param InModel			NNE Model
		 * @param InMaxBatchSize	Maximum batch size
		 * @param InInputSize		Network input size
		 * @param InOutputSize		Network output size
		 * @param InSettings		Inference settings
		 */
		UE_API FNeuralNetworkInference(
			UE::NNE::IModelCPU& InModel,
			const int32 InMaxBatchSize,
			const int32 InInputSize,
			const int32 InOutputSize,
			const FNeuralNetworkInferenceSettings& InSettings = FNeuralNetworkInferenceSettings());

		/**
		 * Evaluate this network.
		 * 
		 * @param Output		The Output Buffer of shape (<= MaxBatchSize, OutputSize)
		 * @param Input			The Input Buffer of shape (<= MaxBatchSize, InputSize)
		 */
		UE_API void Evaluate(TLearningArrayView<2, float> Output, const TLearningArrayView<2, const float> Input);

		// This function will re-build the internal Model Instances used for multi-threading. It should be called whenever the given Model is updated. 
		UE_API void ReloadModelInstances(NNE::IModelCPU& Model, const int32 InInputSize, const int32 InOutputSize);

		UE_API int32 GetMaxBatchSize() const;
		UE_API int32 GetInputSize() const;
		UE_API int32 GetOutputSize() const;

	private:

		int32 MaxBatchSize = 0;
		int32 InputSize = 0;
		int32 OutputSize = 0;
		FNeuralNetworkInferenceSettings Settings;
		TArray<TSharedPtr<NNE::IModelInstanceCPU>, TInlineAllocator<64>> ModelInstances;
	};

	/**
	* Neural-network based function object.
	*/
	struct FNeuralNetworkFunction
	{
		/**
		 * Constructs a Neural Network Function from the given Neural Network
		 *
		 * @param InMaxInstanceNum		Maximum number of instances to evaluate for
		 * @param InNeuralNetwork		Neural network to use
		 * @param InInferenceSettings	Inference settings
		 */
		UE_API FNeuralNetworkFunction(
			const int32 InMaxInstanceNum,
			const TSharedPtr<FNeuralNetwork>& InNeuralNetwork,
			const FNeuralNetworkInferenceSettings& InInferenceSettings = FNeuralNetworkInferenceSettings());

		/**
		 * Evaluate this network for the given instances.
		 *
		 * Note: this function takes a lock, so although it can be called from multiple threads if the set of Instances don't overlap, it will not 
		 * benefit from multi-threading. This is because to use the NNE interface all the instances that need to be evaluated must be gathered and 
		 * scattered into internal buffers. 
		 * 
		 * @param Output		The Output Buffer of shape (<= MaxInstanceNum, OutputSize)
		 * @param Input			The Input Buffer of shape (<= MaxInstanceNum, InputSize)
		 * @param Instances		The instances to evaluate
		 */
		UE_API void Evaluate(
			TLearningArrayView<2, float> Output,
			const TLearningArrayView<2, const float> Input,
			const FIndexSet Instances);

		/** Sets the NeuralNetwork and re-creates the NeuralNetworkInference object. */
		UE_API void UpdateNeuralNetwork(const TSharedPtr<FNeuralNetwork>& NewNeuralNetwork);

		/** Gets the NeuralNetwork associated with this function */
		UE_API const TSharedPtr<FNeuralNetwork>& GetNeuralNetwork() const;

	private:

		int32 MaxInstanceNum = 0;
		FRWLock EvaluationLock;
		TLearningArray<2, float> InputBuffer;
		TLearningArray<2, float> OutputBuffer;
		TSharedPtr<FNeuralNetwork> NeuralNetwork;
		TSharedPtr<FNeuralNetworkInference> NeuralNetworkInference;
		FNeuralNetworkInferenceSettings InferenceSettings;
	};

}

#undef UE_API
