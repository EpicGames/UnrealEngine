// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "UObject/NameTypes.h"

#define UE_API LEARNING_API

namespace UE::NNE::RuntimeBasic
{
	class FModelBuilderElement;
	class FModelBuilder;
}

namespace UE::Learning::Observation
{
	/**
	 * Observation Type
	 *
	 * The core type of an observation object for which different encoding methods are defined.
	 */
	enum class EType : uint8
	{
		// Empty Observation
		Null = 0,

		// Vector of continuous float observations
		Continuous = 1,

		// Set of exclusive discrete observations
		DiscreteExclusive = 2,

		// Set of inclusive discrete observations
		DiscreteInclusive = 3,

		// Set of named exclusive discrete observations
		NamedDiscreteExclusive = 4,

		// Set of named inclusive discrete observations
		NamedDiscreteInclusive = 5,

		// Combination of multiple observations
		And = 6,

		// Exclusive choice from a set of observations
		OrExclusive = 7,

		// Inclusive choice from a set of observations
		OrInclusive = 8,

		// Fixed sized array of observations
		Array = 9,

		// Variable sized, unordered set of observations
		Set = 10,

		// Encoding of another observation 
		Encoding = 11,

		// 1d convolution of another observation 
		Conv1d = 12,

		// 2d convolution of another observation 
		Conv2d = 13,

		// Sparse continuous observation indexed by integers
		Sparse = 14,

		// Sparse continuous observation indexed by names
		NamedSparse = 15,
	};

	enum class ENormalization : uint8
	{
		Manual = 0,
		AutoShared = 1,
		AutoPerDimension = 2,
		AutoDimensionAverage = 3,
	};

	static inline FString GetNormalizationOptionAsString(const ENormalization NormalizationOption)
	{
		switch (NormalizationOption)
		{
		case UE::Learning::Observation::ENormalization::Manual: return TEXT("Manual");
		case UE::Learning::Observation::ENormalization::AutoShared: return TEXT("AutoShared");
		case UE::Learning::Observation::ENormalization::AutoPerDimension: return TEXT("AutoPerDimension");
		case UE::Learning::Observation::ENormalization::AutoDimensionAverage: return TEXT("AutoDimensionAverage");
		default: checkNoEntry(); return TEXT("");
		}
	}

	/**
	 * Observation Schema Element
	 *
	 * A single element in the observation schema representing a part of an observation. Internally this consists of a index used by the schema to
	 * look up the associated observation data and a generation id which can be used to check when this index is no longer valid.
	 */
	struct FSchemaElement { int32 Index = INDEX_NONE; uint32 Generation = INDEX_NONE; };

	struct FSchemaContinuousParameters
	{
		// Number of values in the continuous observation
		int32 Num = 0;

		// Per-element standard deviation values for normalization. If empty, identity scaling is applied. *Only used when normalization option is set to manual.
		TArrayView<const float> Scales;

		// Per-element mean values for normalization. If empty, zero bias is applied. *Only used when normalization option is set to manual.
		TArrayView<const float> Offsets;

		// The normalization option to use
		ENormalization NormalizationOption = ENormalization::Manual;
	};

	struct FSchemaDiscreteExclusiveParameters
	{
		// Number of values in the discrete observation
		int32 Num = 0;
	};

	struct FSchemaDiscreteInclusiveParameters
	{
		// Number of values in the discrete observation
		int32 Num = 0;
	};

	struct FSchemaNamedDiscreteExclusiveParameters
	{
		// Names of the discrete observations.
		TArrayView<const FName> ElementNames;
	};

	struct FSchemaNamedDiscreteInclusiveParameters
	{
		// Names of the discrete observations.
		TArrayView<const FName> ElementNames;
	};

	struct FSchemaAndParameters
	{
		// Names of the sub-observations.
		TArrayView<const FName> ElementNames;

		// The associated sub-observations.
		TArrayView<const FSchemaElement> Elements;
	};

	struct FSchemaOrExclusiveParameters
	{
		// Names of the sub-observations.
		TArrayView<const FName> ElementNames;

		// The associated sub-observations.
		TArrayView<const FSchemaElement> Elements;

		// The size of the encoding to use to combine these sub-observations.
		int32 EncodingSize = 128;
	};

	struct FSchemaOrInclusiveParameters
	{
		// Names of the sub-observations.
		TArrayView<const FName> ElementNames;

		// The associated sub-observations.
		TArrayView<const FSchemaElement> Elements;

		// The size of the attention embedding to use (per-attention-head) for sub-observations
		int32 AttentionEncodingSize = 16;

		// The number of attention heads to use for combining sub-observations
		int32 AttentionHeadNum = 4;

		// The size of the output encoding to use (per-attention-head) for sub-observations
		int32 ValueEncodingSize = 32;
	};

	struct FSchemaArrayParameters
	{
		// The array sub-element.
		FSchemaElement Element;

		// The number of elements in the array.
		int32 Num = 0;
	};

	struct FSchemaSetParameters
	{
		// The set sub-element.
		FSchemaElement Element;

		// The maximum number of elements in the set.
		int32 MaxNum = 0;

		// The size of the attention embedding to use (per-attention-head) for sub-observations
		int32 AttentionEncodingSize = 16;

		// The number of attention heads to use for combining sub-observations
		int32 AttentionHeadNum = 4;

		// The size of the output encoding to use (per-attention-head) for sub-observations
		int32 ValueEncodingSize = 32;
	};

	/** Activation Function to use for encoding */
	enum class EEncodingActivationFunction : uint8
	{
		ELU = 0,
		ReLU = 1,
		TanH = 2,
		GELU = 3,
	};

	struct FSchemaEncodingParameters
	{
		// The sub-element.
		FSchemaElement Element;

		// The size at which the sub-element should be encoded.
		int32 EncodingSize = 32;

		// The number of layers in the encoding
		int32 LayerNum = 1;

		// The activation function to use for encoding
		EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ELU;
	};

	/** Enum of Padding Mode */
	enum class EPaddingMode : uint8
	{
		Zeros = 0,
		Circular = 1,
	};

	struct FSchemaConv1dParameters
	{
		// The sub-element.
		FSchemaElement Element;

		// Length of the 1d convolution input.
		int32 InputLength = 0;

		// Number of input channels.
		int32 InChannels = 0;

		// Number of output channels after convolution.
		int32 OutChannels = 0;

		// Size of the convolving kernel.
		int32 KernelSize = 3;

		// The amount of padding to add to both sides of the input.
		int32 Padding = 1;

		// Zeros or Circular.
		EPaddingMode PaddingMode = EPaddingMode::Zeros;

		// The activation function to use for the convolution layer.
		EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ReLU;
	};

	struct FSchemaConv2dParameters
	{
		// The sub-element.
		FSchemaElement Element;

		// Height of the 2d convolution input.
		int32 InputHeight = 0;
		
		// Width of the 2d convolution input.
		int32 InputWidth = 0;

		// Number of input channels.
		int32 InChannels = 0;

		// Number of output channels after convolution.
		int32 OutChannels = 0;

		// Size of the convolving kernel.
		int32 KernelSize = 3;

		// Size of the convolution stride.
		int32 Stride = 1;

		// The amount of padding to add to both sides of the input.
		int32 Padding = 1;

		// Zeros or Circular.
		EPaddingMode PaddingMode = EPaddingMode::Zeros;

		// The activation function to use for the convolution layer.
		EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ReLU;
	};

	struct FSchemaSparseParameters
	{
		// Number of values in the sparse observation
		int32 Num = 0;
	};

	struct FSchemaNamedSparseParameters
	{
		// Names of the sparse observation elements.
		TArrayView<const FName> ElementNames;
	};

	/**
	 * Observation Schema
	 *
	 * This object allows you to construct a description of the kind of observations you might want to provide as input to a policy. Internally this
	 * object contains a pool of individual elements. This allows them to be constructed performantly and in a cache efficient way. This object is
	 * therefore required to access any data about the individual observation elements that are created.
	 */
	struct FSchema
	{
		UE_API FSchemaElement CreateNull(const FName Tag = NAME_None);
		UE_API FSchemaElement CreateContinuous(const FSchemaContinuousParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateDiscreteExclusive(const FSchemaDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateDiscreteInclusive(const FSchemaDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateNamedDiscreteExclusive(const FSchemaNamedDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateNamedDiscreteInclusive(const FSchemaNamedDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateAnd(const FSchemaAndParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateOrExclusive(const FSchemaOrExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateOrInclusive(const FSchemaOrInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateArray(const FSchemaArrayParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateSet(const FSchemaSetParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateEncoding(const FSchemaEncodingParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateConv1d(const FSchemaConv1dParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateConv2d(const FSchemaConv2dParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateSparse(const FSchemaSparseParameters Parameters, const FName Tag = NAME_None);
		UE_API FSchemaElement CreateNamedSparse(const FSchemaNamedSparseParameters Parameters, const FName Tag = NAME_None);

		UE_API FSchemaContinuousParameters GetContinuous(const FSchemaElement Element) const;
		UE_API FSchemaDiscreteExclusiveParameters GetDiscreteExclusive(const FSchemaElement Element) const;
		UE_API FSchemaDiscreteInclusiveParameters GetDiscreteInclusive(const FSchemaElement Element) const;
		UE_API FSchemaNamedDiscreteExclusiveParameters GetNamedDiscreteExclusive(const FSchemaElement Element) const;
		UE_API FSchemaNamedDiscreteInclusiveParameters GetNamedDiscreteInclusive(const FSchemaElement Element) const;
		UE_API FSchemaAndParameters GetAnd(const FSchemaElement Element) const;
		UE_API FSchemaOrExclusiveParameters GetOrExclusive(const FSchemaElement Element) const;
		UE_API FSchemaOrInclusiveParameters GetOrInclusive(const FSchemaElement Element) const;
		UE_API FSchemaArrayParameters GetArray(const FSchemaElement Element) const;
		UE_API FSchemaSetParameters GetSet(const FSchemaElement Element) const;
		UE_API FSchemaEncodingParameters GetEncoding(const FSchemaElement Element) const;
		UE_API FSchemaConv1dParameters GetConv1d(const FSchemaElement Element) const;
		UE_API FSchemaConv2dParameters GetConv2d(const FSchemaElement Element) const;
		UE_API FSchemaSparseParameters GetSparse(const FSchemaElement Element) const;
		UE_API FSchemaNamedSparseParameters GetNamedSparse(const FSchemaElement Element) const;

		// Checks if the given element is valid
		UE_API bool IsValid(const FSchemaElement Element) const;

		// Gets the type of the given element
		UE_API EType GetType(const FSchemaElement Element) const;

		// Gets the tag of the given element
		UE_API FName GetTag(const FSchemaElement Element) const;

		// Get the observation vector size of the given element
		UE_API int32 GetObservationVectorSize(const FSchemaElement Element) const;

		// Get the encoded vector size of the given element
		UE_API int32 GetEncodedVectorSize(const FSchemaElement Element) const;

		// Get the observation distribution vector size of the given element
		UE_API int32 GetObservationDistributionVectorSize(const FSchemaElement Element) const;

		// Set the current generation
		UE_API void SetGeneration(const uint32 InGeneration);

		// Get the current generation
		UE_API uint32 GetGeneration() const;

		// Checks if the given schema is empty of elements.
		UE_API bool IsEmpty() const;

		// Empty all internal buffers of elements. This invalidates all existing elements.
		UE_API void Empty();

		// Reset all internal buffers (without freeing memory). This invalidates all existing elements.
		UE_API void Reset();

		// Serializes an observation schema. Returns false if serialization fails.
		UE_API bool Serialize(FArchive& Ar);

	private:

		struct FContinuousData
		{
			int32 Num = INDEX_NONE;
			int32 NormalizationIndexOffset = INDEX_NONE;
			ENormalization NormalizationOption = ENormalization::Manual;
		};

		struct FDiscreteExclusiveData
		{
			int32 Num = INDEX_NONE;
		};

		struct FDiscreteInclusiveData
		{
			int32 Num = INDEX_NONE;
		};

		struct FNamedDiscreteExclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FNamedDiscreteInclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FAndData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		struct FOrExclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
			int32 EncodingSize = INDEX_NONE;
		};

		struct FOrInclusiveData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
			int32 AttentionEncodingSize = INDEX_NONE;
			int32 AttentionHeadNum = INDEX_NONE;
			int32 ValueEncodingSize = INDEX_NONE;
		};

		struct FArrayData
		{
			int32 Num = INDEX_NONE;
			int32 ElementIndex = INDEX_NONE;
		};

		struct FSetData
		{
			int32 MaxNum = INDEX_NONE;
			int32 ElementIndex = INDEX_NONE;
			int32 AttentionEncodingSize = INDEX_NONE;
			int32 AttentionHeadNum = INDEX_NONE;
			int32 ValueEncodingSize = INDEX_NONE;
		};

		struct FEncodingData
		{
			int32 ElementIndex = INDEX_NONE;
			int32 EncodingSize = INDEX_NONE;
			int32 LayerNum = INDEX_NONE;
			EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ELU;
		};

		struct FConv1dData
		{
			int32 ElementIndex = INDEX_NONE;
			int32 InputLength = INDEX_NONE;
			int32 InChannels = INDEX_NONE;
			int32 OutChannels = INDEX_NONE;
			int32 KernelSize = INDEX_NONE;
			int32 Padding = INDEX_NONE;
			EPaddingMode PaddingMode = EPaddingMode::Zeros;
			EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ReLU;
		};

		struct FConv2dData
		{
			int32 ElementIndex = INDEX_NONE;
			int32 InputHeight = INDEX_NONE;
			int32 InputWidth = INDEX_NONE;
			int32 InChannels = INDEX_NONE;
			int32 OutChannels = INDEX_NONE;
			int32 KernelSize = INDEX_NONE;
			int32 Stride = INDEX_NONE;
			int32 Padding = INDEX_NONE;
			EPaddingMode PaddingMode = EPaddingMode::Zeros;
			EEncodingActivationFunction ActivationFunction = EEncodingActivationFunction::ReLU;
		};

		struct FSparseData
		{
			int32 Num = INDEX_NONE;
		};

		struct FNamedSparseData
		{
			int32 Num = INDEX_NONE;
			int32 ElementsOffset = INDEX_NONE;
		};

		uint32 Generation = 0;

		/** These have entries for each Schema Element */
		TArray<EType> Types;
		TArray<FName> Tags;
		TArray<int32> ObservationVectorSizes;
		TArray<int32> EncodedVectorSizes;
		TArray<int32> ObservationDistributionVectorSizes;
		TArray<int32> TypeDataIndices;

		/** These are indexed based on the Type using TypeDataIndices */
		TArray<FContinuousData> ContinuousData;
		TArray<FDiscreteExclusiveData> DiscreteExclusiveData;
		TArray<FDiscreteInclusiveData> DiscreteInclusiveData;
		TArray<FNamedDiscreteExclusiveData> NamedDiscreteExclusiveData;
		TArray<FNamedDiscreteInclusiveData> NamedDiscreteInclusiveData;
		TArray<FAndData> AndData;
		TArray<FOrExclusiveData> OrExclusiveData;
		TArray<FOrInclusiveData> OrInclusiveData;
		TArray<FArrayData> ArrayData;
		TArray<FSetData> SetData;
		TArray<FEncodingData> EncodingData;
		TArray<FConv1dData> Conv1dData;
		TArray<FConv2dData> Conv2dData;
		TArray<FSparseData> SparseData;
		TArray<FNamedSparseData> NamedSparseData;

		/** This is an array of all the SubElements and their names, referenced by other elements */
		TArray<FName> SubElementNames;
		TArray<FSchemaElement> SubElementObjects;

		/** Normalization arrays for continuous observations, indexed by NormalizationOffset in FContinuousData */
		TArray<float> NormalizationScales;
		TArray<float> NormalizationOffsets;
	};

	/**
	 * Observation Object Element
	 *
	 * A single element in the observation object representing part of an observation. Internally this consists of a index used by the object to look
	 * up the associated observation data and a generation id which can be used to check when this index is no longer valid.
	 */
	struct FObjectElement { int32 Index = INDEX_NONE; uint32 Generation = INDEX_NONE; };

	struct FObjectContinuousParameters
	{
		// Continuous observation values
		TArrayView<const float> Values;
	};

	struct FObjectDiscreteExclusiveParameters
	{
		// Index of the chosen observation.
		int32 DiscreteIndex;
	};

	struct FObjectDiscreteInclusiveParameters
	{
		// Indices of the chosen observations.
		TArrayView<const int32> DiscreteIndices;
	};

	struct FObjectNamedDiscreteExclusiveParameters
	{
		// Name of the chosen observation.
		FName ElementName;
	};

	struct FObjectNamedDiscreteInclusiveParameters
	{
		// Names of the chosen observations.
		TArrayView<const FName> ElementNames;
	};

	struct FObjectAndParameters
	{
		// Names of the sub-observations.
		TArrayView<const FName> ElementNames;

		// The associated sub-element.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectOrExclusiveParameters
	{
		// Name of the chosen sub-observation.
		FName ElementName;

		// The associated chosen sub-element.
		FObjectElement Element;
	};

	struct FObjectOrInclusiveParameters
	{
		// Names of the chosen sub-observations.
		TArrayView<const FName> ElementNames;

		// The associated chosen sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectArrayParameters
	{
		// Array of sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectSetParameters
	{
		// Set of sub-elements.
		TArrayView<const FObjectElement> Elements;
	};

	struct FObjectEncodingParameters
	{
		// Encoded sub-element.
		FObjectElement Element;
	};

	struct FObjectConv1dParameters
	{
		// Encoded sub-element.
		FObjectElement Element;
	};

	struct FObjectConv2dParameters
	{
		// Encoded sub-element.
		FObjectElement Element;
	};

	struct FObjectSparseParameters
	{
		// Indices of the sparse observations.
		TArrayView<const int32> DiscreteIndices;

		// Sparse observation values
		TArrayView<const float> Values;
	};

	struct FObjectNamedSparseParameters
	{
		// Names of the sparse observations.
		TArrayView<const FName> ElementNames;

		// Sparse observation values
		TArrayView<const float> Values;
	};

	/**
	 * Observation Object
	 *
	 * This object allows you to construct or get data from an instance of an observation you might give as input to a policy. Internally this object
	 * contains a pool of individual elements. This allows them to be constructed performantly and in a cache efficient way. This object is therefore
	 * required to access any data about the individual observation elements that are created.
	 */
	struct FObject
	{
		UE_API FObjectElement CreateNull(const FName Tag = NAME_None);
		UE_API FObjectElement CreateContinuous(const FObjectContinuousParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateDiscreteExclusive(const FObjectDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateDiscreteInclusive(const FObjectDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateNamedDiscreteExclusive(const FObjectNamedDiscreteExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateNamedDiscreteInclusive(const FObjectNamedDiscreteInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateAnd(const FObjectAndParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateOrExclusive(const FObjectOrExclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateOrInclusive(const FObjectOrInclusiveParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateArray(const FObjectArrayParameters Parametes, const FName Tag = NAME_None);
		UE_API FObjectElement CreateSet(const FObjectSetParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateEncoding(const FObjectEncodingParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateConv1d(const FObjectConv1dParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateConv2d(const FObjectConv2dParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateSparse(const FObjectSparseParameters Parameters, const FName Tag = NAME_None);
		UE_API FObjectElement CreateNamedSparse(const FObjectNamedSparseParameters Parameters, const FName Tag = NAME_None);

		UE_API FObjectContinuousParameters GetContinuous(const FObjectElement Element) const;
		UE_API FObjectDiscreteExclusiveParameters GetDiscreteExclusive(const FObjectElement Element) const;
		UE_API FObjectDiscreteInclusiveParameters GetDiscreteInclusive(const FObjectElement Element) const;
		UE_API FObjectNamedDiscreteExclusiveParameters GetNamedDiscreteExclusive(const FObjectElement Element) const;
		UE_API FObjectNamedDiscreteInclusiveParameters GetNamedDiscreteInclusive(const FObjectElement Element) const;
		UE_API FObjectAndParameters GetAnd(const FObjectElement Element) const;
		UE_API FObjectOrExclusiveParameters GetOrExclusive(const FObjectElement Element) const;
		UE_API FObjectOrInclusiveParameters GetOrInclusive(const FObjectElement Element) const;
		UE_API FObjectArrayParameters GetArray(const FObjectElement Element) const;
		UE_API FObjectSetParameters GetSet(const FObjectElement Element) const;
		UE_API FObjectEncodingParameters GetEncoding(const FObjectElement Element) const;
		UE_API FObjectConv1dParameters GetConv1d(const FObjectElement Element) const;
		UE_API FObjectConv2dParameters GetConv2d(const FObjectElement Element) const;
		UE_API FObjectSparseParameters GetSparse(const FObjectElement Element) const;
		UE_API FObjectNamedSparseParameters GetNamedSparse(const FObjectElement Element) const;

		// Checks if the given element is valid
		UE_API bool IsValid(const FObjectElement Element) const;

		// Gets the type of the given element
		UE_API EType GetType(const FObjectElement Element) const;

		// Gets the tag of the given element
		UE_API FName GetTag(const FObjectElement Element) const;

		// Set the current generation
		UE_API void SetGeneration(const uint32 InGeneration);

		// Get the current generation
		UE_API uint32 GetGeneration() const;

		// Checks if the given object is empty of elements.
		UE_API bool IsEmpty() const;

		// Empty all internal buffers of elements. This invalidates all existing elements.
		UE_API void Empty();

		// Reset all internal buffers (without freeing memory). This invalidates all existing elements.
		UE_API void Reset();

		// Serializes an observation object. Returns false if serialization fails.
		UE_API bool Serialize(FArchive& Ar);

	private:

		uint32 Generation = 0;

		TArray<EType> Types;
		TArray<FName> Tags;
		TArray<int32> ContinuousDataOffsets;
		TArray<int32> ContinuousDataNums;
		TArray<int32> DiscreteDataOffsets;
		TArray<int32> DiscreteDataNums;
		TArray<int32> SubElementDataOffsets;
		TArray<int32> SubElementDataNums;

		TArray<float> ContinuousValues;
		TArray<int32> DiscreteValues;
		TArray<FName> SubElementNames;
		TArray<FObjectElement> SubElementObjects;
	};

	/**
	 * Gets a hash value representing object compatibility between schemas i.e. if objects from one schema can be used by objects expecting another
	 * schema. This is not a cryptographic hash, and so `AreSchemaObjectsCompatible` should still be used as the ultimate source of truth.
	 * This function returns an int32 so that it can be used in blueprints.
	 *
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 * @param Salt						Hash salt
	 */
	UE_API int32 GetSchemaObjectsCompatibilityHash(
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const int32 Salt = 0x62625165);

	/**
	 * Test if two schemas are compatible i.e. if objects from one schema can be used by objects expecting another schema.
	 *
	 * @param SchemaA					First Schema
	 * @param SchemaElementA			First Schema Element
	 * @param SchemaB					Second Schema
	 * @param SchemaElementB			Second Schema Element
	 * @returns							true when the schemas are compatible
	 */
	UE_API bool AreSchemaObjectsCompatible(
		const FSchema& SchemaA,
		const FSchemaElement SchemaElementA,
		const FSchema& SchemaB,
		const FSchemaElement SchemaElementB);

	/** Network weight initialization type */
	enum class EWeightInitialization : uint8
	{
		KaimingGaussian = 0,
		KaimingUniform = 1,
	};

	/**
	 * Settings for building a network from a schema.
	 */
	struct FNetworkSettings
	{
		// If to use compressed linear layers. This reduces the memory usage by half at the cost of some small impact on quality and evaluation speed.
		bool bUseCompressedLinearLayers = false;

		// Which weight initialization to use.
		EWeightInitialization WeightInitialization = EWeightInitialization::KaimingGaussian;
	};

	/**
	 * Make a NNE::RuntimeBasic::FModelBuilderElement for the given Schema. This can be used if you want to plug the Encoder generated by this
	 * schema as a part of a larger model you are making with a NNE::RuntimeBasic::FModelBuilder.
	 *
	 * @param OutElement				Output Builder Element
	 * @param Builder					Model Builder
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 * @param NetworkSettings			Network Generation Settings
	 */
	UE_API void MakeEncoderNetworkModelBuilderElementFromSchema(
		NNE::RuntimeBasic::FModelBuilderElement& OutElement,
		NNE::RuntimeBasic::FModelBuilder& Builder,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings = FNetworkSettings());

	/**
	 * Generate FileData for a Neural Network that can act as a Encoder for the given schema. This network will take as input a vector of size
	 * ObservationVectorSize and produce a vector of size EncodedVectorSize.
	 *
	 * @param OutFileData				Output File Data to write to
	 * @param OutInputSize				Size of the vector this network takes as input
	 * @param OutOutputSize				Size of the vector this network takes as output
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 * @param NetworkSettings			Network Generation Settings
	 * @param Seed						The random seed used in initializing the network weights
	 */
	UE_API void GenerateEncoderNetworkFileDataFromSchema(
		TArray<uint8>& OutFileData,
		uint32& OutInputSize,
		uint32& OutOutputSize,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FNetworkSettings& NetworkSettings = FNetworkSettings(),
		const uint32 Seed = 0x08ab1c49);

	/**
	 * Convert an observation object into an observation vector.
	 *
	 * @param OutObservationVector		Output observation vector
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 * @param Object					Observation Object
	 * @param ObjectElement				Observation Object Element
	 */
	UE_API void SetVectorFromObject(
		TLearningArrayView<1, float> OutObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const FObject& Object,
		const FObjectElement ObjectElement);

	/**
	 * Convert an observation vector into an observation object
	 *
	 * @param OutObject					Output Observation Object
	 * @param OutObjectElement			Output Observation Object Element
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 * @param ObservationVector			Input Observation vector
	 */
	UE_API void GetObjectFromVector(
		FObject& OutObject,
		FObjectElement& OutObjectElement,
		const FSchema& Schema,
		const FSchemaElement SchemaElement,
		const TLearningArrayView<1, const float> ObservationVector);

	/**
	 * Outputs a vector mask which has 1s for entries in the observation vector which correspond to continuous observation values and 0s elsewhere. 
	 * This can be useful if you want to apply some kind of numerical modifications to observation vectors such as adding noise or clamping, without 
	 * breaking the parts of the vector which encode the structure.  
	 *
	 * @param OutObservationVectorMask	Observation vector mask
	 * @param InObservationVector		Input observation vector
	 * @param Schema					Observation Schema
	 * @param SchemaElement				Observation Schema Element
	 */
	UE_API void GetContinuousObservationVectorMask(
		TLearningArrayView<1, bool> OutObservationVectorMask,
		const TLearningArrayView<1, const float> InObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement);

	/**
	 * Initializes mins and maxs for observation distribution vectors. Should be called before any calls to FitObservationDistributionMinMax
	 *
	 * @param OutObservationDistributionMin		Minimum values for observations
	 * @param OutObservationDistributionMax		Maximum values for observations
	 */
	UE_API void InitObservationDistributionMinMax(
		TLearningArrayView<1, float> OutObservationDistributionMin,
		TLearningArrayView<1, float> OutObservationDistributionMax);

	/**
	 * Fits the mins and the maxs of each continuous observation in the observation vector. Typically you will want to call this function 
	 * in a loop over multiple observation vectors to refine the fitted mins and maxs since this updates them in place.
	 *
	 * @param InOutObservationDistributionMin		Minimum values for observations
	 * @param InOutObservationDistributionMax		Maximum values for observations
	 * @param InObservationVector					Observation vector to fit the mins and maxs from
	 * @param Schema								Observation Schema
	 * @param SchemaElement							Observation Schema Element
	 */
	UE_API void FitObservationDistributionMinMax(
		TLearningArrayView<1, float> InOutObservationDistributionMin,
		TLearningArrayView<1, float> InOutObservationDistributionMax,
		const TLearningArrayView<1, const float> InObservationVector,
		const FSchema& Schema,
		const FSchemaElement SchemaElement);

	/**
	 * Clamps an observation vector using mins and maxs given by two observation distribution vectors.
	 *
	 * @param InOutObservationVector				Observation vector to clamp
	 * @param ObservationDistributionMin			Minimum values for observations
	 * @param ObservationDistributionMax			Maximum values for observations
	 * @param Schema								Observation Schema
	 * @param SchemaElement							Observation Schema Element
	 */
	UE_API void ClampToObservationDistributionMinMax(
		TLearningArrayView<1, float> InOutObservationVector,
		const TLearningArrayView<1, const float> ObservationDistributionMin,
		const TLearningArrayView<1, const float> ObservationDistributionMax,
		const FSchema& Schema,
		const FSchemaElement SchemaElement);

}

UE_API FArchive& operator<<(FArchive& Ar, UE::Learning::Observation::FSchemaElement& SchemaElement);
UE_API FArchive& operator<<(FArchive& Ar, UE::Learning::Observation::FObjectElement& ObjectElement);
UE_API bool operator==(const UE::Learning::Observation::FSchemaElement& Lhs, const UE::Learning::Observation::FSchemaElement& Rhs);
UE_API bool operator==(const UE::Learning::Observation::FObjectElement& Lhs, const UE::Learning::Observation::FObjectElement& Rhs);
UE_API uint32 GetTypeHash(const UE::Learning::Observation::FSchemaElement& SchemaElement);
UE_API uint32 GetTypeHash(const UE::Learning::Observation::FObjectElement& ObjectElement);

#undef UE_API
