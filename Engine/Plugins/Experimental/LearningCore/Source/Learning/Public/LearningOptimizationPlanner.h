// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	struct IOptimizer;
	struct FProgress;

	/**
	* Basic buffer containing some additional data used during optimization planning
	*/
	struct FOptimizationPlannerBuffer
	{
		UE_API void Resize(
			const int32 SampleNum,
			const int32 StepNum,
			const int32 ActionVectorDimensionNum);

		TLearningArray<3, float> Samples;
		TLearningArray<1, float> Losses;
	};

	namespace OptimizationPlanner
	{

		/**
		* Runs a sequence of action vectors for a set instances
		*
		* @param ActionVectorBuffer				Buffer to write action vectors into
		* @param ResetFunction					Function to run for resetting the environment
		* @param ActionFunction					Function to run for evaluating actions
		* @param UpdateFunction					Function to run for updating the environment
		* @param ActionVectors					Action Vectors of shape (StepNum, InstanceNum, ActionVectorDimNum)
		* @param Instances						Set of instances to run action vectors for
		*/
		UE_API void RunPlan(
			TLearningArrayView<2, float> ActionVectorBuffer,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TLearningArrayView<3, const float> ActionVectors,
			const FIndexSet Instances);

		/**
		* Run the optimization based planner on the provided action vectors to maximize the reward
		*
		* @param InOutActionVectors				Action vectors to adjust of shape (StepNum, ActionVectorDimNum)
		* @param OptimizationPlannerBuffer		Buffer for the optimization planner
		* @param Optimizer						Optimizer to use
		* @param ActionVectorBuffer				Buffer to read/write action vectors into
		* @param RewardBuffer					Buffer to read/write rewards into
		* @param IterationNum					Number of iterations to run the planner for
		* @param ResetFunction					Function to run for resetting the environment
		* @param ActionFunction					Function to run for evaluating actions
		* @param UpdateFunction					Function to run for updating the environment
		* @param RewardFunction					Function to run for evaluating rewards
		* @param SampleInstances				Instances to use for sampling during the optimization
		* @param LogSettings					Log settings to use
		* @param Progress						Optional progress to record progress in
		* @param ActionVectorsLock				Optional lock to use when updating InOutActionVectors
		* @param bActionVectorsUpdatedFlag		Optional signal to set when updating InOutActionVectors
		*/
		UE_API void Plan(
			TLearningArrayView<2, float> InOutActionVectors,
			FOptimizationPlannerBuffer& OptimizationPlannerBuffer,
			IOptimizer& Optimizer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			const int32 IterationNum,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const FIndexSet SampleInstances,
			const ELogSetting LogSettings = ELogSetting::Normal,
			FProgress* Progress = nullptr,
			FRWLock* ActionVectorsLock = nullptr,
			TAtomic<bool>* bActionVectorsUpdatedFlag = nullptr);

		/**
		* Run the optimization based planner windowed on the provided action vectors to maximize the reward.
		* 
		* In windowed mode the optimization planner will only optimize a fixed window of steps over the action
		* vectors at a time. This gives the optimizer an easier job, but limits how for the system can plan ahead.
		*
		* @param InOutActionVectors				Action vectors to adjust of shape (StepNum, ActionVectorDimNum)
		* @param OptimizationPlannerBuffer		Buffer for the optimization planner
		* @param Optimizer						Optimizer to use
		* @param ActionVectorBuffer				Buffer to read/write action vectors into
		* @param RewardBuffer					Buffer to read/write rewards into
		* @param InitialIterationsNum			Number of iterations to run the planner for on the initial window
		* @param FinalIterationsNum				Number of iterations to run the planner for on the final window
		* @param WindowIterationsNum			Number of iterations to run the planner for on the intermediate windows
		* @param WindowStepNum					Number of steps to consider in the window
		* @param ResetFunction					Function to run for resetting the environment
		* @param ResetFromInstanceFunction		Function to run to reset the planning instances from the state of single instance
		* @param ActionFunction					Function to run for evaluating actions
		* @param UpdateFunction					Function to run for updating the environment
		* @param RewardFunction					Function to run for evaluating rewards
		* @param PlanInstance					The instance used to execute the plan. Should not be included in the SampleInstances.
		* @param SampleInstances				Instances to use for sampling during the optimization
		* @param LogSettings					Log settings to use
		* @param Progress						Optional progress to record progress in
		* @param ActionVectorsLock				Optional lock to use when updating InOutActionVectors
		* @param bActionVectorsUpdatedFlag		Optional signal to set when updating InOutActionVectors
		*/
		UE_API void PlanWindowed(
			TLearningArrayView<2, float> InOutActionVectors,
			FOptimizationPlannerBuffer& OptimizationPlannerBuffer,
			IOptimizer& Optimizer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			const int32 InitialIterationsNum,
			const int32 FinalIterationsNum,
			const int32 WindowIterationsNum,
			const int32 WindowStepNum,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances, const int32 Instance)> ResetFromInstanceFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const int32 PlanInstance,
			const FIndexSet SampleInstances,
			const ELogSetting LogSettings = ELogSetting::Normal,
			FProgress* Progress = nullptr,
			FRWLock* ActionVectorsLock = nullptr,
			TAtomic<bool>* bActionVectorsUpdatedFlag = nullptr);

	}
}

#undef UE_API
