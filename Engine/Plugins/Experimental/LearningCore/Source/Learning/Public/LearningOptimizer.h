// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"

namespace UE::Learning
{
	/**
	* Interface for an Optimizer
	*/
	struct IOptimizer
	{
		virtual ~IOptimizer() = default;

		/**
		* Resize the Optimizer
		*
		* @param SampleNum			Number of samples to allocate space for
		* @param DimensionsNum		Number of dimensions to allocate space for
		*/
		virtual void Resize(const int32 SampleNum, const int32 DimensionsNum) = 0;

		/**
		* Reset the Optimizer
		*
		* @param OutSamples			Output parameters for which to compute the loss function of shape (SamplesNum, DimensionsNum)
		* @param InitialGuess		Initial guess at the best solution of shape (DimensionsNum). Can be set to zero if no initial guess is known.
		*/
		virtual void Reset(
			TLearningArrayView<2, float> OutSamples,
			const TLearningArrayView<1, const float> InitialGuess) = 0;

		/**
		* Update the Optimizer
		*
		* @param InOutSamples		Samples to adjust the values of in-place of shape (SamplesNum, DimensionsNum)
		* @param Losses				Losses associated with those samples of shape (SamplesNum)
		* @param LogSettings		Log settings
		*/
		virtual void Update(
			TLearningArrayView<2, float> InOutSamples,
			const TLearningArrayView<1, const float> Losses,
			const ELogSetting LogSettings = ELogSetting::Normal) = 0;
	};
}

