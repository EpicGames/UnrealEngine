// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	struct FProgress;

	/**
	* PCA settings. The resulting number of dimensions will be based on
	* whatever is lower between `MaximumDimensions`, and however many
	* dimensions preserve the given `MaximumVarianceRatio`. If the PCA
	* computation is failing ensure `bStableComputation` is set to true
	* as the non-stable version fails more often than I would expect.
	*/
	struct FPCASettings
	{
		// Rate at which to subsample the data when computing the PCA encoding. 
		// Higher values will make the PCA computation faster but less accurate.
		int32 Subsample = 1;

		// Maximum number of dimensions to keep
		int32 MaximumDimensions = 128;

		// Maximum variance ratio to preserve
		float MaximumVarianceRatio = 0.99f;

		// Use a slower but stable version of the computation
		bool bStableComputation = true;
	};

	/**
	* Details of the result of the PCA fitting.
	*/
	struct FPCAResult
	{
		// Was the PCA transform fit successfully
		bool bSuccess = false;

		// How many dimensions were kept by the PCA transform
		int32 DimensionNum = 0;

		// Number between 0 and 1 saying how much of the variance is preserved
		float VarianceRatioPreserved = 0.0f;
	};

	/**
	* Represents a PCA transformation which can be used to encode data
	*/
	struct FPCAEncoder
	{
		TLearningArray<2, float> Matrix;

		UE_API void Empty();
		UE_API bool IsEmpty() const;
		UE_API bool Serialize(FArchive& Ar);

		UE_API int32 DimensionNum() const;
		UE_API int32 FeatureNum() const;

		/**
		* Fits a PCA transform to some data
		*
		* @param Data				Data to fit the PCA transformation to of shape (SampleNum, FeatureNum). Assumes data is already centered and normalized.
		* @param Settings			Fitting settings
		* @param LogSettings		Logging settings
		* @param Progress			Optional progress object to report progress
		* @returns					Result of PCA fitting process. Used to check for success.
		*/
		UE_API FPCAResult Fit(
			const TLearningArrayView<2, const float> Data,
			const FPCASettings& Settings,
			const ELogSetting LogSettings = ELogSetting::Normal,
			FProgress* Progress = nullptr);

		/**
		* Transform multiple vectors using the PCA transformation
		*
		* @param OutData			Output transformed data of shape (SampleNum, DimensionNum)
		* @param Data				Data to transform of shape (SampleNum, FeatureNum)
		* @param Progress			Optional progress object to report progress
		*/
		UE_API void Transform(
			TLearningArrayView<2, float> OutData,
			const TLearningArrayView<2, const float> Data,
			FProgress* Progress = nullptr) const;

		/**
		* Transform a single vector using the PCA transformation
		*
		* @param OutData			Output transformed data of shape (DimensionNum)
		* @param Data				Data to transform of shape (FeatureNum)
		*/
		UE_API void Transform(
			TLearningArrayView<1, float> OutData,
			const TLearningArrayView<1, const float> Data) const;

		/**
		* Inverse Transform a single vector using the PCA transformation
		*
		* @param OutData			Output transformed data of shape (FeatureNum)
		* @param Data				Data to transform of shape (DimensionNum)
		*/
		UE_API void InverseTransform(
			TLearningArrayView<1, float> OutData,
			const TLearningArrayView<1, const float> Data) const;
	};


};

#undef UE_API
