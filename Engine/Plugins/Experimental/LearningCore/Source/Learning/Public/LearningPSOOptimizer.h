// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningOptimizer.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Settings for the PSO Optimizer. Defaults from https://www.jstage.jst.go.jp/article/imt/3/1/3_1_103/_pdf/-char/ja
	*/
	struct FPSOOptimizerSettings
	{
		// Amount to pull particles toward their local best
		float LocalGain = 1.49455f;

		// Amount to pull particles toward the global best
		float GlobalGain = 1.49455f;

		// How much of the particle velocity to preserve between steps
		float Momentum = 0.729f;
	};

	/**
	* PSO Optimizer
	*
	* Gradient free optimizer based on Particle Swarm Optimization
	*/
	struct FPSOOptimizer : public IOptimizer
	{
		UE_API FPSOOptimizer(const uint32 Seed, const FPSOOptimizerSettings& InSettings = FPSOOptimizerSettings());

		UE_API virtual void Resize(const int32 SampleNum, const int32 DimensionsNum) override final;

		UE_API virtual void Reset(
			TLearningArrayView<2, float> OutSamples,
			const TLearningArrayView<1, const float> InitialGuess) override final;

		UE_API virtual void Update(
			TLearningArrayView<2, float> InOutSamples,
			const TLearningArrayView<1, const float> Losses,
			const ELogSetting LogSettings = ELogSetting::Normal) override final;

	private:

		uint32 Seed = 0;
		FPSOOptimizerSettings Settings;

		int32 Iterations = 0;
		TLearningArray<2, float> LocalBestPositions;
		TLearningArray<1, float> LocalBestLoss;
		TLearningArray<1, float> GlobalBestPosition;
		float GlobalBestLoss = 0.0f;
		TLearningArray<2, float> Velocities;

		TLearningArray<2, float> LocalUniformSamples;
		TLearningArray<2, float> GlobalUniformSamples;
	};
}


#undef UE_API
