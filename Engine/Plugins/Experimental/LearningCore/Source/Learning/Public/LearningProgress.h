// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Internationalization/Text.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	/**
	* Simple thread-safe structure to record progress of some long-running computation.
	*/
	struct FProgress
	{
		UE_API int32 GetProgress() const;
		UE_API void SetProgress(const int32 InProgress);

		UE_API void GetMessage(FText& OutMessage);
		UE_API void SetMessage(const FText& InMessage);

		UE_API void Decrement();
		UE_API void Decrement(const int32 Num);
		UE_API void Done();

	private:
		FRWLock Lock;
		FText Message;
		TAtomic<int32> Progress = 0;
	};

	/**
	* Scoped read lock, which can optionally be null
	*/
	struct FScopeNullableReadLock
	{
	public:

		UE_NODISCARD_CTOR UE_API FScopeNullableReadLock(FRWLock* InLock);
		UE_API ~FScopeNullableReadLock();

		FScopeNullableReadLock() = delete;
		FScopeNullableReadLock(const FScopeNullableReadLock& InScopeLock) = delete;
		FScopeNullableReadLock& operator=(FScopeNullableReadLock& InScopeLock) = delete;

	private:
		FRWLock* Lock = nullptr;
	};

	/**
	* Scoped write lock, which can optionally be null
	*/
	struct FScopeNullableWriteLock
	{
	public:

		UE_NODISCARD_CTOR UE_API FScopeNullableWriteLock(FRWLock* InLock);
		UE_API ~FScopeNullableWriteLock();

		FScopeNullableWriteLock() = delete;
		FScopeNullableWriteLock(const FScopeNullableWriteLock& InScopeLock) = delete;
		FScopeNullableWriteLock& operator=(FScopeNullableWriteLock& InScopeLock) = delete;

	private:
		FRWLock* Lock = nullptr;
	};

}

#undef UE_API
