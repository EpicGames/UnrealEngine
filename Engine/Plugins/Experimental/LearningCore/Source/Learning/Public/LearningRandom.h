// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Math/Vector.h"
#include "Math/Quat.h"

#define UE_API LEARNING_API

namespace UE::Learning
{
	struct FFrameSet;
	struct FFrameRangeSet;
}

/**
* This is a simple hashing-based pseudo-random-number-generator
* that separates out the generation of random numbers from the
* mutation of the state. This is important for performance when
* generating a lot of random numbers, and with ispc this
* implementation can outperform FRandomStream by up to 100x while
* producing higher quality randomness.
* 
* The basic way it works is that when generating multiple random
* numbers, rather than generating them one after the other each 
* time mutating the state, you instead seed the random number 
* generator individually for each number you want by xoring the 
* state with another random number of your choice:
* 
* 	float R0 = UE::Learning::Random::Float(State ^ 0x6591b5b6);
* 	float R1 = UE::Learning::Random::Float(State ^ 0x88f6747a);
* 
* Once you're done generating numbers you can then mutate the state:
* 
* 	State = UE::Learning::Random::Int(State ^ 0xec664ea3);
* 
* Then, if you want to generate a large array of random numbers,
* each number can be generated independently using the index of
* the array to get the individual seed:
* 
* 	for (int32 Idx = 0; Idx < Num; Idx++)
* 	{
* 		const uint32 Seed = UE::Learning::Random::Int(Idx ^ 0x56df2e17);
* 
* 		Item[Idx] = UE::Learning::Random::Float(State ^ Seed);
* 	}
* 
* 	State = UE::Learning::Random::Int(State ^ 0x017f54f9);
* 
* As each number no longer needs to wait on the previous number
* to be generated and the state mutated, we can port this style
* of generation effectively to ispc and use a vectorized SIMD
* version to make it extremely fast.
*/

namespace UE::Learning::Random
{
	/////////////////////////////////////////////////////////
	//
	// These functions generate random values without mutating the state
	// 
	/////////////////////////////////////////////////////////

	UE_API uint32 Int(const uint32 State);

	UE_API float Float(const uint32 State);

	UE_API int32 IntInRange(const uint32 State, const int32 Min, const int32 Max);

	UE_API float Uniform(
		const uint32 State,
		const float Min = 0.0f,
		const float Max = 1.0f);

	UE_API float Gaussian(
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f);

	UE_API FVector VectorGaussian(
		const uint32 State,
		const FVector Mean = FVector::ZeroVector,
		const FVector Std = FVector::OneVector);

	UE_API float ClippedGaussian(
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f);

	UE_API FVector PlanarUniform(
		const uint32 State,
		const float Min = 0.0f,
		const float Max = 1.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector PlanarGaussian(
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector PlanarClippedGaussian(
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector PlanarDirection(
		const uint32 State,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FQuat Rotation(const uint32 Input);

	UE_API void IntArray(
		TLearningArrayView<1, uint32> Output,
		const uint32 State);

	UE_API void IntArray(
		TLearningArrayView<1, uint32> Output,
		const uint32 State,
		const FIndexSet Indices);

	UE_API void FloatArray(
		TLearningArrayView<1, float> Output,
		const uint32 State);

	UE_API void UniformArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Min = 0.0f,
		const float Max = 1.0f);

	UE_API void GaussianArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f);

	UE_API void ClippedGaussianArray(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f);

	UE_API void PlanarClippedGaussianArray(
		TLearningArrayView<1, FVector> Output,
		const uint32 State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API void PlanarDirectionArray(
		TLearningArrayView<1, FVector> Output,
		const uint32 State,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API void DistributionIndependantNormal(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const float Scale);

	UE_API void DistributionIndependantNormalMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const TLearningArrayView<1, const bool> Masked,
		const TLearningArrayView<1, const float> MaskedValues,
		const float Scale);

	UE_API void DistributionMultinoulli(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale);

	UE_API void DistributionMultinoulliMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale);

	UE_API void DistributionBernoulli(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale);

	UE_API void DistributionBernoulliMasked(
		TLearningArrayView<1, float> Output,
		const uint32 State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale);

	UE_API void FrameInFrameSet(
		int32& OutEntryIdx,
		int32& OutFrameIdx,
		const FFrameSet& FrameSet,
		const uint32 State);

	UE_API void FrameInFrameRangeSet(
		int32& OutEntryIdx,
		int32& OutRangeIdx,
		int32& OutRangeFrame,
		const FFrameRangeSet& FrameRangeSet,
		const uint32 State);

	/////////////////////////////////////////////////////////
	//
	// Here we provide a more normal interface which updates
	// the state in-place each time a new value is generated
	// 
	/////////////////////////////////////////////////////////

	UE_API uint32 SampleInt(uint32& State);

	UE_API float SampleFloat(uint32& State);

	UE_API int32 SampleIntInRange(uint32& State, const int32 Min, const int32 Max);

	UE_API float SampleUniform(
		uint32& State,
		const float Min = 0.0f,
		const float Max = 1.0f);

	UE_API float SampleGaussian(
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f);

	UE_API FVector SampleVectorGaussian(
		uint32& State,
		const FVector Mean = FVector::ZeroVector,
		const FVector Std = FVector::OneVector);

	UE_API float SampleClippedGaussian(
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f);

	UE_API FVector SamplePlanarUniform(
		uint32& State,
		const float Min = 0.0f,
		const float Max = 1.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector SamplePlanarGaussian(
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector SamplePlanarClippedGaussian(
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FVector SamplePlanarDirection(
		uint32& State,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API FQuat SampleRotation(uint32& State);

	UE_API void SampleIntArray(
		TLearningArrayView<1, uint32> Output,
		uint32& State);

	UE_API void SampleIntArray(
		TLearningArrayView<1, uint32> Output,
		uint32& State,
		const FIndexSet Indices);

	UE_API void SampleFloatArray(
		TLearningArrayView<1, float> Output,
		uint32& State);

	UE_API void SampleUniformArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Min = 0.0f,
		const float Max = 1.0f);

	UE_API void SampleGaussianArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f);
	
	UE_API void SampleClippedGaussianArray(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f);

	UE_API void SamplePlanarClippedGaussianArray(
		TLearningArrayView<1, FVector> Output,
		uint32& State,
		const float Mean = 0.0f,
		const float Std = 1.0f,
		const float Clip = 10.0f,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API void SamplePlanarDirectionArray(
		TLearningArrayView<1, FVector> Output,
		uint32& State,
		const FVector Axis0 = FVector::ForwardVector,
		const FVector Axis1 = FVector::RightVector);

	UE_API void SampleDistributionIndependantNormal(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const float Scale);

	UE_API void SampleDistributionIndependantNormalMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> LogStd,
		const TLearningArrayView<1, const bool> Masked,
		const TLearningArrayView<1, const float> MaskedValues,
		const float Scale);

	UE_API void SampleDistributionMultinoulli(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale);

	UE_API void SampleDistributionMultinoulliMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale);

	UE_API void SampleDistributionBernoulli(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const float Scale);

	UE_API void SampleDistributionBernoulliMasked(
		TLearningArrayView<1, float> Output,
		uint32& State,
		const TLearningArrayView<1, const float> Logits,
		const TLearningArrayView<1, const bool> Masked,
		const float Scale);

	UE_API void SampleFrameInFrameSet(
		int32& OutEntryIdx,
		int32& OutFrameIdx,
		uint32& State,
		const FFrameSet& FrameSet);

	UE_API void SampleFrameInFrameRangeSet(
		int32& OutEntryIdx,
		int32& OutRangeIdx,
		int32& OutRangeFrame,
		uint32& State,
		const FFrameRangeSet& FrameRangeSet);
}

#undef UE_API