// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainerElementTypeCompatibility.h"

#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Templates/Less.h"
#include "Templates/Sorting.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/IsConstructible.h"

#include <type_traits>

#include "MultiArrayView.h"

namespace UE::MultiArray::Private
{
	// Copied from Core/Public/Containers/Array.h
	//
	// Assume elements are compatible with themselves - avoids problems with generated copy
	// constructors of arrays of forwarded types, e.g.:
	//
	// struct FThing;
	//
	// struct FOuter
	// {
	//     TMultiArray<2, FThing> Arr; // this will cause errors without this workaround
	// };
	//
	// This should be changed to use std::disjunction and std::is_constructible, and the usage
	// changed to use ::value instead of ::Value, when std::disjunction (C++17) is available everywhere.
	template <typename DestType, typename SourceType>
	constexpr bool TMultiArrayElementsAreCompatible_V = std::is_same_v<DestType, std::decay_t<SourceType>> || std::is_constructible_v<DestType, SourceType>;
}


/**
 * Templated dynamic multi-dimensional array
 *
 * A dynamically sized array of typed elements.  Makes the assumption that your elements are relocatable;
 * i.e. that they can be transparently moved to new memory without a copy constructor. 
 *
 **/
template<uint8 InDimNum, typename InElementType, typename InAllocatorType = FDefaultAllocator>
class TMultiArray
{
public:
	static constexpr uint8 DimNum = InDimNum;
	using ElementType = InElementType;
	using AllocatorType = InAllocatorType;

	using ElementAllocatorType = std::conditional_t<
		AllocatorType::NeedsElementType,
		typename AllocatorType::template ForElementType<ElementType>,
		typename AllocatorType::ForAnyElementType
	>;

	static_assert(DimNum > 1, "TMultiArray requires a positive, non-zero number of dimensions");

public:

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	inline TMultiArrayView<DimNum - 1, ElementType> operator[](int64 Index)
	{
		RangeCheck(0, Index);

		TMultiArrayShape<DimNum - 1> NewShape;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			NewShape[Idx] = ArrayShape[Idx + 1];
		}

		return TMultiArrayView<DimNum - 1, ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 * 
	 * Const version of the above.
	 *
	 * @returns Reference to indexed element.
	 */
	inline TMultiArrayView<DimNum - 1, const ElementType> operator[](int64 Index) const
	{
		RangeCheck(0, Index);

		TMultiArrayShape<DimNum - 1> NewShape;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			NewShape[Idx] = ArrayShape[Idx + 1];
		}

		return TMultiArrayView<DimNum - 1, const ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Flattens the array view into a single dimension.
	 *
	 * @returns Flattened array view.
	 */
	inline TMultiArrayView<1, ElementType> Flatten()
	{
		return TMultiArrayView<1, ElementType>(GetData(), Num());
	}

	/**
	 * Flattens the array view into a single dimension.
	 *
	 * @returns Flattened array view.
	 */
	inline TMultiArrayView<1, const ElementType> Flatten() const
	{
		return TMultiArrayView<1, const ElementType>(GetData(), Num());
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	inline TMultiArrayView<DimNum - 1, ElementType> Flatten(uint8 Dimension)
	{
		checkf((Dimension < DimNum - 1), TEXT("MultiArray flatten dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx + 1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, ElementType>(GetData(), NewShape);
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	inline TMultiArrayView<DimNum - 1, const ElementType> Flatten(uint8 Dimension) const
	{
		checkf((Dimension < DimNum - 1), TEXT("MultiArray flatten dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx + 1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, const ElementType>(GetData(), NewShape);
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	template<uint8 Dimension>
	inline TMultiArrayView<DimNum - 1, ElementType> Flatten()
	{
		static_assert(Dimension < DimNum - 1, "MultiArray flatten dimension out of bounds");

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx + 1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, ElementType>(GetData(), NewShape);
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	template<uint8 Dimension>
	inline TMultiArrayView<DimNum - 1, const ElementType> Flatten() const
	{
		static_assert(Dimension < DimNum - 1, "MultiArray flatten dimension out of bounds");

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx + 1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, const ElementType>(GetData(), NewShape);
	}


public:

	/**
	 * Constructor, initializes element number counters.
	 */
	inline TMultiArray() = default;

	/**
	 * Constructor from a raw array of elements.
	 *
	 * @param Ptr   A pointer to an array of elements to copy.
	 * @param Nums The number of elements on each dimension.
	 */
	inline TMultiArray(const ElementType* Ptr, TMultiArrayShape<DimNum> Shape)
	{
		check(Ptr != nullptr || Shape.Total() == 0);

		CopyToEmpty(Ptr, Shape);
	}

	template <typename OtherElementType>
	inline explicit TMultiArray(const TMultiArrayView<DimNum, OtherElementType>& Other)
		: TMultiArray(Other.GetData(), Other.Shape()) {}

	/**
	 * Copy constructor with changed allocator. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::MultiArray::Private::TMultiArrayElementsAreCompatible_V<ElementType, const OtherElementType&>)
	>
		inline explicit TMultiArray(const TMultiArray<DimNum, OtherElementType, OtherAllocator>& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Shape());
	}

	/**
	 * Copy constructor. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	inline TMultiArray(const TMultiArray& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Shape());
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * AllocatorType changing version.
	 *
	 * @param Other The source array to assign from.
	 */
	template<typename OtherAllocatorType>
	TMultiArray& operator=(const TMultiArray<DimNum, ElementType, OtherAllocatorType>& Other)
	{
		DestructItems(GetData(), Num());
		CopyToEmpty(Other.GetData(), Other.Shape());
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * @param Other The source array to assign from.
	 */
	TMultiArray& operator=(const TMultiArray& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), Num());
			CopyToEmpty(Other.GetData(), Other.Shape());
		}
		return *this;
	}

	template <typename OtherElementType>
	TMultiArray& operator=(const TMultiArrayView<DimNum, OtherElementType>& Other);

private:

	/**
	 * Moves or copies array. Depends on the array type traits.
	 *
	 * This override copies.
	 *
	 * @param ToArray Array to move into.
	 * @param FromArray Array to move from.
	 */
	template <typename FromArrayType, typename ToArrayType>
	static inline void MoveOrCopy(ToArrayType& ToArray, FromArrayType& FromArray)
	{
		ToArray.CopyToEmpty(FromArray.GetData(), FromArray.Shape());
	}

public:
	/**
	 * Move constructor.
	 *
	 * @param Other Array to move from.
	 */
	inline TMultiArray(TMultiArray&& Other)
	{
		MoveOrCopy(*this, Other);
	}

	/**
	 * Move constructor.
	 *
	 * @param Other Array to move from.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::MultiArray::Private::TMultiArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
		inline explicit TMultiArray(TMultiArray<DimNum, OtherElementType, OtherAllocator>&& Other)
	{
		MoveOrCopy(*this, Other);
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other Array to assign and move from.
	 */
	TMultiArray& operator=(TMultiArray&& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), Num());
			MoveOrCopy(*this, Other);
		}
		return *this;
	}

	/** Destructor. */
	~TMultiArray()
	{
		DestructItems(GetData(), Num());

		// note ArrayShape and data pointer are not invalidated
		// they are left unchanged and use-after-destruct will see them the same as before destruct
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline ElementType* GetData()
	{
		return (ElementType*)AllocatorInstance.GetAllocation();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline const ElementType* GetData() const
	{
		return (const ElementType*)AllocatorInstance.GetAllocation();
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline ElementType* begin() { return GetData(); }
	inline ElementType* end() { return GetData() + Num(); }

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline const ElementType* begin() const { return GetData(); }
	inline const ElementType* end() const { return GetData() + Num(); }

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	inline uint32 GetTypeSize() const
	{
		return sizeof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than zero and less
	 * than maximum.
	 */
	inline void CheckInvariants() const
	{
		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			checkSlow(ArrayShape[Idx] >= 0);
		}
	}

	/**
	 * Checks if a dimension is within the allowed number of dimensions
	 *
	 * @param Dimension Dimension of the array.
	 */
	inline void DimensionCheck(uint8 Dimension) const
	{
		checkf((Dimension < DimNum), TEXT("MultiArray dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch
	}

	/**
	 * Checks if index is in dimension range.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Index to check.
	 */
	inline void RangeCheck(uint8 Dimension, int64 Index) const
	{
		DimensionCheck(Dimension);
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayShape[Dimension]), TEXT("Array index out of bounds: %lld from a dimension of size %lld"), (long long)Index, (long long)ArrayShape[Dimension]); // & for one branch
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in dimension range.
	 * Length is 0 is allowed on empty dimensions; Index must be 0 in that case.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	inline void SliceRangeCheck(uint8 Dimension, int64 Index, int64 InNum) const
	{
		DimensionCheck(Dimension);

		checkf(Index >= 0, TEXT("Invalid index (%lld)"), (long long)Index);
		checkf(InNum >= 0, TEXT("Invalid count (%lld)"), (long long)InNum);
		checkf(Index + InNum <= ArrayShape[Dimension], TEXT("Range (index: %lld, count: %lld) lies outside the view of %lld elements"), (long long)Index, (long long)InNum, (long long)ArrayShape[Dimension]);
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return Num() == 0;
	}

	/**
	 * Returns true if the dimension is empty and contains no elements.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns True if the dimension is empty.
	 * @see Num
	 */
	bool IsEmpty(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension] == 0;
	}

	/**
	 * Returns number of dimensions.
	 *
	 * @returns Number of dimensions in array.
	 */
	inline uint8 Rank() const
	{
		return DimNum;
	}

	/**
	 * Returns total number of elements
	 *
	 * @returns Total number of elements in array.
	 */
	inline int64 Num() const
	{
		return ArrayShape.Total();
	}

	/**
	 * Returns the total number of bytes used by the array
	 *
	 * @returns Total number of bytes used by the array.
	 */
	inline int64 NumBytes() const
	{
		return ArrayShape.Total() * sizeof(ElementType);
	}

	/**
	 * Returns number of elements in a dimension.
	 *
	 * @returns Number of elements in array.
	 */
	template<uint8 InDimIdx>
	inline int64 Num() const
	{
		static_assert(InDimIdx < DimNum);
		return ArrayShape[InDimIdx];
	}

	/**
	 * Returns number of elements in a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Number of elements in array.
	 */
	inline int64 Num(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension];
	}

	/**
	 * Returns the number of elements in each dimension.
	 *
	 * @returns Number of elements in each dimension.
	 */
	inline TMultiArrayShape<DimNum> Shape() const
	{
		return ArrayShape;
	}

	/**
	 * Returns the stride for a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Stride of that dimension.
	 */
	inline int64 Stride(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		int64 Total = 1;
		for (uint8 Idx = Dimension + 1; Idx < DimNum; Idx++)
		{
			Total *= ArrayShape[Idx];
		}
		return Total;
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView<DimNum, ElementType> Slice(int64 Index, int64 InNum)
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView<DimNum, ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView<DimNum, const ElementType> Slice(int64 Index, int64 InNum) const
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView<DimNum, const ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename ComparisonType>
	bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + Num(); Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

public:

	/**
	 * Empties the array. It calls the destructors on held items if needed.
	 */
	void Empty()
	{
		int64 CurrTotal = Num();
		DestructItems(GetData(), CurrTotal);
		ArrayShape = TMultiArrayShape<DimNum>();
		AllocatorResizeAllocation(CurrTotal, 0);
	}

	/**
	 * Resizes array to given number of elements.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNum(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
			ConstructItems<ElementType>(GetData() + CurrTotal, GetData(), NewTotal - CurrTotal);
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

	/**
	 * Resizes array to given number of elements. New elements will be zeroed.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNumZeroed(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
			FMemory::Memzero((void*)(GetData() + CurrTotal), (NewTotal - CurrTotal) * sizeof(ElementType));
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

	/**
	 * Resizes array to given number of elements. New elements will be uninitialized.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNumUninitialized(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

private:
	void AllocatorResizeAllocation(int64 CurrentNum, int64 NewNum)
	{
		if constexpr (TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
		{
			AllocatorInstance.ResizeAllocation(CurrentNum, NewNum, sizeof(ElementType), alignof(ElementType));
		}
		else
		{
			AllocatorInstance.ResizeAllocation(CurrentNum, NewNum, sizeof(ElementType));
		}
	}

	/**
	 * Copies data from one array into this array. Uses the fast path if the
	 * data in question does not need a constructor.
	 *
	 * @param OtherData A pointer to the data to copy
	 * @param OtherNums The dimensions of the data to copy
	 */
	template <typename OtherElementType>
	void CopyToEmpty(const OtherElementType* OtherData, TMultiArrayShape<DimNum> OtherShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 OtherTotal = OtherShape.Total();

		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			int64 NewNum = OtherShape[Idx];
			checkf(NewNum == OtherShape[Idx], TEXT("Invalid number of elements to add to this array type: %lld"), (long long)NewNum);

			ArrayShape[Idx] = NewNum;
		}

		int64 NewTotal = ArrayShape.Total();

		AllocatorResizeAllocation(CurrTotal, NewTotal);
		ConstructItems<ElementType>(GetData(), OtherData, OtherTotal);
	}

public:

	const ElementAllocatorType& GetAllocatorInstance() const { return AllocatorInstance; }
	ElementAllocatorType& GetAllocatorInstance() { return AllocatorInstance; }

public:

	inline operator TMultiArrayView<DimNum, ElementType>()
	{
		return TMultiArrayView<DimNum, ElementType>(GetData(), ArrayShape);
	}

	inline operator TMultiArrayView<DimNum, const ElementType>() const
	{
		return TMultiArrayView<DimNum, const ElementType>(GetData(), ArrayShape);
	}

protected:

	ElementAllocatorType AllocatorInstance;
	TMultiArrayShape<DimNum> ArrayShape;
};


/**
* Specialization for single dimensional MultiArray
*/
template<typename InElementType, typename InAllocatorType>
class TMultiArray<1, InElementType, InAllocatorType>
{
public:
	static constexpr uint8 DimNum = 1;
	using ElementType = InElementType;
	using AllocatorType = InAllocatorType;

	using ElementAllocatorType = std::conditional_t<
		AllocatorType::NeedsElementType,
		typename AllocatorType::template ForElementType<ElementType>,
		typename AllocatorType::ForAnyElementType
	>;

	/**
	 * Initializer list constructor
	 */
	TMultiArray(std::initializer_list<InElementType> InitList)
	{
		// This is not strictly legal, as std::initializer_list's iterators are not guaranteed to be pointers, but
		// this appears to be the case on all of our implementations.  Also, if it's not true on a new implementation,
		// it will fail to compile rather than behave badly.
		CopyToEmpty<InElementType>(InitList.begin(), { (int64)InitList.size() });
	}

	/**
	 * Initializer list assignment operator. First deletes all currently contained elements
	 * and then copies from initializer list.
	 *
	 * @param InitList The initializer_list to copy from.
	 */
	TMultiArray& operator=(std::initializer_list<InElementType> InitList)
	{
		DestructItems(GetData(), Num());
		// This is not strictly legal, as std::initializer_list's iterators are not guaranteed to be pointers, but
		// this appears to be the case on all of our implementations.  Also, if it's not true on a new implementation,
		// it will fail to compile rather than behave badly.
		CopyToEmpty<InElementType>(InitList.begin(), { (int64)InitList.size() });
		return *this;
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	inline ElementType& operator[](int64 Index)
	{
		RangeCheck(0, Index);
		return GetData()[Index];
	}

	/**
	 * Array bracket operator. Returns reference to element at given index.
	 *
	 * Const version of the above.
	 *
	 * @returns Reference to indexed element.
	 */
	inline const ElementType& operator[](int64 Index) const
	{
		RangeCheck(0, Index);
		return GetData()[Index];
	}

	/**
	 * Flattens the array view into a single dimension.
	 *
	 * @returns Flattened array view.
	 */
	inline TMultiArrayView<1, ElementType> Flatten()
	{
		return TMultiArrayView<1, ElementType>(GetData(), Num());
	}

	/**
	 * Flattens the array view into a single dimension.
	 *
	 * @returns Flattened array view.
	 */
	inline TMultiArrayView<1, const ElementType> Flatten() const
	{
		return TMultiArrayView<1, const ElementType>(GetData(), Num());
	}

	/** Explicit conversion to TArrayView. */
	inline TArrayView<ElementType> ArrayView()
	{
		return TArrayView<ElementType>(GetData(), Num());
	}

	/** Explicit conversion to TArrayView. */
	inline TArrayView<const ElementType> ArrayView() const
	{
		return TArrayView<const ElementType>(GetData(), Num());
	}

	/** Implicit cast to TArrayView. */
	inline operator TArrayView<ElementType>()
	{
		return TArrayView<ElementType>(GetData(), Num());
	}

	/** Implicit cast to TArrayView. */
	inline operator TArrayView<const ElementType>() const
	{
		return TArrayView<const ElementType>(GetData(), Num());
	}

public:

	/**
	 * Constructor, initializes element number counters.
	 */
	inline TMultiArray() = default;

	/**
	 * Constructor from a raw array of elements.
	 *
	 * @param Ptr   A pointer to an array of elements to copy.
	 * @param Nums The number of elements on each dimension.
	 */
	inline TMultiArray(const ElementType* Ptr, TMultiArrayShape<DimNum> Shape)
	{
		check(Ptr != nullptr || Shape.Total() == 0);

		CopyToEmpty(Ptr, Shape);
	}

	template <typename OtherElementType>
	explicit TMultiArray(const TMultiArrayView<DimNum, OtherElementType>& Other)
		: TMultiArray(Other.GetData(), Other.Shape()) {}

	/**
	 * Copy constructor with changed allocator. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::MultiArray::Private::TMultiArrayElementsAreCompatible_V<ElementType, const OtherElementType&>)
	>
		inline explicit TMultiArray(const TMultiArray<DimNum, OtherElementType, OtherAllocator>& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Shape());
	}

	/**
	 * Copy constructor. Use the common routine to perform the copy.
	 *
	 * @param Other The source array to copy.
	 */
	inline TMultiArray(const TMultiArray& Other)
	{
		CopyToEmpty(Other.GetData(), Other.Shape());
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * AllocatorType changing version.
	 *
	 * @param Other The source array to assign from.
	 */
	template<typename OtherAllocatorType>
	TMultiArray& operator=(const TMultiArray<DimNum, ElementType, OtherAllocatorType>& Other)
	{
		DestructItems(GetData(), Num());
		CopyToEmpty(Other.GetData(), Other.Shape());
		return *this;
	}

	/**
	 * Assignment operator. First deletes all currently contained elements
	 * and then copies from other array.
	 *
	 * @param Other The source array to assign from.
	 */
	TMultiArray& operator=(const TMultiArray& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), Num());
			CopyToEmpty(Other.GetData(), Other.Shape());
		}
		return *this;
	}

	template <typename OtherElementType>
	TMultiArray& operator=(const TMultiArrayView<DimNum, OtherElementType>& Other);

private:

	/**
	 * Moves or copies array. Depends on the array type traits.
	 *
	 * This override copies.
	 *
	 * @param ToArray Array to move into.
	 * @param FromArray Array to move from.
	 */
	template <typename FromArrayType, typename ToArrayType>
	static inline void MoveOrCopy(ToArrayType& ToArray, FromArrayType& FromArray)
	{
		ToArray.CopyToEmpty(FromArray.GetData(), FromArray.Shape());
	}

public:
	/**
	 * Move constructor.
	 *
	 * @param Other Array to move from.
	 */
	inline TMultiArray(TMultiArray&& Other)
	{
		MoveOrCopy(*this, Other);
	}

	/**
	 * Move constructor.
	 *
	 * @param Other Array to move from.
	 */
	template <
		typename OtherElementType,
		typename OtherAllocator
		UE_REQUIRES(UE::MultiArray::Private::TMultiArrayElementsAreCompatible_V<ElementType, OtherElementType&&>)
	>
		inline explicit TMultiArray(TMultiArray<DimNum, OtherElementType, OtherAllocator>&& Other)
	{
		MoveOrCopy(*this, Other);
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other Array to assign and move from.
	 */
	TMultiArray& operator=(TMultiArray&& Other)
	{
		if (this != &Other)
		{
			DestructItems(GetData(), Num());
			MoveOrCopy(*this, Other);
		}
		return *this;
	}

	/** Destructor. */
	~TMultiArray()
	{
		DestructItems(GetData(), Num());

		// Note: ArrayShape and data pointer are not invalidated.
		// They are left unchanged and use-after-destruct will see them the same as before destruct.
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline ElementType* GetData()
	{
		return (ElementType*)AllocatorInstance.GetAllocation();
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline const ElementType* GetData() const
	{
		return (const ElementType*)AllocatorInstance.GetAllocation();
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline ElementType* begin() { return GetData(); }
	inline ElementType* end() { return GetData() + Num(); }


	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline const ElementType* begin() const { return GetData(); }
	inline const ElementType* end() const { return GetData() + Num(); }

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	inline uint32 GetTypeSize() const
	{
		return sizeof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than zero and less
	 * than maximum.
	 */
	inline void CheckInvariants() const
	{
		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			checkSlow(ArrayShape[Idx] >= 0);
		}
	}

	/**
	 * Checks if a dimension is within the allowed number of dimensions
	 *
	 * @param Dimension Dimension of the array.
	 */
	inline void DimensionCheck(uint8 Dimension) const
	{
		checkf((Dimension < DimNum), TEXT("MultiArray dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch
	}

	/**
	 * Checks if index is in dimension range.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Index to check.
	 */
	inline void RangeCheck(uint8 Dimension, int64 Index) const
	{
		DimensionCheck(Dimension);
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayShape[Dimension]), TEXT("Array index out of bounds: %lld from a dimension of size %lld"), (long long)Index, (long long)ArrayShape[Dimension]); // & for one branch
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in dimension range.
	 * Length is 0 is allowed on empty dimensions; Index must be 0 in that case.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	inline void SliceRangeCheck(uint8 Dimension, int64 Index, int64 InNum) const
	{
		DimensionCheck(Dimension);

		checkf(Index >= 0, TEXT("Invalid index (%lld)"), (long long)Index);
		checkf(InNum >= 0, TEXT("Invalid count (%lld)"), (long long)InNum);
		checkf(Index + InNum <= ArrayShape[Dimension], TEXT("Range (index: %lld, count: %lld) lies outside the view of %lld elements"), (long long)Index, (long long)InNum, (long long)ArrayShape[Dimension]);
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return Num() == 0;
	}

	/**
	 * Returns true if the dimension is empty and contains no elements.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns True if the dimension is empty.
	 * @see Num
	 */
	bool IsEmpty(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension] == 0;
	}

	/**
	 * Returns number of dimensions.
	 *
	 * @returns Number of dimensions in array.
	 */
	inline uint8 Rank() const
	{
		return DimNum;
	}

	/**
	 * Returns total number of elements
	 *
	 * @returns Total number of elements in array.
	 */
	inline int64 Num() const
	{
		return ArrayShape.Total();
	}

	/**
	 * Returns the total number of bytes used by the array
	 *
	 * @returns Total number of bytes used by the array.
	 */
	inline int64 NumBytes() const
	{
		return ArrayShape.Total() * sizeof(ElementType);
	}

	/**
	 * Returns number of elements in a dimension.
	 *
	 * @returns Number of elements in array.
	 */
	template<uint8 InDimIdx>
	inline int64 Num() const
	{
		static_assert(InDimIdx < DimNum);
		return ArrayShape[InDimIdx];
	}

	/**
	 * Returns number of elements in a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Number of elements in array.
	 */
	inline int64 Num(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension];
	}

	/**
	 * Returns the number of elements in each dimension.
	 *
	 * @returns Number of elements in each dimension.
	 */
	inline TMultiArrayShape<DimNum> Shape() const
	{
		return ArrayShape;
	}

	/**
	 * Returns the stride for a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Stride of that dimension.
	 */
	inline int64 Stride(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		int64 Total = 1;
		for (uint8 Idx = Dimension + 1; Idx < DimNum; Idx++) //-V621
		{
			Total *= ArrayShape[Idx];
		}
		return Total;
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView<DimNum, ElementType> Slice(int64 Index, int64 InNum)
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView<DimNum, ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView<DimNum, const ElementType> Slice(int64 Index, int64 InNum) const
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView<DimNum, const ElementType>(GetData() + Index * Stride(0), NewShape);
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename ComparisonType>
	bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + Num(); Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

public:

	/**
	 * Empties the array. It calls the destructors on held items if needed.
	 */
	void Empty()
	{
		int64 CurrTotal = Num();
		DestructItems(GetData(), CurrTotal);
		ArrayShape = TMultiArrayShape<DimNum>();
		AllocatorResizeAllocation(CurrTotal, 0);
	}

	/**
	 * Resizes array to given number of elements.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNum(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
			ConstructItems<ElementType>(GetData() + CurrTotal, GetData(), NewTotal - CurrTotal);
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

	/**
	 * Resizes array to given number of elements. New elements will be zeroed.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNumZeroed(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
			FMemory::Memzero((void*)(GetData() + CurrTotal), (NewTotal - CurrTotal) * sizeof(ElementType));
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

	/**
	 * Resizes array to given number of elements. New elements will be uninitialized.
	 *
	 * @param NewShape New shape of the array.
	 */
	void SetNumUninitialized(TMultiArrayShape<DimNum> NewShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 NewTotal = NewShape.Total();

		if (NewTotal > CurrTotal)
		{
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}
		else if (NewTotal < CurrTotal)
		{
			DestructItems<ElementType>(GetData() + NewTotal, CurrTotal - NewTotal);
			AllocatorResizeAllocation(CurrTotal, NewTotal);
		}

		ArrayShape = NewShape;
	}

private:
	void AllocatorResizeAllocation(int64 CurrentNum, int64 NewNum)
	{
		if constexpr (TAllocatorTraits<AllocatorType>::SupportsElementAlignment)
		{
			AllocatorInstance.ResizeAllocation(CurrentNum, NewNum, sizeof(ElementType), alignof(ElementType));
		}
		else
		{
			AllocatorInstance.ResizeAllocation(CurrentNum, NewNum, sizeof(ElementType));
		}
	}

	/**
	 * Copies data from one array into this array. Uses the fast path if the
	 * data in question does not need a constructor.
	 *
	 * @param OtherData A pointer to the data to copy
	 * @param OtherNums The dimensions of the data to copy
	 */
	template <typename OtherElementType>
	void CopyToEmpty(const OtherElementType* OtherData, TMultiArrayShape<DimNum> OtherShape)
	{
		int64 CurrTotal = ArrayShape.Total();
		int64 OtherTotal = OtherShape.Total();

		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			int64 NewNum = OtherShape[Idx];

			checkf(NewNum == OtherShape[Idx], TEXT("Invalid number of elements to add to this array type: %lld"), (long long)NewNum);

			ArrayShape[Idx] = NewNum;
		}

		int64 NewTotal = ArrayShape.Total();

		AllocatorResizeAllocation(CurrTotal, NewTotal);
		ConstructItems<ElementType>(GetData(), OtherData, OtherTotal);
	}

public:

	const ElementAllocatorType& GetAllocatorInstance() const { return AllocatorInstance; }
	ElementAllocatorType& GetAllocatorInstance() { return AllocatorInstance; }

public:

	inline operator TMultiArrayView<DimNum, ElementType>()
	{
		return TMultiArrayView<DimNum, ElementType>(GetData(), ArrayShape);
	}

	inline operator TMultiArrayView<DimNum, const ElementType>() const
	{
		return TMultiArrayView<DimNum, const ElementType>(GetData(), ArrayShape);
	}

protected:

	ElementAllocatorType AllocatorInstance;
	TMultiArrayShape<DimNum> ArrayShape;
};
