// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Invoke.h"
#include "Templates/UnrealTypeTraits.h"
#include "Traits/ElementType.h"
#include "Math/UnrealMathUtility.h"
#include <type_traits>

namespace UE::MultiArrayView::Private
{
	/**
	 * Copied from Core/Public/Containers/ArrayView.h
	 *
	 * Trait testing whether a type is compatible with the view type
	 *
	 * The extra stars here are *IMPORTANT*
	 * They prevent TMultiArrayView<Base>(TArray<Derived>&) from compiling!
	 */
	template <typename T, typename ElementType>
	constexpr bool TIsCompatibleElementType_V = std::is_convertible_v<T**, ElementType* const*>;

	// Copied from Core/Public/Containers/ArrayView.h
	//
	// Simply forwards to an unqualified GetData(), but can be called from within TArrayView
	// where GetData() is already a member and so hides any others.
	template <typename T>
	inline decltype(auto) GetDataHelper(T&& Arg)
	{
		return GetData(Forward<T>(Arg));
	}

	// Copied from Core/Public/Containers/ArrayView.h
	//
	// Gets the data from the passed argument and proceeds to reinterpret the resulting elements
	template <typename T>
	inline decltype(auto) GetReinterpretedDataHelper(T&& Arg)
	{
		auto NaturalPtr = GetData(Forward<T>(Arg));
		using NaturalElementType = std::remove_pointer_t<decltype(NaturalPtr)>;

		auto EndPtr = NaturalPtr + GetNum(Arg);
		TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretRange(NaturalPtr, EndPtr);

		return reinterpret_cast<typename TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretType*>(NaturalPtr);
	}

	/**
	 * Copied from Core/Public/Containers/ArrayView.h
	 *
	 * Trait testing whether a type is compatible with the view type
	 */
	template <typename RangeType, typename ElementType>
	struct TIsCompatibleRangeType
	{
		static constexpr bool Value = TIsCompatibleElementType_V<std::remove_pointer_t<decltype(GetData(DeclVal<RangeType&>()))>, ElementType>;

		template <typename T>
		static decltype(auto) GetData(T&& Arg)
		{
			return UE::MultiArrayView::Private::GetDataHelper(Forward<T>(Arg));
		}
	};

	/**
	 * Copied from Core/Public/Containers/ArrayView.h
	 *
	 * Trait testing whether a type is reinterpretable in a way that permits use with the view type
	 */
	template <typename RangeType, typename ElementType>
	struct TIsReinterpretableRangeType
	{
	private:
		using NaturalElementType = std::remove_pointer_t<decltype(GetData(DeclVal<RangeType&>()))>;

	public:
		static constexpr bool Value =
			!std::is_same_v<typename TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretType, NaturalElementType>
			&&
			TIsCompatibleElementType_V<typename TContainerElementTypeCompatibility<NaturalElementType>::ReinterpretType, ElementType>;

		template <typename T>
		static decltype(auto) GetData(T&& Arg)
		{
			return UE::MultiArrayView::Private::GetReinterpretedDataHelper(Forward<T>(Arg));
		}
	};
}

template<uint8 InDimNum>
class TMultiArrayShape
{
public:
	static constexpr uint8 DimNum = InDimNum;

	static_assert(DimNum >= 1, "TMultiArrayShape requires a positive, non-zero number of dimensions");

public:

	TMultiArrayShape() = default;

	TMultiArrayShape(std::initializer_list<int64> InNums)
	{
		checkf(InNums.size() == DimNum, TEXT("Wrong number of elements in constructor"));

		uint8 Idx = 0;
		for (int64 InNum : InNums)
		{
			Nums[Idx] = InNum;
			Idx++;
		}
	}

	TMultiArrayShape(const int64* InNums)
	{
		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			Nums[Idx] = InNums[Idx];
		}
	}

	int64& operator[](uint8 Dimension)
	{
		return Nums[Dimension];
	}

	const int64& operator[](uint8 Dimension) const
	{
		return Nums[Dimension];
	}

	int64 Total() const
	{
		if constexpr (DimNum == 1)
		{
			return Nums[0];
		}
		else
		{
			int64 Total = Nums[0];
			for (uint8 Idx = 1; Idx < DimNum; Idx++)
			{
				Total *= Nums[Idx];
			}
			return Total;
		}
	}

private:
	int64 Nums[DimNum] = { 0 };
};

/**
 * Templated fixed-size view of multi-dimensional array
 *
 * A statically sized view of a multi-dimensional array of typed elements. 
 *
 * @see TArrayView
 */
template<uint8 InDimNum, typename InElementType>
class TMultiArrayView
{
public:
	static constexpr uint8 DimNum = InDimNum;
	using ElementType = InElementType;

	static_assert(DimNum > 1, "TMultiArrayView requires a positive, non-zero number of dimensions");

public:

	/**
	 * Constructor.
	 */
	TMultiArrayView() : DataPtr(nullptr) { }

	/**
	 * Construct a view of an arbitrary pointer
	 *
	 * @param InData	The data to view
	 * @param InShape	The number of elements on each dimension
	 */
	inline TMultiArrayView(ElementType* InData, TMultiArrayShape<DimNum> InShape)
		: DataPtr(InData)
		, ArrayShape(InShape)
	{
		CheckInvariants();
	}

	/**
	 * Array bracket operator. Returns reference to sub-view at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	inline TMultiArrayView<DimNum - 1, ElementType> operator[](int64 Index) const
	{
		RangeCheck(0, Index);

		TMultiArrayShape<DimNum - 1> NewShape;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			NewShape[Idx] = ArrayShape[Idx + 1];
		}

		return TMultiArrayView<DimNum - 1, ElementType>(DataPtr + Index * Stride(0), NewShape);
	}

	/**
	 * Flattens the array view into a single dimension.
	 *
	 * @returns Flattened array view.
	 */
	inline TMultiArrayView<1, ElementType> Flatten() const
	{
		return TMultiArrayView<1, ElementType>(DataPtr, Num());
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	inline TMultiArrayView<DimNum - 1, ElementType> Flatten(uint8 Dimension) const
	{
		checkf((Dimension < DimNum - 1), TEXT("MultiArray flatten dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx+1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, ElementType>(DataPtr, NewShape);
	}

	/**
	 * Flattens the array on a given dimension, merging that dimension and the following one.
	 * e.g. a 3D MultiArrayView with shape [10, 5, 3] flattened on dimension 0 will become a 2D MultiArrayView with shape [10 * 5, 3]
	 *
	 * @returns View flattened on the given dimension.
	 */
	template<uint8 Dimension>
	inline TMultiArrayView<DimNum - 1, ElementType> Flatten() const
	{
		static_assert(Dimension < DimNum - 1, "MultiArray flatten dimension out of bounds");

		TMultiArrayShape<DimNum - 1> NewShape;
		uint8 SrcIdx = 0;
		for (uint8 Idx = 0; Idx < DimNum - 1; Idx++)
		{
			if (Idx == Dimension)
			{
				NewShape[Idx] = ArrayShape[SrcIdx] * ArrayShape[SrcIdx + 1];
				SrcIdx += 2;
			}
			else
			{
				NewShape[Idx] = ArrayShape[SrcIdx];
				SrcIdx += 1;
			}
		}

		return TMultiArrayView<DimNum - 1, ElementType>(DataPtr, NewShape);
	}

public:

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline ElementType* GetData() const
	{
		return DataPtr;
	}


	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline ElementType* begin() const { return GetData(); }
	inline ElementType* end() const { return GetData() + Num(); }

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	inline static constexpr size_t GetTypeSize()
	{
		return sizeof(ElementType);
	}

	/**
	 * Helper function returning the alignment of the inner type.
	 */
	inline static constexpr size_t GetTypeAlignment()
	{
		return alignof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than zero on each dimension
	 */
	inline void CheckInvariants() const
	{
		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			checkSlow(ArrayShape[Idx] >= 0);
		}
	}

	/**
	 * Checks if a dimension is within the allowed number of dimensions
	 *
	 * @param Dimension Dimension of the array.
	 */
	inline void DimensionCheck(uint8 Dimension) const
	{
		checkf((Dimension < DimNum), TEXT("MultiArray dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch
	}

	/**
	 * Checks if index is in dimension range.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Index to check.
	 */
	inline void RangeCheck(uint8 Dimension, int64 Index) const
	{
		DimensionCheck(Dimension);
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayShape[Dimension]), TEXT("MultiArray index out of bounds: %lld from a dimension of size %lld"), (long long)Index, (long long)ArrayShape[Dimension]); // & for one branch
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in dimension range.
	 * Length is 0 is allowed on empty dimensions; Index must be 0 in that case.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	inline void SliceRangeCheck(uint8 Dimension, int64 Index, int64 InNum) const
	{
		DimensionCheck(Dimension);

		checkf(Index >= 0, TEXT("Invalid index (%lld)"), (long long)Index);
		checkf(InNum >= 0, TEXT("Invalid count (%lld)"), (long long)InNum);
		checkf(Index + InNum <= ArrayShape[Dimension], TEXT("Range (index: %lld, count: %lld) lies outside the view of %lld elements"), (long long)Index, (long long)InNum, (long long)ArrayShape[Dimension]);
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return Num() == 0;
	}

	/**
	 * Returns true if the dimension is empty and contains no elements.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns True if the dimension is empty.
	 * @see Num
	 */
	bool IsEmpty(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension] == 0;
	}

	/**
	 * Returns the number of dimensions.
	 *
	 * @returns Number of dimensions in array.
	 */
	inline uint8 Rank() const
	{
		return DimNum;
	}

	/**
	 * Returns the total number of elements
	 *
	 * @returns Total number of elements in array.
	 */
	inline int64 Num() const
	{
		return ArrayShape.Total();
	}

	/**
	 * Returns the total number of bytes used by the array
	 *
	 * @returns Total number of bytes used by the array.
	 */
	inline int64 NumBytes() const
	{
		return ArrayShape.Total() * sizeof(ElementType);
	}

	/**
	 * Returns the number of elements in a dimension.
	 *
	 * @returns Number of elements in array.
	 */
	template<uint8 InDimIdx>
	inline int64 Num() const
	{
		static_assert(InDimIdx < DimNum);
		return ArrayShape[InDimIdx];
	}

	/**
	 * Returns the number of elements in a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Number of elements in array.
	 */
	inline int64 Num(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension];
	}

	/**
	 * Returns the number of elements in each dimension.
	 *
	 * @returns Number of elements in each dimension.
	 */
	inline TMultiArrayShape<DimNum> Shape() const
	{
		return ArrayShape;
	}

	/**
	 * Returns the stride for a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Stride of that dimension.
	 */
	inline int64 Stride(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		int64 Total = 1;
		for (uint8 Idx = Dimension + 1; Idx < DimNum; Idx++)
		{
			Total *= ArrayShape[Idx];
		}
		return Total;
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView Slice(int64 Index, int64 InNum) const
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView(DataPtr + Index * Stride(0), NewShape);
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename ComparisonType>
	bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + Num(); Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

	/** Implicit cast for const. */
	inline operator TMultiArrayView<DimNum, const ElementType>() const
	{
		return TMultiArrayView<DimNum, const ElementType>(DataPtr, ArrayShape);
	}

private:

	ElementType* DataPtr;
	TMultiArrayShape<DimNum> ArrayShape;
};


/**
* Specialization for single dimensional MultiArrayView
*/
template<typename InElementType>
class TMultiArrayView<1, InElementType>
{
public:
	static constexpr uint8 DimNum = 1;
	using ElementType = InElementType;

	/**
	 * Constructor.
	 */
	TMultiArrayView() : DataPtr(nullptr) { }

private:
	template <typename T>
	using TIsCompatibleRangeType = UE::MultiArrayView::Private::TIsCompatibleRangeType<T, ElementType>;

	template <typename T>
	using TIsReinterpretableRangeType = UE::MultiArrayView::Private::TIsReinterpretableRangeType<T, ElementType>;

public:
	/**
	 * Constructor from another range
	 *
	 * @param Other The source range to copy
	 */
	template <
		typename OtherRangeType,
		typename CVUnqualifiedOtherRangeType = std::remove_cv_t<std::remove_reference_t<OtherRangeType>>
		UE_REQUIRES(
			TAnd<
				TIsContiguousContainer<CVUnqualifiedOtherRangeType>,
				TOr<
					TIsCompatibleRangeType<OtherRangeType>,
					TIsReinterpretableRangeType<OtherRangeType>
				>
			>::Value
		)
	>
	inline TMultiArrayView(OtherRangeType&& Other)
		: DataPtr(std::conditional_t<
			TIsCompatibleRangeType<OtherRangeType>::Value,
			TIsCompatibleRangeType<OtherRangeType>,
			TIsReinterpretableRangeType<OtherRangeType>
		>::GetData(Forward<OtherRangeType>(Other)))
	{
		const auto InCount = GetNum(Forward<OtherRangeType>(Other));
		check((InCount >= 0) && ((sizeof(InCount) < sizeof(int64)) || (InCount <= static_cast<decltype(InCount)>(TNumericLimits<int64>::Max()))));
		ArrayShape[0] = (int64)InCount;
	}

	/**
	 * Construct a view of an arbitrary pointer
	 *
	 * @param InData	The data to view
	 * @param InNums	The number of elements on each dimension
	 */
	inline TMultiArrayView(ElementType* InData, TMultiArrayShape<DimNum> InShape)
		: DataPtr(InData)
		, ArrayShape(InShape)
	{
		CheckInvariants();
	}

	/**
	 * Construct a view of an arbitrary pointer
	 *
	 * @param InData	The data to view
	 * @param InNum	The number of elements
	 */
	template <
		typename OtherElementType
		UE_REQUIRES(UE::MultiArrayView::Private::TIsCompatibleElementType_V<OtherElementType, ElementType>)
	>
	inline TMultiArrayView(OtherElementType* InData, int64 InNum)
		: DataPtr(InData)
	{
		ArrayShape[0] = InNum;
		CheckInvariants();
	}

	/**
	 * Construct a view of an initializer list.
	 *
	 * The caller is responsible for ensuring that the view does not outlive the initializer list.
	 */
	inline TMultiArrayView(std::initializer_list<ElementType> List)
		: DataPtr(UE::MultiArrayView::Private::GetDataHelper(List))
	{
		ArrayShape[0] = GetNum(List);
		CheckInvariants();
	}

public:

	/**
	 * Array bracket operator. Returns a reference to the element at given index.
	 *
	 * @returns Reference to indexed element.
	 */
	inline ElementType& operator[](int64 Index) const
	{
		RangeCheck(0, Index);
		return DataPtr[Index];
	}

	/** Explicit conversion to TArrayView. */
	inline TArrayView<ElementType> ArrayView()
	{
		return TArrayView<ElementType>(GetData(), Num());
	}

	/** Explicit conversion to TArrayView. */
	inline TArrayView<const ElementType> ArrayView() const
	{
		return TArrayView<const ElementType>(GetData(), Num());
	}

	/** Implicit cast to TArrayView. */
	inline operator TArrayView<ElementType>()
	{
		return TArrayView<ElementType>(DataPtr, ArrayShape[0]);
	}

	/** Implicit cast to TArrayView. */
	inline operator TArrayView<const ElementType>() const
	{
		return TArrayView<const ElementType>(DataPtr, ArrayShape[0]);
	}

public:

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if empty
	 */
	inline ElementType* GetData() const
	{
		return DataPtr;
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline ElementType* begin() const { return GetData(); }
	inline ElementType* end() const { return GetData() + Num(); }

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	inline static constexpr size_t GetTypeSize()
	{
		return sizeof(ElementType);
	}

	/**
	 * Helper function returning the alignment of the inner type.
	 */
	inline static constexpr size_t GetTypeAlignment()
	{
		return alignof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than zero on each dimension
	 */
	inline void CheckInvariants() const
	{
		for (uint8 Idx = 0; Idx < DimNum; Idx++)
		{
			checkSlow(ArrayShape[Idx] >= 0);
		}
	}

	/**
	 * Checks if a dimension is within the allowed number of dimensions
	 *
	 * @param Dimension Dimension of the array.
	 */
	inline void DimensionCheck(uint8 Dimension) const
	{
		checkf((Dimension < DimNum), TEXT("MultiArray dimension out of bounds: %lld from a rank of %lld"), (long long)Dimension, (long long)DimNum); // & for one branch
	}

	/**
	 * Checks if index is in dimension range.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Index to check.
	 */
	inline void RangeCheck(uint8 Dimension, int64 Index) const
	{
		DimensionCheck(Dimension);
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayShape[Dimension]), TEXT("MultiArray index out of bounds: %lld from a dimension of size %lld"), (long long)Index, (long long)ArrayShape[Dimension]); // & for one branch
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in dimension range.
	 * Length is 0 is allowed on empty dimensions; Index must be 0 in that case.
	 *
	 * @param Dimension Dimension of the array.
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	inline void SliceRangeCheck(uint8 Dimension, int64 Index, int64 InNum) const
	{
		DimensionCheck(Dimension);

		checkf(Index >= 0, TEXT("Invalid index (%lld)"), (long long)Index);
		checkf(InNum >= 0, TEXT("Invalid count (%lld)"), (long long)InNum);
		checkf(Index + InNum <= ArrayShape[Dimension], TEXT("Range (index: %lld, count: %lld) lies outside the view of %lld elements"), (long long)Index, (long long)InNum, (long long)ArrayShape[Dimension]);
	}

	/**
	 * Returns true if the array is empty and contains no elements.
	 *
	 * @returns True if the array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return Num() == 0;
	}

	/**
	 * Returns true if the dimension is empty and contains no elements.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns True if the dimension is empty.
	 * @see Num
	 */
	bool IsEmpty(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension] == 0;
	}

	/**
	 * Returns the number of dimensions.
	 *
	 * @returns Number of dimensions in array.
	 */
	inline uint8 Rank() const
	{
		return DimNum;
	}

	/**
	 * Returns the total number of elements
	 *
	 * @returns Total number of elements in array.
	 */
	inline int64 Num() const
	{
		return ArrayShape.Total();
	}

	/**
	 * Returns the total number of bytes used by the array
	 *
	 * @returns Total number of bytes used by the array.
	 */
	inline int64 NumBytes() const
	{
		return ArrayShape.Total() * sizeof(ElementType);
	}

	/**
	 * Returns the number of elements in a dimension.
	 *
	 * @returns Number of elements in array.
	 */
	template<uint8 InDimIdx>
	inline int64 Num() const
	{
		static_assert(InDimIdx < DimNum);
		return ArrayShape[InDimIdx];
	}

	/**
	 * Returns the number of elements in a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Number of elements in array.
	 */
	inline int64 Num(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		return ArrayShape[Dimension];
	}

	/**
	 * Returns the number of elements in each dimension.
	 *
	 * @returns Number of elements in each dimension.
	 */
	inline TMultiArrayShape<DimNum> Shape() const
	{
		return ArrayShape;
	}

	/**
	 * Returns the stride for a dimension.
	 *
	 * @param Dimension Dimension of the array.
	 * @returns Stride of that dimension.
	 */
	inline int64 Stride(uint8 Dimension) const
	{
		DimensionCheck(Dimension);

		int64 Total = 1;
		for (uint8 Idx = Dimension + 1; Idx < DimNum; Idx++) //-V621
		{
			Total *= ArrayShape[Idx];
		}
		return Total;
	}

	/**
	 * Returns a sliced view. Slicing outside of the range of the view is illegal.
	 *
	 * @param Index Starting index of the new view
	 * @param InNum Number of elements in the new view
	 * @returns Sliced view
	 */
	[[nodiscard]] inline TMultiArrayView Slice(int64 Index, int64 InNum) const
	{
		SliceRangeCheck(0, Index, InNum);
		TMultiArrayShape NewShape = ArrayShape;
		NewShape[0] = InNum;
		return TMultiArrayView(DataPtr + Index * Stride(0), NewShape);
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @returns	True if found. False otherwise.
	 */
	template <typename ComparisonType>
	bool Contains(const ComparisonType& Item) const
	{
		for (const ElementType* RESTRICT Data = GetData(), *RESTRICT DataEnd = Data + Num(); Data != DataEnd; ++Data)
		{
			if (*Data == Item)
			{
				return true;
			}
		}
		return false;
	}

	/** Implicit cast for const. */
	inline operator TMultiArrayView<DimNum, const ElementType>() const
	{
		return TMultiArrayView<DimNum, const ElementType>(DataPtr, ArrayShape);
	}

private:

	ElementType* DataPtr;
	TMultiArrayShape<DimNum> ArrayShape;
};

template<uint8 InDimNum, typename InElementType>
using TConstMultiArrayView = TMultiArrayView<InDimNum, const InElementType>;
