// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningExperience.h"

#include "LearningObservation.h"
#include "LearningAction.h"
#include "LearningCompletion.h"
#include "LearningLog.h"

#include "Dom/JsonObject.h"
#include "Templates/Function.h"

namespace UE::Learning
{
	void FEpisodeBuffer::Resize(const int32 InMaxInstanceNum, const int32 InMaxStepNum)
	{
		MaxInstanceNum = InMaxInstanceNum;
		MaxStepNum = InMaxStepNum;

		// Observations
		for (int32 NameIndex = 0; NameIndex < ObservationNames.Num(); NameIndex++)
		{
			int32 ObservationSize = ObservationSizes[NameIndex];
			ObservationArrays[NameIndex].SetNumUninitialized({ InMaxInstanceNum, InMaxStepNum, ObservationSize });
		}

		// Actions
		for (int32 NameIndex = 0; NameIndex < ActionNames.Num(); NameIndex++)
		{
			int32 ActionSize = ActionSizes[NameIndex];
			ActionArrays[NameIndex].SetNumUninitialized({ InMaxInstanceNum, InMaxStepNum, ActionSize });
		}

		// Action Modifiers
		for (int32 NameIndex = 0; NameIndex < ActionModifierNames.Num(); NameIndex++)
		{
			int32 ActionModifierSize = ActionModifierSizes[NameIndex];
			ActionModifierArrays[NameIndex].SetNumUninitialized({ InMaxInstanceNum, InMaxStepNum, ActionModifierSize });
		}

		// Memory States
		for (int32 NameIndex = 0; NameIndex < MemoryStateNames.Num(); NameIndex++)
		{
			int32 MemoryStateSize = MemoryStateSizes[NameIndex];
			MemoryStateArrays[NameIndex].SetNumUninitialized({ InMaxInstanceNum, InMaxStepNum, MemoryStateSize });
		}

		// Rewards
		for (int32 NameIndex = 0; NameIndex < RewardNames.Num(); NameIndex++)
		{
			int32 RewardSize = RewardSizes[NameIndex];
			RewardArrays[NameIndex].SetNumUninitialized({ InMaxInstanceNum, InMaxStepNum, RewardSize });
		}

		// Episode Step Nums
		EpisodeStepNums.SetNumUninitialized({ InMaxInstanceNum });
		Array::Zero(EpisodeStepNums);

		bHasBeenSized = true;
	}

	void FEpisodeBuffer::Reset(const FIndexSet Instances)
	{
		Array::Zero(EpisodeStepNums, Instances);
	}

	int32 FEpisodeBuffer::AddObservations(const FName& Name, const int32 SchemaId, const int32 Size)
	{
		checkf(!ObservationNames.Contains(Name), TEXT("Observation name collision!"));

		int32 ObservationId = ObservationNames.Num();
		ObservationNames.Add(Name);
		ObservationSchemaIds.Add(SchemaId);
		ObservationSizes.Add(Size);
		TLearningArray<3, float>& NewObservation = ObservationArrays.AddDefaulted_GetRef();

		if (bHasBeenSized)
		{
			NewObservation.SetNumUninitialized({ MaxInstanceNum, MaxStepNum, Size });
		}

		return ObservationId;
	}

	void FEpisodeBuffer::PushObservations(const int32 ObservationId, const TLearningArrayView<2, const float> InObservations, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushObservations);

		checkf(ObservationId >= 0 && ObservationId < ObservationArrays.Num(), TEXT("Observation id invalid!"));

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(ObservationArrays[ObservationId][InstanceIdx][EpisodeStepNums[InstanceIdx]], InObservations[InstanceIdx]);
		}
	}

	const TLearningArrayView<2, const float> FEpisodeBuffer::GetObservations(const int32 ObservationId, const int32 InstanceIdx) const
	{
		checkf(ObservationId >= 0 && ObservationId < ObservationArrays.Num(), TEXT("Observation id invalid!"));

		return ObservationArrays[ObservationId][InstanceIdx].Slice(0, EpisodeStepNums[InstanceIdx]);
	}

	int32 FEpisodeBuffer::AddActions(const FName& Name, const int32 SchemaId, const int32 Size)
	{
		checkf(!ActionNames.Contains(Name), TEXT("Action name collision!"));

		int32 ActionId = ActionNames.Num();
		ActionNames.Add(Name);
		ActionSchemaIds.Add(SchemaId);
		ActionSizes.Add(Size);
		TLearningArray<3, float>& NewAction = ActionArrays.AddDefaulted_GetRef();

		if (bHasBeenSized)
		{
			NewAction.SetNumUninitialized({ MaxInstanceNum, MaxStepNum, Size });
		}

		return ActionId;
	}

	void FEpisodeBuffer::PushActions(const int32 ActionId, const TLearningArrayView<2, const float> InActions, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushActions);

		checkf(ActionId >= 0 && ActionId < ActionArrays.Num(), TEXT("Action id invalid!"));

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(ActionArrays[ActionId][InstanceIdx][EpisodeStepNums[InstanceIdx]], InActions[InstanceIdx]);
		}
	}

	const TLearningArrayView<2, const float> FEpisodeBuffer::GetActions(const int32 ActionId, const int32 InstanceIdx) const
	{
		checkf(ActionId >= 0 && ActionId < ActionArrays.Num(), TEXT("Action id invalid!"));

		return ActionArrays[ActionId][InstanceIdx].Slice(0, EpisodeStepNums[InstanceIdx]);
	}

	int32 FEpisodeBuffer::AddActionModifiers(const FName& Name, const int32 SchemaId, const int32 Size)
	{
		checkf(!ActionModifierNames.Contains(Name), TEXT("Action Modifier name collision!"));

		int32 ActionModifierId = ActionModifierNames.Num();
		ActionModifierNames.Add(Name);
		ActionModifierSchemaIds.Add(SchemaId);
		ActionModifierSizes.Add(Size);
		TLearningArray<3, float>& NewActionModifier = ActionModifierArrays.AddDefaulted_GetRef();

		if (bHasBeenSized)
		{
			NewActionModifier.SetNumUninitialized({ MaxInstanceNum, MaxStepNum, Size });
		}

		return ActionModifierId;
	}

	void FEpisodeBuffer::PushActionModifiers(const int32 ActionModifierId, const TLearningArrayView<2, const float> InActionModifiers, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushActionModifiers);

		checkf(ActionModifierId >= 0 && ActionModifierId < ActionModifierArrays.Num(), TEXT("Action Modifier id invalid!"));

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(ActionModifierArrays[ActionModifierId][InstanceIdx][EpisodeStepNums[InstanceIdx]], InActionModifiers[InstanceIdx]);
		}
	}

	const TLearningArrayView<2, const float> FEpisodeBuffer::GetActionModifiers(const int32 ActionModifierId, const int32 InstanceIdx) const
	{
		checkf(ActionModifierId >= 0 && ActionModifierId < ActionModifierArrays.Num(), TEXT("Action Modifier id invalid!"));

		return ActionModifierArrays[ActionModifierId][InstanceIdx].Slice(0, EpisodeStepNums[InstanceIdx]);
	}

	int32 FEpisodeBuffer::AddMemoryStates(const FName& Name, const int32 Size)
	{
		checkf(!MemoryStateNames.Contains(Name), TEXT("Memory State name collision!"));

		int32 MemoryStateId = MemoryStateNames.Num();
		MemoryStateNames.Add(Name);
		MemoryStateSizes.Add(Size);
		TLearningArray<3, float>& NewMemoryState = MemoryStateArrays.AddDefaulted_GetRef();

		if (bHasBeenSized)
		{
			NewMemoryState.SetNumUninitialized({ MaxInstanceNum, MaxStepNum, Size });
		}

		return MemoryStateId;
	}

	void FEpisodeBuffer::PushMemoryStates(const int32 MemoryStateId, const TLearningArrayView<2, const float> InMemoryStates, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushMemoryStates);

		checkf(MemoryStateId >= 0 && MemoryStateId < MemoryStateArrays.Num(), TEXT("Memory state id invalid!"));

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(MemoryStateArrays[MemoryStateId][InstanceIdx][EpisodeStepNums[InstanceIdx]], InMemoryStates[InstanceIdx]);
		}
	}

	const TLearningArrayView<2, const float> FEpisodeBuffer::GetMemoryStates(const int32 MemoryStateId, const int32 InstanceIdx) const
	{
		checkf(MemoryStateId >= 0 && MemoryStateId < MemoryStateArrays.Num(), TEXT("Memory state id invalid!"));

		return MemoryStateArrays[MemoryStateId][InstanceIdx].Slice(0, EpisodeStepNums[InstanceIdx]);
	}

	int32 FEpisodeBuffer::AddRewards(const FName& Name, const int32 Size)
	{
		checkf(!RewardNames.Contains(Name), TEXT("Reward name collision!"));

		int32 RewardId = RewardNames.Num();
		RewardNames.Add(Name);
		RewardSizes.Add(Size);
		TLearningArray<3, float>& NewReward = RewardArrays.AddDefaulted_GetRef();

		if (bHasBeenSized)
		{
			NewReward.SetNumUninitialized({ MaxInstanceNum, MaxStepNum, Size });
		}

		return RewardId;
	}

	void FEpisodeBuffer::PushRewards(const int32 RewardId, const TLearningArrayView<1, const float> InRewards, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushRewards);

		checkf(RewardId >= 0 && RewardId < RewardArrays.Num(), TEXT("Reward id invalid!"));

		TLearningArrayView<2, const float> RewardsReshaped(InRewards.GetData(), { InRewards.Num(), 1 });
		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(RewardArrays[RewardId][InstanceIdx][EpisodeStepNums[InstanceIdx]], RewardsReshaped[InstanceIdx]);
		}
	}

	void FEpisodeBuffer::PushRewards(const int32 RewardId, const TLearningArrayView<2, const float> InRewards, const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::PushRewards);

		checkf(RewardId >= 0 && RewardId < RewardArrays.Num(), TEXT("Reward id invalid!"));

		for (const int32 InstanceIdx : Instances)
		{
			Array::Copy(RewardArrays[RewardId][InstanceIdx][EpisodeStepNums[InstanceIdx]], InRewards[InstanceIdx]);
		}
	}

	const TLearningArrayView<2, const float> FEpisodeBuffer::GetRewards(const int32 RewardId, const int32 InstanceIdx) const
	{
		checkf(RewardId >= 0 && RewardId < RewardArrays.Num(), TEXT("Reward id invalid!"));

		return RewardArrays[RewardId][InstanceIdx].Slice(0, EpisodeStepNums[InstanceIdx]);
	}

	void FEpisodeBuffer::IncrementEpisodeStepNums(const FIndexSet Instances)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FEpisodeBuffer::IncrementEpisodeStepNums);

		for (const int32 InstanceIdx : Instances)
		{
			checkf(EpisodeStepNums[InstanceIdx] < MaxStepNum, TEXT("Episode Buffer full!"));
			EpisodeStepNums[InstanceIdx]++;
		}
	}

	const TLearningArrayView<1, const int32> FEpisodeBuffer::GetEpisodeStepNums() const
	{
		return EpisodeStepNums;
	}

	int32 FEpisodeBuffer::GetMaxInstanceNum() const
	{
		return MaxInstanceNum;
	}

	int32 FEpisodeBuffer::GetMaxStepNum() const
	{
		return MaxStepNum;
	}

	void FReplayBuffer::Resize(
		const FEpisodeBuffer& EpisodeBuffer,
		const int32 InMaxEpisodeNum,
		const int32 InMaxStepNum)
	{
		bHasCompletions = true;
		bHasFinalObservations = true;
		bHasFinalMemoryStates = true;

		MaxEpisodeNum = InMaxEpisodeNum;
		EpisodeNum = 0;

		MaxStepNum = InMaxStepNum;
		StepNum = 0;

		EpisodeStarts.SetNumUninitialized({ InMaxEpisodeNum });
		EpisodeLengths.SetNumUninitialized({ InMaxEpisodeNum });
		EpisodeCompletionModes.SetNumUninitialized({ InMaxEpisodeNum });

		// Observations
		Observations.Empty();
		for (int32 NameIndex = 0; NameIndex < EpisodeBuffer.ObservationNames.Num(); NameIndex++)
		{
			ObservationNames.Add(EpisodeBuffer.ObservationNames[NameIndex]);
			ObservationSchemaIds.Add(EpisodeBuffer.ObservationSchemaIds[NameIndex]);

			int32 ObservationSize = EpisodeBuffer.ObservationSizes[NameIndex];

			TLearningArray<2, float>& NewObservation = Observations.AddDefaulted_GetRef();
			NewObservation.SetNumUninitialized({ InMaxStepNum, ObservationSize });

			TLearningArray<2, float>& NewEpisodeFinalObservation = EpisodeFinalObservations.AddDefaulted_GetRef();
			NewEpisodeFinalObservation.SetNumUninitialized({ InMaxEpisodeNum, ObservationSize });
		}

		// Actions
		Actions.Empty();
		for (int32 NameIndex = 0; NameIndex < EpisodeBuffer.ActionNames.Num(); NameIndex++)
		{
			ActionNames.Add(EpisodeBuffer.ActionNames[NameIndex]);
			ActionSchemaIds.Add(EpisodeBuffer.ActionSchemaIds[NameIndex]);

			int32 ActionSize = EpisodeBuffer.ActionSizes[NameIndex];
			TLearningArray<2, float>& NewAction = Actions.AddDefaulted_GetRef();
			NewAction.SetNumUninitialized({ InMaxStepNum, ActionSize });
		}

		// Action Modifiers
		ActionModifiers.Empty();
		for (int32 NameIndex = 0; NameIndex < EpisodeBuffer.ActionModifierNames.Num(); NameIndex++)
		{
			ActionModifierNames.Add(EpisodeBuffer.ActionModifierNames[NameIndex]);
			ActionModifierSchemaIds.Add(EpisodeBuffer.ActionModifierSchemaIds[NameIndex]);

			int32 ActionModifierSize = EpisodeBuffer.ActionModifierSizes[NameIndex];
			TLearningArray<2, float>& NewActionModifier = ActionModifiers.AddDefaulted_GetRef();
			NewActionModifier.SetNumUninitialized({ InMaxStepNum, ActionModifierSize });
		}

		// Memory States
		MemoryStates.Empty();
		for (int32 NameIndex = 0; NameIndex < EpisodeBuffer.MemoryStateNames.Num(); NameIndex++)
		{
			MemoryStateNames.Add(EpisodeBuffer.MemoryStateNames[NameIndex]);

			int32 MemoryStateSize = EpisodeBuffer.MemoryStateSizes[NameIndex];

			TLearningArray<2, float>& NewMemoryState = MemoryStates.AddDefaulted_GetRef();
			NewMemoryState.SetNumUninitialized({ InMaxStepNum, MemoryStateSize });

			TLearningArray<2, float>& NewEpisodeFinalMemoryState = EpisodeFinalMemoryStates.AddDefaulted_GetRef();
			NewEpisodeFinalMemoryState.SetNumUninitialized({ InMaxEpisodeNum, MemoryStateSize });
		}

		// Rewards
		Rewards.Empty();
		for (int32 NameIndex = 0; NameIndex < EpisodeBuffer.RewardNames.Num(); NameIndex++)
		{
			RewardNames.Add(EpisodeBuffer.RewardNames[NameIndex]);

			int32 RewardSize = EpisodeBuffer.RewardSizes[NameIndex];
			TLearningArray<2, float>& NewReward = Rewards.AddDefaulted_GetRef();
			NewReward.SetNumUninitialized({ InMaxStepNum, RewardSize });
		}
	}

	void FReplayBuffer::Reset()
	{
		EpisodeNum = 0;
		StepNum = 0;
	}

	bool FReplayBuffer::AddEpisodes(
		const TLearningArrayView<1, const ECompletionMode> InEpisodeCompletionModes,
		const TArrayView<const TLearningArrayView<2, const float>> InEpisodeFinalObservations,
		const TArrayView<const TLearningArrayView<2, const float>> InEpisodeFinalMemoryStates,
		const FEpisodeBuffer& EpisodeBuffer,
		const FIndexSet Instances,
		const bool bAddTruncatedEpisodeWhenFull)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::FReplayBuffer::AddEpisodes);

		checkf(EpisodeBuffer.ObservationArrays.Num() == Observations.Num(), TEXT("Observation number mismatch!"));
		checkf(EpisodeBuffer.ObservationArrays.Num() == InEpisodeFinalObservations.Num(), TEXT("Final Observation number mismatch!"));
		checkf(EpisodeBuffer.ActionArrays.Num() == Actions.Num(), TEXT("Action number mismatch!"));
		checkf(EpisodeBuffer.ActionModifierArrays.Num() == ActionModifiers.Num(), TEXT("Action Modifier number mismatch!"));
		checkf(EpisodeBuffer.MemoryStateArrays.Num() == MemoryStates.Num(), TEXT("Memory State number mismatch!"));
		checkf(EpisodeBuffer.MemoryStateArrays.Num() == InEpisodeFinalMemoryStates.Num(), TEXT("Final Memory State number mismatch!"));
		checkf(EpisodeBuffer.RewardArrays.Num() == Rewards.Num(), TEXT("Reward number mismatch!"));

		for (const int32 InstanceIdx : Instances)
		{
			checkf(InEpisodeCompletionModes[InstanceIdx] != ECompletionMode::Running,
				TEXT("Tried to add experience from episode that is still running..."));

			const int32 EpisodeStepNum = EpisodeBuffer.GetEpisodeStepNums()[InstanceIdx];

			// Is there space for the full episode in the buffer?
			if (EpisodeNum < MaxEpisodeNum && StepNum + EpisodeStepNum <= MaxStepNum)
			{
				// Copy the data into the replay buffer

				for(int32 Index = 0; Index < Observations.Num(); Index++)
				{
					Array::Copy(
						Observations[Index].Slice(StepNum, EpisodeStepNum),
						EpisodeBuffer.ObservationArrays[Index][InstanceIdx].Slice(0, EpisodeBuffer.EpisodeStepNums[InstanceIdx]));
					Array::Copy(
						EpisodeFinalObservations[Index][EpisodeNum],
						InEpisodeFinalObservations[Index][InstanceIdx]);
				}

				for (int32 Index = 0; Index < Actions.Num(); Index++)
				{
					Array::Copy(
						Actions[Index].Slice(StepNum, EpisodeStepNum),
						EpisodeBuffer.ActionArrays[Index][InstanceIdx].Slice(0, EpisodeBuffer.EpisodeStepNums[InstanceIdx]));
				}

				for (int32 Index = 0; Index < ActionModifiers.Num(); Index++)
				{
					Array::Copy(
						ActionModifiers[Index].Slice(StepNum, EpisodeStepNum),
						EpisodeBuffer.ActionModifierArrays[Index][InstanceIdx].Slice(0, EpisodeBuffer.EpisodeStepNums[InstanceIdx]));
				}

				for (int32 Index = 0; Index < MemoryStates.Num(); Index++)
				{
					Array::Copy(
						MemoryStates[Index].Slice(StepNum, EpisodeStepNum),
						EpisodeBuffer.MemoryStateArrays[Index][InstanceIdx].Slice(0, EpisodeBuffer.EpisodeStepNums[InstanceIdx]));
					Array::Copy(
						EpisodeFinalMemoryStates[Index][EpisodeNum],
						InEpisodeFinalMemoryStates[Index][InstanceIdx]);
				}

				for (int32 Index = 0; Index < Rewards.Num(); Index++)
				{
					Array::Copy(
						Rewards[Index].Slice(StepNum, EpisodeStepNum),
						EpisodeBuffer.RewardArrays[Index][InstanceIdx].Slice(0, EpisodeBuffer.EpisodeStepNums[InstanceIdx]));
				}

				// Write the Episode start, length, completion
				EpisodeStarts[EpisodeNum] = StepNum;
				EpisodeLengths[EpisodeNum] = EpisodeStepNum;
				EpisodeCompletionModes[EpisodeNum] = InEpisodeCompletionModes[InstanceIdx];

				// Increment the Counts
				EpisodeNum++;
				StepNum += EpisodeStepNum;

				// Continue onto next Episode
				continue;
			}

			// Is there space for a partial episode in the buffer?
			if (bAddTruncatedEpisodeWhenFull && EpisodeNum < MaxEpisodeNum && StepNum < MaxStepNum)
			{
				const int32 PartialStepNum = MaxStepNum - StepNum;
				check(PartialStepNum > 0 && PartialStepNum < EpisodeStepNum);

				// Copy the data into the replay buffer

				for (int32 Index = 0; Index < Observations.Num(); Index++)
				{
					Array::Copy(
						Observations[Index].Slice(StepNum, PartialStepNum),
						EpisodeBuffer.ObservationArrays[Index][InstanceIdx].Slice(0, PartialStepNum));
					Array::Copy(
						EpisodeFinalObservations[Index][EpisodeNum],
						InEpisodeFinalObservations[Index][InstanceIdx]);
				}

				for (int32 Index = 0; Index < Actions.Num(); Index++)
				{
					Array::Copy(
						Actions[Index].Slice(StepNum, PartialStepNum),
						EpisodeBuffer.ActionArrays[Index][InstanceIdx].Slice(0, PartialStepNum));
				}

				for (int32 Index = 0; Index < ActionModifiers.Num(); Index++)
				{
					Array::Copy(
						ActionModifiers[Index].Slice(StepNum, PartialStepNum),
						EpisodeBuffer.ActionModifierArrays[Index][InstanceIdx].Slice(0, PartialStepNum));
				}

				for (int32 Index = 0; Index < MemoryStates.Num(); Index++)
				{
					Array::Copy(
						MemoryStates[Index].Slice(StepNum, PartialStepNum),
						EpisodeBuffer.MemoryStateArrays[Index][InstanceIdx].Slice(0, PartialStepNum));
					Array::Copy(
						EpisodeFinalMemoryStates[Index][EpisodeNum],
						InEpisodeFinalMemoryStates[Index][InstanceIdx]);
				}

				for (int32 Index = 0; Index < Rewards.Num(); Index++)
				{
					Array::Copy(
						Rewards[Index].Slice(StepNum, PartialStepNum),
						EpisodeBuffer.RewardArrays[Index][InstanceIdx].Slice(0, PartialStepNum));
				}

				// Write the Episode start, length, completion
				EpisodeStarts[EpisodeNum] = StepNum;
				EpisodeLengths[EpisodeNum] = PartialStepNum;
				EpisodeCompletionModes[EpisodeNum] = ECompletionMode::Truncated;

				// Increment the Counts
				EpisodeNum++;
				StepNum += PartialStepNum;
			}

			// Otherwise buffer is full
			return true;
		}

		return (EpisodeNum == MaxEpisodeNum) || (StepNum == MaxStepNum);
	}

	void FReplayBuffer::AddRecords(
		const int32 InEpisodeNum,
		const int32 InStepNum,
		const int32 ObservationSchemaId,
		const int32 ObservationNum,
		const int32 ActionSchemaId,
		const int32 ActionNum,
		const TLearningArrayView<1, const int32> RecordedEpisodeStarts,
		const TLearningArrayView<1, const int32> RecordedEpisodeLengths,
		const TLearningArrayView<2, const float> RecordedObservations,
		const TLearningArrayView<2, const float> RecordedActions)
	{
		bHasCompletions = false;
		bHasFinalObservations = false;
		bHasFinalMemoryStates = false;

		MaxEpisodeNum = InEpisodeNum;
		EpisodeNum = InEpisodeNum;

		MaxStepNum = InStepNum;
		StepNum = InStepNum;

		EpisodeStarts.SetNumUninitialized({ InEpisodeNum });
		UE::Learning::Array::Copy(EpisodeStarts, RecordedEpisodeStarts);

		EpisodeLengths.SetNumUninitialized({ InEpisodeNum });
		UE::Learning::Array::Copy(EpisodeLengths, RecordedEpisodeLengths);

		Observations.Empty();
		ObservationNames.Add("Observations");
		ObservationSchemaIds.Add(ObservationSchemaId);
		TLearningArray<2, float>& NewObservation = Observations.AddDefaulted_GetRef();
		NewObservation.SetNumUninitialized({ InStepNum, ObservationNum });
		UE::Learning::Array::Copy(NewObservation, RecordedObservations);

		Actions.Empty();
		ActionNames.Add("Actions");
		ActionSchemaIds.Add(ActionSchemaId);
		TLearningArray<2, float>& NewAction = Actions.AddDefaulted_GetRef();
		NewAction.SetNumUninitialized({ InStepNum, ActionNum });
		UE::Learning::Array::Copy(NewAction, RecordedActions);
	}

	bool FReplayBuffer::HasCompletions() const
	{
		return bHasCompletions;
	}

	bool FReplayBuffer::HasFinalObservations() const
	{
		return bHasFinalObservations;
	}

	bool FReplayBuffer::HasFinalMemoryStates() const
	{
		return bHasFinalMemoryStates;
	}

	int32 FReplayBuffer::GetMaxEpisodeNum() const
	{
		return MaxEpisodeNum;
	}

	int32 FReplayBuffer::GetMaxStepNum() const
	{
		return MaxStepNum;
	}

	int32 FReplayBuffer::GetEpisodeNum() const
	{
		return EpisodeNum;
	}

	int32 FReplayBuffer::GetStepNum() const
	{
		return StepNum;
	}

	const TLearningArrayView<1, const int32> FReplayBuffer::GetEpisodeStarts() const
	{
		return EpisodeStarts.Slice(0, EpisodeNum);
	}

	const TLearningArrayView<1, const int32> FReplayBuffer::GetEpisodeLengths() const
	{
		return EpisodeLengths.Slice(0, EpisodeNum);
	}

	const TLearningArrayView<1, const ECompletionMode> FReplayBuffer::GetEpisodeCompletionModes() const
	{
		return EpisodeCompletionModes.Slice(0, EpisodeNum);
	}

	int32 FReplayBuffer::GetObservationsNum() const
	{
		return Observations.Num();
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetObservations(const int32 Index) const
	{
		return Observations[Index].Slice(0, StepNum);
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetEpisodeFinalObservations(const int32 Index) const
	{
		return EpisodeFinalObservations[Index].Slice(0, EpisodeNum);
	}

	int32 FReplayBuffer::GetActionsNum() const
	{
		return Actions.Num();
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetActions(const int32 Index) const
	{
		return Actions[Index].Slice(0, StepNum);
	}

	int32 FReplayBuffer::GetActionModifiersNum() const
	{
		return ActionModifiers.Num();
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetActionModifiers(const int32 Index) const
	{
		return ActionModifiers[Index].Slice(0, StepNum);
	}

	int32 FReplayBuffer::GetMemoryStatesNum() const
	{
		return MemoryStates.Num();
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetMemoryStates(const int32 Index) const
	{
		return MemoryStates[Index].Slice(0, StepNum);
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetEpisodeFinalMemoryStates(const int32 Index) const
	{
		return EpisodeFinalMemoryStates[Index].Slice(0, EpisodeNum);
	}

	int32 FReplayBuffer::GetRewardsNum() const
	{
		return Rewards.Num();
	}

	const TLearningArrayView<2, const float> FReplayBuffer::GetRewards(const int32 Index) const
	{
		return Rewards[Index].Slice(0, StepNum);
	}

	TSharedRef<FJsonObject> FReplayBuffer::AsJsonConfig(const int32 ReplayBufferId) const
	{
		TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

		ConfigObject->SetNumberField(TEXT("Id"), ReplayBufferId);

		// Replay Buffer Settings
		ConfigObject->SetNumberField(TEXT("MaxEpisodeNum"), MaxEpisodeNum);
		ConfigObject->SetNumberField(TEXT("MaxStepNum"), MaxStepNum);

		ConfigObject->SetBoolField(TEXT("HasCompletions"), HasCompletions());
		ConfigObject->SetBoolField(TEXT("HasFinalObservations"), HasFinalObservations());
		ConfigObject->SetBoolField(TEXT("HasFinalMemoryStates"), HasFinalMemoryStates());

		// Observations
		TArray<TSharedPtr<FJsonValue>> ObservationObjects;
		for(int32 Index = 0; Index < Observations.Num(); Index++)
		{
			TSharedPtr<FJsonObject> BufferObject = MakeShared<FJsonObject>();
			BufferObject->SetNumberField(TEXT("Id"), Index);
			BufferObject->SetStringField(TEXT("Name"), ObservationNames[Index].ToString());
			BufferObject->SetNumberField(TEXT("SchemaId"), ObservationSchemaIds[Index]);
			BufferObject->SetNumberField(TEXT("VectorDimensionNum"), Observations[Index].Num<1>());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(BufferObject);
			ObservationObjects.Add(JsonValue);
		}
		ConfigObject->SetArrayField(TEXT("Observations"), ObservationObjects);

		// Actions
		TArray<TSharedPtr<FJsonValue>> ActionObjects;
		for (int32 Index = 0; Index < Actions.Num(); Index++)
		{
			TSharedPtr<FJsonObject> BufferObject = MakeShared<FJsonObject>();
			BufferObject->SetNumberField(TEXT("Id"), Index);
			BufferObject->SetStringField(TEXT("Name"), ActionNames[Index].ToString());
			BufferObject->SetNumberField(TEXT("SchemaId"), ActionSchemaIds[Index]);
			BufferObject->SetNumberField(TEXT("VectorDimensionNum"), Actions[Index].Num<1>());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(BufferObject);
			ActionObjects.Add(JsonValue);
		}
		ConfigObject->SetArrayField(TEXT("Actions"), ActionObjects);

		// Action Modifiers
		TArray<TSharedPtr<FJsonValue>> ActionModifierObjects;
		for (int32 Index = 0; Index < ActionModifiers.Num(); Index++)
		{
			TSharedPtr<FJsonObject> BufferObject = MakeShared<FJsonObject>();
			BufferObject->SetNumberField(TEXT("Id"), Index);
			BufferObject->SetStringField(TEXT("Name"), ActionModifierNames[Index].ToString());
			BufferObject->SetNumberField(TEXT("SchemaId"), ActionModifierSchemaIds[Index]);
			BufferObject->SetNumberField(TEXT("VectorDimensionNum"), ActionModifiers[Index].Num<1>());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(BufferObject);
			ActionModifierObjects.Add(JsonValue);
		}
		ConfigObject->SetArrayField(TEXT("ActionModifiers"), ActionModifierObjects);

		// Memory States
		TArray<TSharedPtr<FJsonValue>> MemoryStateObjects;
		for (int32 Index = 0; Index < MemoryStates.Num(); Index++)
		{
			TSharedPtr<FJsonObject> BufferObject = MakeShared<FJsonObject>();
			BufferObject->SetNumberField(TEXT("Id"), Index);
			BufferObject->SetStringField(TEXT("Name"), MemoryStateNames[Index].ToString());
			BufferObject->SetNumberField(TEXT("VectorDimensionNum"), MemoryStates[Index].Num<1>());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(BufferObject);
			MemoryStateObjects.Add(JsonValue);
		}
		ConfigObject->SetArrayField(TEXT("MemoryStates"), MemoryStateObjects);

		// Rewards
		TArray<TSharedPtr<FJsonValue>> RewardObjects;
		for (int32 Index = 0; Index < Rewards.Num(); Index++)
		{
			TSharedPtr<FJsonObject> BufferObject = MakeShared<FJsonObject>();
			BufferObject->SetNumberField(TEXT("Id"), Index);
			BufferObject->SetStringField(TEXT("Name"), RewardNames[Index].ToString());
			BufferObject->SetNumberField(TEXT("VectorDimensionNum"), Rewards[Index].Num<1>());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(BufferObject);
			RewardObjects.Add(JsonValue);
		}
		ConfigObject->SetArrayField(TEXT("Rewards"), RewardObjects);

		return ConfigObject;
	}

	namespace Experience
	{
		void GatherExperienceUntilReplayBufferFull(
			FReplayBuffer& ReplayBuffer,
			FEpisodeBuffer& EpisodeBuffer,
			FResetInstanceBuffer& ResetBuffer,
			const TArrayView<const TLearningArrayView<2, const float>> ObservationVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> ActionVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> PreEvaluationMemoryStateVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> MemoryStateVectorBuffers,
			const TArrayView<const TLearningArrayView<1, const float>> RewardBuffers,
			TLearningArrayView<1, ECompletionMode> CompletionBuffer,
			TLearningArrayView<1, ECompletionMode> EpisodeCompletionBuffer,
			TLearningArrayView<1, ECompletionMode> AllCompletionBuffer,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> ObservationFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> PolicyFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> ActionFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> UpdateFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> RewardFunctions,
			const TFunctionRef<void(const FIndexSet Instances)> CompletionFunction,
			const FIndexSet Instances)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Learning::Experience::GatherExperienceUntilReplayBufferFull);

			// Reset Everything
			ReplayBuffer.Reset();
			EpisodeBuffer.Reset(Instances);
			ResetFunction(Instances);

			while (true)
			{
				// Encode Observations
				for (int32 Index = 0; Index < ObservationFunctions.Num(); Index++)
				{
					ObservationFunctions[Index](Instances);
					EpisodeBuffer.PushObservations(Index, ObservationVectorBuffers[Index], Instances);
				}

				// Evaluate Policy
				for (int32 Index = 0; Index < PolicyFunctions.Num(); Index++)
				{
					PolicyFunctions[Index](Instances);
					EpisodeBuffer.PushMemoryStates(Index, PreEvaluationMemoryStateVectorBuffers[Index], Instances);
				}

				// Decode Actions
				for (int32 Index = 0; Index < ActionFunctions.Num(); Index++)
				{
					ActionFunctions[Index](Instances);
					EpisodeBuffer.PushActions(Index, ActionVectorBuffers[Index], Instances);
				}

				// Update Environment
				for (int32 Index = 0; Index < UpdateFunctions.Num(); Index++)
				{
					UpdateFunctions[Index](Instances);
				}

				// Compute Rewards
				for (int32 Index = 0; Index < RewardFunctions.Num(); Index++)
				{
					RewardFunctions[Index](Instances);
					EpisodeBuffer.PushRewards(Index, RewardBuffers[Index], Instances);
				}

				EpisodeBuffer.IncrementEpisodeStepNums(Instances);

				// Evaluate Completions
				CompletionFunction(Instances);

				Completion::EvaluateEndOfEpisodeCompletions(
					EpisodeCompletionBuffer,
					EpisodeBuffer.GetEpisodeStepNums(),
					EpisodeBuffer.GetMaxStepNum(),
					Instances);

				for (const int32 Instance : Instances)
				{
					AllCompletionBuffer[Instance] = Completion::Or(CompletionBuffer[Instance], EpisodeCompletionBuffer[Instance]);
				}

				ResetBuffer.SetResetInstancesFromCompletions(AllCompletionBuffer, Instances);

				if (ResetBuffer.GetResetInstanceNum() == 0)
				{
					continue;
				}

				// Evaluate Observations again for instances that are completed
				for (int32 Index = 0; Index < ObservationFunctions.Num(); Index++)
				{
					ObservationFunctions[Index](ResetBuffer.GetResetInstances());
				}

				// Push completed instances to Replay Buffer and return if full

				if (ReplayBuffer.AddEpisodes(
					AllCompletionBuffer,
					ObservationVectorBuffers,
					MemoryStateVectorBuffers,
					EpisodeBuffer,
					ResetBuffer.GetResetInstances()))
				{
					return;
				}

				// Just reset Episode Buffer for instances who reached the maximum episode length
				ResetBuffer.SetResetInstancesFromCompletions(EpisodeCompletionBuffer, Instances);
				EpisodeBuffer.Reset(ResetBuffer.GetResetInstances());

				// Call Reset Function for instances which signaled a completion
				ResetBuffer.SetResetInstancesFromCompletions(CompletionBuffer, Instances);
				ResetFunction(ResetBuffer.GetResetInstances());
			}
		}

	}
}
