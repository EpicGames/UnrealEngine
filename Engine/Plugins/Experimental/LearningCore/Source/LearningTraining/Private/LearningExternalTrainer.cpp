// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningExternalTrainer.h"

#include "LearningExperience.h"
#include "LearningNeuralNetwork.h"
#include "LearningSharedMemoryTraining.h"
#include "LearningSocketTraining.h"
#include "LearningProgress.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"

#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

#include "Algo/AllOf.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "Sockets.h"
#include "Common/TcpSocketBuilder.h"
#include "IPAddressAsyncResolve.h"
#include "SocketSubsystem.h"

namespace UE::Learning
{
	FSharedMemoryTrainerServerProcess::FSharedMemoryTrainerServerProcess(
		const FString& TaskName,
		const FString& CustomTrainerPath,
		const FString& TrainerFileName,
		const FString& PythonExecutablePath,
		const FString& PythonContentPath,
		const FString& InIntermediatePath,
		const FString& NneCpuContentPath,
		const float InTimeout,
		const ESubprocessFlags TrainingProcessFlags,
		const ELogSetting LogSettings)
	{
		check(FPaths::FileExists(PythonExecutablePath));
		check(FPaths::DirectoryExists(PythonContentPath));

		Timeout = InTimeout;
		IntermediatePath = InIntermediatePath;

		Controls = SharedMemory::Allocate<1, volatile int32>({ SharedMemoryTraining::GetControlNum() });

		// We need to zero the control memory before we start the training sub-process since it may contain
		// uninitialized values or those left over from previous runs.
		Array::Zero(Controls.View);

		// Set the ID columns to -1
		Controls.View[(uint8)SharedMemoryTraining::EControls::NetworkId] = -1;
		Controls.View[(uint8)SharedMemoryTraining::EControls::ReplayBufferId] = -1;

		const FString TimeStamp = FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S"));
		const FString TrainerType = TEXT("SharedMemory");
		
		// Make a config directory for the task
		int32 TaskId = 0;
		while (true)
		{
			FString CandidatePath = InIntermediatePath / TaskName + FString::FromInt(TaskId) / TEXT("Configs");
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.DirectoryExists(*CandidatePath))
			{
				PlatformFile.CreateDirectory(*CandidatePath);
				ConfigPath = CandidatePath;
				break;
			}

			TaskId++;
		}

		IFileManager& FileManager = IFileManager::Get();

		const FString CommandLineArguments = FString::Printf(TEXT("\"%s\" \"%s\" -p \"%s\" -m \"%s\" \"%s\" --nne-cpu-path \"%s\" SharedMemory \"%s\" -g \"%s\""),
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*(PythonContentPath / TEXT("train.py"))),
			*TaskName,
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*CustomTrainerPath),
			*TrainerFileName,
			LogSettings == ELogSetting::Normal ? TEXT("-l") : TEXT(""),
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*NneCpuContentPath),
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*(InIntermediatePath / TaskName + FString::FromInt(TaskId))),
			*Controls.GetPlatformGuidString());

		bool bLaunched = TrainingProcess.Launch(
			FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonExecutablePath),
			CommandLineArguments, 
			TrainingProcessFlags);

		ensure(bLaunched);
	}

	bool FSharedMemoryTrainerServerProcess::IsRunning() const
	{
		return TrainingProcess.IsRunning();
	}

	bool FSharedMemoryTrainerServerProcess::Wait()
	{
		const float SleepTime = 0.001f;
		float WaitTime = 0.0f;

		while (TrainingProcess.Update())
		{
			FPlatformProcess::Sleep(SleepTime);
			WaitTime += SleepTime;

			if (WaitTime > Timeout)
			{
				return false;
			}
		}

		return true;
	}

	void FSharedMemoryTrainerServerProcess::Terminate()
	{
		TrainingProcess.Terminate();
	}

	TSharedMemoryArrayView<1, volatile int32> FSharedMemoryTrainerServerProcess::GetControlsSharedMemoryArrayView() const
	{
		return Controls;
	}

	const FString& FSharedMemoryTrainerServerProcess::GetIntermediatePath() const
	{
		return IntermediatePath;
	}

	const FString& FSharedMemoryTrainerServerProcess::GetConfigPath() const
	{
		return ConfigPath;
	}

	FSubprocess* FSharedMemoryTrainerServerProcess::GetTrainingSubprocess()
	{
		return &TrainingProcess;
	}

	void FSharedMemoryTrainerServerProcess::Deallocate()
	{
		if (Controls.Region != nullptr)
		{
			SharedMemory::Deallocate(Controls);
		}
	}

	void FSharedMemoryTrainer::FSharedMemoryExperienceContainer::Deallocate()
	{
		if (EpisodeStarts.Region != nullptr)
		{
			SharedMemory::Deallocate(EpisodeStarts);
			SharedMemory::Deallocate(EpisodeLengths);
			SharedMemory::Deallocate(EpisodeCompletionModes);

			for(TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : EpisodeFinalObservations)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : EpisodeFinalMemoryStates)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : Observations)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : Actions)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : ActionModifiers)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : MemoryStates)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}

			for (TSharedMemoryArrayView<2, float>& SharedMemoryArrayView : Rewards)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}
		}
	}

	FSharedMemoryTrainer::FSharedMemoryTrainer(
		const FString& InTaskName,
		const TSharedPtr<FSharedMemoryTrainerServerProcess>& ExternalTrainerProcess,
		const bool bUseExternalTrainingProcess,
		const FGuid ControlsGuid,
		const FString& InConfigPath,
		const FString& InIntermediatePath,
		const float InTimeout)
	{
		TaskName = InTaskName;
		Timeout = InTimeout;
		bUseExternalTrainer = bUseExternalTrainingProcess;

		if (ExternalTrainerProcess)
		{
			TrainingProcess = ExternalTrainerProcess;
			ConfigPath = ExternalTrainerProcess->GetConfigPath();
			IntermediatePath = ExternalTrainerProcess->GetIntermediatePath();
			Controls = ExternalTrainerProcess->GetControlsSharedMemoryArrayView();	
		}
		else if (!bUseExternalTrainingProcess)
		{
			UE_LOGF(LogLearning, Error, "ExternalTrainerProcess was null, but attach to existing trainer process was false. Either pass in external trainer process or set bUseExternalTrainingProcess to true in the settings (along with associated settings).")
			return;
		}
		else
		{
			Controls = SharedMemory::Map<1, volatile int32>(ControlsGuid, { SharedMemoryTraining::GetControlNum() });

			// We need to zero the control memory before we start the training sub-process since it may contain
			// uninitialized values or those left over from previous runs.
			Array::Zero(Controls.View);

			// Set the ID columns to -1
			Controls.View[(uint8)SharedMemoryTraining::EControls::NetworkId] = -1;
			Controls.View[(uint8)SharedMemoryTraining::EControls::ReplayBufferId] = -1;

			ConfigPath = InConfigPath;
			IntermediatePath = InIntermediatePath;
		}
	}

	FSharedMemoryTrainer::~FSharedMemoryTrainer()
	{
		Terminate();
	}

	bool FSharedMemoryTrainer::IsValid()
	{
		return bUseExternalTrainer || TrainingProcess != nullptr;
	}

	ETrainerResponse FSharedMemoryTrainer::Wait()
	{
		return ETrainerResponse::Success;
	}

	bool FSharedMemoryTrainer::HasNetworkOrCompleted()
	{
		return SharedMemoryTraining::HasNetworkOrCompleted(Controls.View);
	}

	void FSharedMemoryTrainer::Terminate()
	{
		Deallocate();
	}

	ETrainerResponse FSharedMemoryTrainer::SendStop()
	{
		checkf(Controls.Region, TEXT("SendStop: Controls Shared Memory Region is nullptr"));
		return SharedMemoryTraining::SendStop(Controls.View);
	}

	ETrainerResponse FSharedMemoryTrainer::SendConfigs(
		const TSharedRef<FJsonObject>& DataConfigObject,
		const TSharedRef<FJsonObject>& ModelConfigObject,
		const TSharedRef<FJsonObject>& TrainerConfigObject,
		const ELogSetting LogSettings)
	{
		IFileManager& FileManager = IFileManager::Get();

		// Add intermediate path as a temp directory for the tensorboard logging on python side
		TrainerConfigObject->SetStringField(TEXT("TempDirectory"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*IntermediatePath));
		
		TSharedRef<FJsonObject> SharedMemoryConfigObject = MakeShared<FJsonObject>();

		TArray<TSharedPtr<FJsonValue>> NetworkGuidsArray;
		for(int32 Index = 0; Index < NeuralNetworkSharedMemoryArrayViews.Num(); Index++)
		{
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetNumberField(TEXT("NetworkId"), Index);
			JsonObject->SetStringField(TEXT("Guid"), *NeuralNetworkSharedMemoryArrayViews[Index].GetPlatformGuidString());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(JsonObject);
			NetworkGuidsArray.Add(JsonValue);
		}
		SharedMemoryConfigObject->SetArrayField(TEXT("NetworkGuids"), NetworkGuidsArray);

		TArray<TSharedPtr<FJsonValue>> ExperienceContainerObjectsArray;
		for (const FSharedMemoryExperienceContainer& SharedMemoryExperienceContainer : SharedMemoryExperienceContainers)
		{
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			JsonObject->SetStringField(TEXT("EpisodeStartsGuid"), *SharedMemoryExperienceContainer.EpisodeStarts.GetPlatformGuidString());
			JsonObject->SetStringField(TEXT("EpisodeLengthsGuid"), *SharedMemoryExperienceContainer.EpisodeLengths.GetPlatformGuidString());
			JsonObject->SetStringField(TEXT("EpisodeCompletionModesGuid"), *SharedMemoryExperienceContainer.EpisodeCompletionModes.GetPlatformGuidString());

			TArray<TSharedPtr<FJsonValue>> EpisodeFinalObservationsGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& EpisodeFinalObservations : SharedMemoryExperienceContainer.EpisodeFinalObservations)
			{
				EpisodeFinalObservationsGuidsArray.Add(MakeShared<FJsonValueString>(*EpisodeFinalObservations.GetPlatformGuidString()));
				
			}
			JsonObject->SetArrayField(TEXT("EpisodeFinalObservationsGuids"), EpisodeFinalObservationsGuidsArray);

			TArray<TSharedPtr<FJsonValue>> EpisodeFinalMemoryStatesGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& EpisodeFinalMemoryStates : SharedMemoryExperienceContainer.EpisodeFinalMemoryStates)
			{
				EpisodeFinalMemoryStatesGuidsArray.Add(MakeShared<FJsonValueString>(*EpisodeFinalMemoryStates.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("EpisodeFinalMemoryStatesGuids"), EpisodeFinalMemoryStatesGuidsArray);

			TArray<TSharedPtr<FJsonValue>> ObservationsGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& Observations : SharedMemoryExperienceContainer.Observations)
			{
				ObservationsGuidsArray.Add(MakeShared<FJsonValueString>(*Observations.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("ObservationsGuids"), ObservationsGuidsArray);

			TArray<TSharedPtr<FJsonValue>> ActionsGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& Actions : SharedMemoryExperienceContainer.Actions)
			{
				ActionsGuidsArray.Add(MakeShared<FJsonValueString>(*Actions.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("ActionsGuids"), ActionsGuidsArray);

			TArray<TSharedPtr<FJsonValue>> ActionModifiersGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& ActionModifiers : SharedMemoryExperienceContainer.ActionModifiers)
			{
				ActionModifiersGuidsArray.Add(MakeShared<FJsonValueString>(*ActionModifiers.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("ActionModifiersGuids"), ActionModifiersGuidsArray);


			TArray<TSharedPtr<FJsonValue>> MemoryStatesGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& MemoryStates : SharedMemoryExperienceContainer.MemoryStates)
			{
				MemoryStatesGuidsArray.Add(MakeShared<FJsonValueString>(*MemoryStates.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("MemoryStatesGuids"), MemoryStatesGuidsArray);

			TArray<TSharedPtr<FJsonValue>> RewardsGuidsArray;
			for (const TSharedMemoryArrayView<2, float>& Rewards : SharedMemoryExperienceContainer.Rewards)
			{
				RewardsGuidsArray.Add(MakeShared<FJsonValueString>(*Rewards.GetPlatformGuidString()));

			}
			JsonObject->SetArrayField(TEXT("RewardsGuids"), RewardsGuidsArray);
			
			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(JsonObject);
			ExperienceContainerObjectsArray.Add(JsonValue);
		}
		SharedMemoryConfigObject->SetArrayField(TEXT("ReplayBuffers"), ExperienceContainerObjectsArray);
		
		// Write all the configs
		FString SharedMemoryConfigString;
		TSharedRef<TJsonWriter<>> SharedMemoryJsonWriter = TJsonWriterFactory<>::Create(&SharedMemoryConfigString, 0);
		FJsonSerializer::Serialize(SharedMemoryConfigObject, SharedMemoryJsonWriter, true);
		FFileHelper::SaveStringToFile(SharedMemoryConfigString, *(ConfigPath / FString::Printf(TEXT("shared-memory-%s.json"), *Controls.GetPlatformGuidString())));

		// Merge model config into data config for shared memory protocol compatibility
		for (const auto& Pair : ModelConfigObject->Values)
		{
			DataConfigObject->SetField(Pair.Key, Pair.Value);
		}

		FString DataConfigString;
		TSharedRef<TJsonWriter<>> DataJsonWriter = TJsonWriterFactory<>::Create(&DataConfigString, 0);
		FJsonSerializer::Serialize(DataConfigObject, DataJsonWriter, true);
		FFileHelper::SaveStringToFile(DataConfigString, *(ConfigPath / "data-config.json"));

		FString TrainerConfigString;
		TSharedRef<TJsonWriter<>> TrainerJsonWriter = TJsonWriterFactory<>::Create(&TrainerConfigString, 0);
		FJsonSerializer::Serialize(TrainerConfigObject, TrainerJsonWriter, true);
		FFileHelper::SaveStringToFile(TrainerConfigString, *(ConfigPath / "trainer-config.json"));

		UE_LOGF(LogLearning, Display, "Wrote Config Files to %ls. Sending Config Signal...", *ConfigPath);

		return SharedMemoryTraining::SendConfigSignal(Controls.View, LogSettings);
	}

	int32 FSharedMemoryTrainer::AddNetwork(const ULearningNeuralNetworkData& Network)
	{
		const int32 NetworkId = NeuralNetworkSharedMemoryArrayViews.Num();
		NeuralNetworkSharedMemoryArrayViews.Add(SharedMemory::Allocate<1, uint8>({ Network.GetSnapshotByteNum() }));
		return NetworkId;
	}

	ETrainerResponse FSharedMemoryTrainer::ReceiveNetwork(
		const int32 NetworkId,
		ULearningNeuralNetworkData& OutNetwork,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		checkf(Controls.Region, TEXT("ReceiveNetwork: Controls Shared Memory Region is nullptr"));
		if (!ensureMsgf(NeuralNetworkSharedMemoryArrayViews.Num() >= NetworkId, TEXT("Network %d has not been added. Call AddNetwork prior to ReceiveNetwork."), NetworkId))
		{
			return ETrainerResponse::Unexpected;
		}

		return SharedMemoryTraining::RecvNetwork(
			Controls.View,
			NetworkId,
			OutNetwork,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			NeuralNetworkSharedMemoryArrayViews[NetworkId].View,
			Timeout,
			NetworkLock,
			LogSettings);
	}

	TArray<ETrainerResponse> FSharedMemoryTrainer::ReceiveNetworks(
		const TArray<int32>& NetworkIds,
		TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
		TArray<FRWLock*> NetworkLocks,
		const ELogSetting LogSettings)
	{
		checkf(Controls.Region, TEXT("ReceiveNetworks: Controls Shared Memory Region is nullptr"));		
		check(NetworkIds.Num() == Networks.Num());

		TArray<ETrainerResponse> Responses;
		Responses.Init(ETrainerResponse::Unexpected, NetworkIds.Num());

		for (int32 i = 0; i < NetworkIds.Num(); ++i)
		{
			if (!ensureMsgf(NeuralNetworkSharedMemoryArrayViews.Num() >= NetworkIds[i], TEXT("Network %d has not been added. Call AddNetwork prior to ReceiveNetwork."), NetworkIds[i]))
			{
				return Responses;
			}

			Responses[i] = SharedMemoryTraining::RecvNetwork(
				Controls.View,
				NetworkIds[i],
				*Networks[i],
				TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
				NeuralNetworkSharedMemoryArrayViews[NetworkIds[i]].View,
				Timeout,
				NetworkLocks.IsValidIndex(i) ? NetworkLocks[i] : nullptr,
				LogSettings);
		}
		return Responses;
	}

	ETrainerResponse FSharedMemoryTrainer::SendNetwork(
		const int32 NetworkId,
		const ULearningNeuralNetworkData& Network,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		checkf(Controls.Region, TEXT("SendNetwork: Controls Shared Memory Region is nullptr"));
		if (!ensureMsgf(NeuralNetworkSharedMemoryArrayViews.Num() >= NetworkId, TEXT("Network %d has not been added. Call AddNetwork prior to SendNetwork."), NetworkId))
		{
			return ETrainerResponse::Unexpected;
		}

		return SharedMemoryTraining::SendNetwork(
			Controls.View,
			NetworkId,
			NeuralNetworkSharedMemoryArrayViews[NetworkId].View,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			Network,
			Timeout,
			NetworkLock,
			LogSettings);
	}

	int32 FSharedMemoryTrainer::AddReplayBuffer(const FReplayBuffer& ReplayBuffer)
	{
		FSharedMemoryExperienceContainer ExperienceContainer;

		ExperienceContainer.EpisodeStarts = SharedMemory::Allocate<1, int32>({ ReplayBuffer.GetMaxEpisodeNum() });
		ExperienceContainer.EpisodeLengths = SharedMemory::Allocate<1, int32>({ ReplayBuffer.GetMaxEpisodeNum() });

		if (ReplayBuffer.HasCompletions())
		{
			ExperienceContainer.EpisodeCompletionModes = SharedMemory::Allocate<1, ECompletionMode>({ ReplayBuffer.GetMaxEpisodeNum() });
		}

		if (ReplayBuffer.HasFinalObservations())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
			{
				const int32 DimNum = ReplayBuffer.GetEpisodeFinalObservations(Index).Num<1>();
				ExperienceContainer.EpisodeFinalObservations.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxEpisodeNum(), DimNum }));
			}
		}

		if (ReplayBuffer.HasFinalMemoryStates())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
			{
				const int32 DimNum = ReplayBuffer.GetEpisodeFinalMemoryStates(Index).Num<1>();
				ExperienceContainer.EpisodeFinalMemoryStates.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxEpisodeNum(), DimNum }));
			}
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
		{
			const int32 DimNum = ReplayBuffer.GetObservations(Index).Num<1>();
			ExperienceContainer.Observations.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxStepNum(), DimNum }));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionsNum(); Index++)
		{
			const int32 DimNum = ReplayBuffer.GetActions(Index).Num<1>();
			ExperienceContainer.Actions.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxStepNum(), DimNum }));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionModifiersNum(); Index++)
		{
			const int32 DimNum = ReplayBuffer.GetActionModifiers(Index).Num<1>();
			ExperienceContainer.ActionModifiers.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxStepNum(), DimNum }));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
		{
			const int32 DimNum = ReplayBuffer.GetMemoryStates(Index).Num<1>();
			ExperienceContainer.MemoryStates.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxStepNum(), DimNum }));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetRewardsNum(); Index++)
		{
			const int32 DimNum = ReplayBuffer.GetRewards(Index).Num<1>();
			ExperienceContainer.Rewards.Add(SharedMemory::Allocate<2, float>({ ReplayBuffer.GetMaxStepNum(), DimNum }));
		}
		
		const int32 ReplayBufferId = SharedMemoryExperienceContainers.Num();
		SharedMemoryExperienceContainers.Add(ExperienceContainer);
		return ReplayBufferId;
	}

	ETrainerResponse FSharedMemoryTrainer::SendReplayBuffer(const int32 ReplayBufferId, const FReplayBuffer& ReplayBuffer, const ELogSetting LogSettings)
	{
		checkf(Controls.Region, TEXT("SendReplayBuffer: Controls Shared Memory Region is nullptr"));
		if (!ensureMsgf(SharedMemoryExperienceContainers.Num() >= ReplayBufferId, TEXT("ReplayBuffer %d has not been added. Call AddReplayBuffer prior to SendReplayBuffer."), ReplayBufferId))
		{
			return ETrainerResponse::Unexpected;
		}

		TArray<TLearningArrayView<2, float>> EpisodeFinalObservations;
		for (TSharedMemoryArrayView<2, float>& EpisodeFinalObs : SharedMemoryExperienceContainers[ReplayBufferId].EpisodeFinalObservations)
		{
			EpisodeFinalObservations.Add(EpisodeFinalObs.View);
		}

		TArray<TLearningArrayView<2, float>> EpisodeFinalMemoryStates;
		for (TSharedMemoryArrayView<2, float>& EpisodeFinalMems : SharedMemoryExperienceContainers[ReplayBufferId].EpisodeFinalMemoryStates)
		{
			EpisodeFinalMemoryStates.Add(EpisodeFinalMems.View);
		}

		TArray<TLearningArrayView<2, float>> Observations;
		for (TSharedMemoryArrayView<2, float>& Obs : SharedMemoryExperienceContainers[ReplayBufferId].Observations)
		{
			Observations.Add(Obs.View);
		}

		TArray<TLearningArrayView<2, float>> Actions;
		for (TSharedMemoryArrayView<2, float>& Acts : SharedMemoryExperienceContainers[ReplayBufferId].Actions)
		{
			Actions.Add(Acts.View);
		}

		TArray<TLearningArrayView<2, float>> ActionModifiers;
		for (TSharedMemoryArrayView<2, float>& Mods : SharedMemoryExperienceContainers[ReplayBufferId].ActionModifiers)
		{
			ActionModifiers.Add(Mods.View);
		}

		TArray<TLearningArrayView<2, float>> MemoryStates;
		for (TSharedMemoryArrayView<2, float>& Mems : SharedMemoryExperienceContainers[ReplayBufferId].MemoryStates)
		{
			MemoryStates.Add(Mems.View);
		}

		TArray<TLearningArrayView<2, float>> Rewards;
		for (TSharedMemoryArrayView<2, float>& Rews : SharedMemoryExperienceContainers[ReplayBufferId].Rewards)
		{
			Rewards.Add(Rews.View);
		}

		TLearningArrayView<1, ECompletionMode> EmptyCompletionsArray;
		return SharedMemoryTraining::SendExperience(
			SharedMemoryExperienceContainers[ReplayBufferId].EpisodeStarts.View,
			SharedMemoryExperienceContainers[ReplayBufferId].EpisodeLengths.View,
			ReplayBuffer.HasCompletions() ? SharedMemoryExperienceContainers[ReplayBufferId].EpisodeCompletionModes.View : EmptyCompletionsArray,
			EpisodeFinalObservations,
			EpisodeFinalMemoryStates,
			Observations,
			Actions,
			ActionModifiers,
			MemoryStates,
			Rewards,
			Controls.View,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			ReplayBufferId,
			ReplayBuffer,
			Timeout,
			LogSettings);
	}

	void FSharedMemoryTrainer::Deallocate()
	{
		for (TSharedMemoryArrayView<1, uint8>& SharedMemoryArrayView : NeuralNetworkSharedMemoryArrayViews)
		{
			if (SharedMemoryArrayView.Region != nullptr)
			{
				SharedMemory::Deallocate(SharedMemoryArrayView);
			}
		}
		NeuralNetworkSharedMemoryArrayViews.Empty();

		for (FSharedMemoryExperienceContainer& SharedMemoryExperienceContainer : SharedMemoryExperienceContainers)
		{
			SharedMemoryExperienceContainer.Deallocate();
		}
		SharedMemoryExperienceContainers.Empty();
	}

	FSocketTrainerServerProcess::FSocketTrainerServerProcess(
		const FString& TaskName,
		const FString& CustomTrainerPath,
		const FString& TrainerFileName,
		const FString& PythonExecutablePath,
		const FString& PythonContentPath,
		const FString& IntermediatePath,
		const FString& NneCpuContentPath,
		const TCHAR* IpAddress,
		const uint32 Port,
		const float InTimeout,
		const ESubprocessFlags TrainingProcessFlags,
		const ELogSetting LogSettings)
	{
		Timeout = InTimeout;

		check(FPaths::FileExists(PythonExecutablePath));
		check(FPaths::DirectoryExists(PythonContentPath));

		IFileManager& FileManager = IFileManager::Get();

		const FString CommandLineArguments = FString::Printf(TEXT("\"%s\" \"%s\" -p \"%s\" -m \"%s\" \"%s\" --nne-cpu-path \"%s\" Socket \"%s:%i\" \"%s\""),
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*(PythonContentPath / TEXT("train.py"))),
			*TaskName,
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*CustomTrainerPath),
			*TrainerFileName,
			LogSettings == ELogSetting::Normal ? TEXT("-l") : TEXT(""),
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*NneCpuContentPath),
			IpAddress,
			Port,
			*FileManager.ConvertToAbsolutePathForExternalAppForRead(*IntermediatePath));

		bool bLaunched = TrainingProcess.Launch(
			FileManager.ConvertToAbsolutePathForExternalAppForRead(*PythonExecutablePath),
			CommandLineArguments, 
			TrainingProcessFlags);

		ensure(bLaunched);

		if (PLATFORM_MAC)
		{
			// TODO we seem to have to sleep on Mac so the trainer can start listening before we try to connect
			FPlatformProcess::Sleep(1.0f);
		}
	}

	bool FSocketTrainerServerProcess::IsRunning() const
	{
		return TrainingProcess.IsRunning();
	}

	bool FSocketTrainerServerProcess::Wait()
	{
		const float SleepTime = 0.001f;
		float WaitTime = 0.0f;

		while (TrainingProcess.Update())
		{
			FPlatformProcess::Sleep(SleepTime);
			WaitTime += SleepTime;

			if (WaitTime > Timeout)
			{
				return false;
			}
		}

		return true;
	}

	void FSocketTrainerServerProcess::Terminate()
	{
		TrainingProcess.Terminate();
	}

	FSubprocess* FSocketTrainerServerProcess::GetTrainingSubprocess()
	{
		return &TrainingProcess;
	}

	FSocketTrainer::FSocketTrainer(
		ETrainerResponse& OutResponse,
		const TSharedPtr<FSocketTrainerServerProcess>& ExternalTrainerProcess,
		const bool bUseExternalTrainerProcess,
		const TCHAR* IpAddressOrHostname,
		const uint32 Port,
		const float InTimeout,
		const bool IsHostname)
	{
		Timeout = InTimeout;
		bUseExternalTrainer = bUseExternalTrainerProcess;

		if (ExternalTrainerProcess)
		{
			TrainingProcess = ExternalTrainerProcess;
		}

		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		checkf(SocketSubsystem, TEXT("Could not get socket subsystem"));

		TSharedRef<FInternetAddr> Address = SocketSubsystem->CreateInternetAddr();
		Address->SetPort(Port);

		if (IsHostname)
		{
			FResolveInfo* ResolveInfo = SocketSubsystem->GetHostByName(TCHAR_TO_ANSI(IpAddressOrHostname));
			
			const float SleepTime = 0.001f;
			float WaitTime = 0.0f;
			while (!ResolveInfo->IsComplete())
			{
				FPlatformProcess::Sleep(SleepTime);
				WaitTime += SleepTime;

				if (WaitTime > Timeout)
				{
					UE_LOGF(LogLearning, Warning, "Timed-out waiting for DNS...");
					break;
				}
			}

			if (ResolveInfo->IsComplete() && ResolveInfo->GetErrorCode() == 0)
			{
				uint32 Ip;
				const FInternetAddr& ResolvedAddress = ResolveInfo->GetResolvedAddress();
				ResolvedAddress.GetIp(Ip);
				Address->SetIp(Ip);
			}
			else
			{
				UE_LOGF(LogLearning, Error, "Unable to resolve hostname %ls. Error code %d...", IpAddressOrHostname, ResolveInfo->GetErrorCode());
				OutResponse = ETrainerResponse::Unexpected;
				return;
			}
		}
		else
		{
			bool bIsValid = false;
			Address->SetIp(IpAddressOrHostname, bIsValid);

			if (!bIsValid)
			{
				UE_LOGF(LogLearning, Error, "Invalid Ip Address \"%ls\"...", IpAddressOrHostname);
				OutResponse = ETrainerResponse::Unexpected;
				return;
			}
		}

		Socket = FTcpSocketBuilder(TEXT("LearningTrainerSocket")).AsBlocking().Build();
		OutResponse = SocketTraining::WaitForConnection(
			*Socket,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			*Address,
			Timeout);
	}

	FSocketTrainer::~FSocketTrainer()
	{
		Terminate();
	}

	bool FSocketTrainer::IsValid()
	{
		return bUseExternalTrainer || TrainingProcess != nullptr;
	}

	ETrainerResponse FSocketTrainer::Wait()
	{
		return ETrainerResponse::Success;
	}

	bool FSocketTrainer::HasNetworkOrCompleted()
	{
		return SocketTraining::HasNetworkOrCompleted(*Socket, TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr);
	}

	void FSocketTrainer::Terminate()
	{
		if (Socket)
		{
			Socket->Close();
			Socket = nullptr;
		}
	}

	ETrainerResponse FSocketTrainer::SendStop()
	{
		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return ETrainerResponse::Unexpected;
		}

		return SocketTraining::SendStop(*Socket, TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr, Timeout);
	}

	ETrainerResponse FSocketTrainer::SendConfigs(
		const TSharedRef<FJsonObject>& DataConfigObject,
		const TSharedRef<FJsonObject>& ModelConfigObject,
		const TSharedRef<FJsonObject>& TrainerConfigObject,
		const ELogSetting LogSettings)
	{

		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return ETrainerResponse::Unexpected;
		}

		// Merge model config into data config for socket protocol compatibility
		for (const auto& Pair : ModelConfigObject->Values)
		{
			DataConfigObject->SetField(Pair.Key, Pair.Value);
		}

		DataConfigObject->SetObjectField(TEXT("TrainerSettings"), TrainerConfigObject);

		FString ConfigString;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ConfigString, 0);
		FJsonSerializer::Serialize(DataConfigObject, JsonWriter, true);

		return SocketTraining::SendConfig(
			*Socket,
			ConfigString,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			Timeout,
			LogSettings);
	}

	int32 FSocketTrainer::AddNetwork(const ULearningNeuralNetworkData& Network)
	{
		const int32 NetworkId = NetworkBuffers.Num();
		NetworkBuffers.Add(TLearningArray<1, uint8>());
		NetworkBuffers[NetworkId].SetNumUninitialized({Network.GetSnapshotByteNum()});
		return NetworkId;
	}
	
	TArray<ETrainerResponse> FSocketTrainer::ReceiveNetworks(
		const TArray<int32>& NetworkIds,
		TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
		TArray<FRWLock*> NetworkLocks,
		const ELogSetting LogSettings)
	{
		check(NetworkIds.Num() == Networks.Num());

		TArray<ETrainerResponse> Responses;
		Responses.Init(ETrainerResponse::Unexpected, NetworkIds.Num());

		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return Responses;
		}

		for (int32 i = 0; i < NetworkIds.Num(); i++)
		{
			if (!ensureMsgf(NetworkBuffers.Num() >= NetworkIds[i], TEXT("Network %d has not been added. Call AddNetwork prior to ReceiveNetwork."), NetworkIds[i]))
			{
				Responses[i] = ETrainerResponse::Unexpected;
				continue;
			}
			
			Responses[i] = SocketTraining::RecvNetwork(
				*Socket,
				NetworkIds[i],
				NetworksVersion,
				*Networks[i],
				TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
				NetworkBuffers[NetworkIds[i]],
				Timeout,
				NetworkLocks.IsValidIndex(i) ? NetworkLocks[i] : nullptr,
				LogSettings);
			}

		if (!Responses.Contains(ETrainerResponse::Completed))
		{
			bool bAllSuccess = Algo::AllOf(Responses, [](const ETrainerResponse& Response) { 
				return Response == ETrainerResponse::Success; 
			});
			if (bAllSuccess)
			{
				return ReceiveQueuedNetworks(NetworkIds, Networks, NetworkLocks, LogSettings);
			}
		}
		return Responses;
	}

	TArray<ETrainerResponse> FSocketTrainer::ReceiveQueuedNetworks(
		const TArray<int32>& NetworkIds,
		TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
		TArray<FRWLock*> NetworkLocks,
		const ELogSetting LogSettings)
	{
		check(NetworkIds.Num() == Networks.Num());

		TArray<ETrainerResponse> Responses;
		Responses.Init(ETrainerResponse::Unexpected, NetworkIds.Num());

		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return Responses;
		}
		
		UE_LOGF(LogLearning, Display, "Checking if there is any newer networks... Current version: %d", NetworksVersion);

		for (int32 i = 0; i < NetworkIds.Num(); i++)
		{
			if (!ensureMsgf(NetworkBuffers.Num() >= NetworkIds[i], TEXT("Network %d has not been added. Call AddNetwork prior to ReceiveNetwork."), NetworkIds[i]))
			{
				Responses[i] = ETrainerResponse::Unexpected;
				continue;
			}
						
			// Signal timeout is 0.0f to avoid blocking when checking queued networks
			const float SignalTimeout = 0.0f;
			ETrainerResponse Response = SocketTraining::RecvNetwork(
				*Socket,
				NetworkIds[i],
				NetworksVersion,
				*Networks[i],
				TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
				NetworkBuffers[NetworkIds[i]],
				Timeout,
				NetworkLocks.IsValidIndex(i) ? NetworkLocks[i] : nullptr,
				LogSettings,
				SignalTimeout);
			
			// If it's the first network and we received a NetworkSignalTimeout, networks queue is consumed and can end recursion
			if (i == 0 && Response == ETrainerResponse::NetworkSignalTimeout)
			{
				UE_LOGF(LogLearning, Display, "Using the most updated networks version: %d", NetworksVersion);
				Responses.Init(ETrainerResponse::Success, NetworkIds.Num());
				return Responses;
			}

			Responses[i] = Response;
		}

		if (!Responses.Contains(ETrainerResponse::Completed))
		{
			bool bAllSuccess = Algo::AllOf(Responses, [](const ETrainerResponse& Response) { 
				return Response == ETrainerResponse::Success; 
			});

			// Recursively check if there's newer networks in the socket
			if (bAllSuccess)
			{
				return ReceiveQueuedNetworks(NetworkIds, Networks, NetworkLocks, LogSettings);
			}
		}
		return Responses;
	}
	
	ETrainerResponse FSocketTrainer::ReceiveNetwork(
		const int32 NetworkId,
		ULearningNeuralNetworkData& OutNetwork,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return ETrainerResponse::Unexpected;
		}

		if (!ensureMsgf(NetworkBuffers.Num() >= NetworkId, TEXT("Network %d has not been added. Call AddNetwork prior to ReceiveNetwork."), NetworkId))
		{
			return ETrainerResponse::Unexpected;
		}

		return SocketTraining::RecvNetwork(
			*Socket,
			NetworkId,
			NetworksVersion,
			OutNetwork,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			NetworkBuffers[NetworkId],
			Timeout,
			NetworkLock,
			LogSettings);
	}

	ETrainerResponse FSocketTrainer::SendNetwork(
		const int32 NetworkId,
		const ULearningNeuralNetworkData& Network,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return ETrainerResponse::Unexpected;
		}

		if (!ensureMsgf(NetworkBuffers.Num() >= NetworkId, TEXT("Network %d has not been added. Call AddNetwork prior to SendNetwork."), NetworkId))
		{
			return ETrainerResponse::Unexpected;
		}

		return SocketTraining::SendNetwork(
			*Socket,
			NetworkBuffers[NetworkId],
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			NetworkId,
			Network,
			Timeout,
			NetworkLock,
			LogSettings);
	}

	int32 FSocketTrainer::AddReplayBuffer(const FReplayBuffer& ReplayBuffer)
	{
		LastReplayBufferId++;
		return LastReplayBufferId;
	}

	ETrainerResponse FSocketTrainer::SendReplayBuffer(const int32 ReplayBufferId, const FReplayBuffer& ReplayBuffer, const ELogSetting LogSettings)
	{
		if (!Socket)
		{
			UE_LOGF(LogLearning, Error, "Training socket is nullptr");
			return ETrainerResponse::Unexpected;
		}

		if (!ensureMsgf(ReplayBufferId <= LastReplayBufferId, TEXT("ReplayBuffer %d has not been added. Call AddReplayBuffer prior to SendReplayBuffer."), ReplayBufferId))
		{
			return ETrainerResponse::Unexpected;
		}

		return SocketTraining::SendExperience(
			*Socket,
			NetworksVersion,
			ReplayBufferId,
			ReplayBuffer,
			TrainingProcess.IsValid() ? TrainingProcess->GetTrainingSubprocess() : nullptr,
			Timeout,
			LogSettings);
	}

	FFileTrainer::FFileTrainer(const FString& InIntermediatePath, ETrainerExportFlags InExportFlags)
	{
		IntermediatePath = InIntermediatePath;
		FileSaveDirectory = IntermediatePath / TEXT("TrainingMaterials") / FDateTime::Now().ToString(TEXT("%Y%m%d%H%M%S"));
		ExportFlags = InExportFlags;
	}

	FFileTrainer::~FFileTrainer(){}

	ETrainerResponse FFileTrainer::SendConfigs(
		const TSharedRef<FJsonObject>& DataConfigObject,
		const TSharedRef<FJsonObject>& ModelConfigObject,
		const TSharedRef<FJsonObject>& TrainerConfigObject,
		const ELogSetting LogSettings)
	{
		IFileManager& FileManager = IFileManager::Get();

		// Add intermediate path as a temp directory for the artifacts logging on python side
		TrainerConfigObject->SetStringField(TEXT("TempDirectory"), *FileManager.ConvertToAbsolutePathForExternalAppForRead(*IntermediatePath));

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::ReplayBuffers))
		{
			const FString ReplayBuffersDir = FileSaveDirectory / TEXT("ReplayBuffers");
			PlatformFile.CreateDirectoryTree(*ReplayBuffersDir);
			FString DataConfigString;
			TSharedRef<TJsonWriter<>> DataJsonWriter = TJsonWriterFactory<>::Create(&DataConfigString, 0);
			FJsonSerializer::Serialize(DataConfigObject, DataJsonWriter, true);
			FFileHelper::SaveStringToFile(DataConfigString, *(ReplayBuffersDir / "data-config.json"));
		}

		if (EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::Networks))
		{
			const FString NetworksDir = FileSaveDirectory / TEXT("Networks");
			PlatformFile.CreateDirectoryTree(*NetworksDir);
			FString ModelConfigString;
			TSharedRef<TJsonWriter<>> ModelJsonWriter = TJsonWriterFactory<>::Create(&ModelConfigString, 0);
			FJsonSerializer::Serialize(ModelConfigObject, ModelJsonWriter, true);
			FFileHelper::SaveStringToFile(ModelConfigString, *(NetworksDir / "model-config.json"));
		}

		if (EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::TrainerConfig))
		{
			FString TrainerConfigString;
			TSharedRef<TJsonWriter<>> TrainerJsonWriter = TJsonWriterFactory<>::Create(&TrainerConfigString, 0);
			FJsonSerializer::Serialize(TrainerConfigObject, TrainerJsonWriter, true);
			FFileHelper::SaveStringToFile(TrainerConfigString, *(FileSaveDirectory / "trainer-config.json"));
		}

		UE_LOGF(LogLearning, Display, "FileTrainer: wrote config files to %ls.", *IntermediatePath);

		return ETrainerResponse::Success;
	}

	int32 FFileTrainer::AddNetwork(const ULearningNeuralNetworkData& Network)
	{
		const int32 Id = NetworksCount;
		NetworksCount++;
		return Id;
	}

	ETrainerResponse FFileTrainer::SendNetwork(
		const int32 NetworkId,
		const ULearningNeuralNetworkData& Network,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		if (!EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::Networks))
		{
			return ETrainerResponse::Success;
		}

		if (!ensureMsgf(NetworksCount > NetworkId, TEXT("Network %d has not been added. Call AddNetwork prior to SendNetwork."), NetworkId))
		{
			return ETrainerResponse::Unexpected;
		}

		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		const FString NetworksDir = FileSaveDirectory / TEXT("Networks");
		FileManager.CreateDirectoryTree(*NetworksDir);

		const int32 ByteNum = Network.GetSnapshotByteNum();
		TLearningArray<1, uint8> Snapshot;
		Snapshot.SetNumUninitialized({ ByteNum });

		FScopeNullableReadLock ScopeLock(NetworkLock);
		Network.SaveToSnapshot(MakeArrayView(Snapshot.GetData(), Snapshot.Num()));

		const FString Base = FString::Printf(TEXT("network_%d"), NetworkId);
		const FString BinPath = NetworksDir / (Base + TEXT(".bin"));
		const FString TmpPath = BinPath + TEXT(".tmp");

		TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*TmpPath));
		if (!FileWriter) 
		{
			return ETrainerResponse::Unexpected; 
		}

		FileWriter->Serialize(Snapshot.GetData(), Snapshot.Num());
		FileWriter->Close();

		if (!FileManager.MoveFile(*BinPath, *TmpPath))
		{
			FileManager.DeleteFile(*TmpPath);
			return ETrainerResponse::Unexpected;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "FileTrainer: saved network %d snapshot to %ls", NetworkId, *BinPath);
		}

		return ETrainerResponse::Success;
	}

	int32 FFileTrainer::AddReplayBuffer(const FReplayBuffer& ReplayBuffer)
	{
		const int32 Id = ReplayBufferCount++;
		return Id;
	}

	ETrainerResponse FFileTrainer::SendReplayBuffer(const int32 ReplayBufferId, const FReplayBuffer& ReplayBuffer, const ELogSetting LogSettings)
	{
		if (!EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::ReplayBuffers))
		{
			return ETrainerResponse::Success;
		}

		if (ReplayBufferId < 0)
		{
			return ETrainerResponse::Unexpected;
		}

		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		FileManager.CreateDirectoryTree(*FileSaveDirectory);

		const FString BinPath = FileSaveDirectory / TEXT("ReplayBuffers") / FString::Printf(TEXT("replaybuffer_%d.bin"), ReplayBufferId);
		const FString TempPath = BinPath + TEXT(".tmp");

		TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*TempPath));
		if (!FileWriter)
		{
			return ETrainerResponse::Unexpected;
		}

		FileWriter->Serialize((void*)&ReplayBufferId, sizeof(int32));

		const int32 EpisodeNum = ReplayBuffer.GetEpisodeNum();
		FileWriter->Serialize((void*)&EpisodeNum, sizeof(int32));

		const int32 StepNum = ReplayBuffer.GetStepNum();
		FileWriter->Serialize((void*)&StepNum, sizeof(int32));

		FileWriter->Serialize((void*)ReplayBuffer.GetEpisodeStarts().GetData(), sizeof(int32) * ReplayBuffer.GetEpisodeStarts().Num());
		FileWriter->Serialize((void*)ReplayBuffer.GetEpisodeLengths().GetData(), sizeof(int32) * ReplayBuffer.GetEpisodeLengths().Num());
	
		for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
		{
			FileWriter->Serialize((void*)ReplayBuffer.GetObservations(Index).GetData(), ReplayBuffer.GetObservations(Index).Num() * sizeof(float));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionsNum(); Index++)
		{
			FileWriter->Serialize((void*)ReplayBuffer.GetActions(Index).GetData(), ReplayBuffer.GetActions(Index).Num() * sizeof(float));
		}

		FileWriter->Close();

		if (!FileManager.MoveFile(*BinPath, *TempPath))
		{
			FileManager.DeleteFile(*TempPath);
			return ETrainerResponse::Unexpected;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "FileTrainer: wrote replay buffer (%d eps, %d steps) to %ls", ReplayBuffer.GetEpisodeNum(), ReplayBuffer.GetStepNum(), *BinPath);
		}

		return ETrainerResponse::Success;
	}
}

