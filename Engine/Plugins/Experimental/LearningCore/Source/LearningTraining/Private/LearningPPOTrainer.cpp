// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningPPOTrainer.h"

#include "LearningArray.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "LearningNeuralNetwork.h"
#include "LearningExperience.h"
#include "LearningProgress.h"
#include "LearningSharedMemory.h"
#include "LearningSharedMemoryTraining.h"
#include "LearningSocketTraining.h"
#include "LearningObservation.h"
#include "LearningAction.h"

#include "Misc/Guid.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"

#include "Dom/JsonObject.h"

#include "Sockets.h"
#include "Common/TcpSocketBuilder.h"
#include "SocketSubsystem.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningPPOTrainer)

ULearningSocketPPOTrainerServerCommandlet::ULearningSocketPPOTrainerServerCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

int32 ULearningSocketPPOTrainerServerCommandlet::Main(const FString& Commandline)
{
	UE_LOGF(LogLearning, Display, "Running PPO Training Server Commandlet...");

#if WITH_EDITOR
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;

	UCommandlet::ParseCommandLine(*Commandline, Tokens, Switches, Params);

	const FString* PythonExecutiblePathParam = Params.Find(TEXT("PythonExecutiblePath"));
	const FString* PythonContentPathParam = Params.Find(TEXT("PythonContentPath"));
	const FString* IntermediatePathParam = Params.Find(TEXT("IntermediatePath"));
	const FString* IpAddressParam = Params.Find(TEXT("IpAddress"));
	const FString* PortParam = Params.Find(TEXT("Port"));
	const FString* LogSettingsParam = Params.Find(TEXT("LogSettings"));

	const FString PythonExecutiblePath = PythonExecutiblePathParam ? *PythonExecutiblePathParam : UE::Learning::Trainer::GetPythonExecutablePath(FPaths::ProjectIntermediateDir());
	const FString PythonContentPath = PythonContentPathParam ? *PythonContentPathParam : UE::Learning::Trainer::GetPythonContentPath(FPaths::EngineDir());
	const FString IntermediatePath = IntermediatePathParam ? *IntermediatePathParam : UE::Learning::Trainer::GetIntermediatePath(FPaths::ProjectIntermediateDir());

	const TCHAR* IpAddress = IpAddressParam ? *(*IpAddressParam) : UE::Learning::Trainer::DefaultIp;
	const uint32 Port = PortParam ? FCString::Atoi(*(*PortParam)) : UE::Learning::Trainer::DefaultPort;
	
	UE::Learning::ELogSetting LogSettings = UE::Learning::ELogSetting::Normal;
	if (LogSettingsParam)
	{
		if (*LogSettingsParam == TEXT("Normal"))
		{
			LogSettings = UE::Learning::ELogSetting::Normal;
		}
		else if (*LogSettingsParam == TEXT("Silent"))
		{
			LogSettings = UE::Learning::ELogSetting::Silent;
		}
		else
		{
			checkNoEntry();
			return 1;
		}
	}
	
	const FString* NneCpuPathParam = Params.Find(TEXT("NneCpuPath"));
	const FString NneCpuContentPath = NneCpuPathParam ? *NneCpuPathParam : UE::Learning::Trainer::GetNneCpuContentPath();

	UE_LOGF(LogLearning, Display, "---  PPO Training Server Arguments ---");
	UE_LOGF(LogLearning, Display, "PythonExecutiblePath: %ls", *PythonExecutiblePath);
	UE_LOGF(LogLearning, Display, "PythonContentPath: %ls", *PythonContentPath);
	UE_LOGF(LogLearning, Display, "IntermediatePath: %ls", *IntermediatePath);
	UE_LOGF(LogLearning, Display, "NneCpuContentPath: %ls", *NneCpuContentPath);
	UE_LOGF(LogLearning, Display, "IpAddress: %ls", IpAddress);
	UE_LOGF(LogLearning, Display, "Port: %i", Port);
	UE_LOGF(LogLearning, Display, "LogSettings: %ls", LogSettings == UE::Learning::ELogSetting::Normal ? TEXT("Normal") : TEXT("Silent"));

	UE::Learning::FSocketTrainerServerProcess ServerProcess(
		TEXT("Training"),
		UE::Learning::Trainer::GetProjectPythonContentPath(),
		TEXT("learning_core.train_ppo"),
		PythonExecutiblePath,
		PythonContentPath,
		IntermediatePath,
		NneCpuContentPath,
		IpAddress,
		Port,
		UE::Learning::Trainer::DefaultTimeout,
		UE::Learning::ESubprocessFlags::None,
		LogSettings);

	while (ServerProcess.IsRunning())
	{
		FPlatformProcess::Sleep(0.01f);
	}

#else
	checkNoEntry();
#endif

	return 0;
}

namespace UE::Learning::PPOTrainer
{
	ETrainerResponse Train(
		IExternalTrainer* ExternalTrainer,
		FReplayBuffer& ReplayBuffer,
		FEpisodeBuffer& EpisodeBuffer,
		FResetInstanceBuffer& ResetBuffer,
		ULearningNeuralNetworkData& PolicyNetwork,
		ULearningNeuralNetworkData& CriticNetwork,
		ULearningNeuralNetworkData& EncoderNetwork,
		ULearningNeuralNetworkData& DecoderNetwork,
		TLearningArrayView<2, float> ObservationVectorBuffer,
		TLearningArrayView<2, float> ActionVectorBuffer,
		TLearningArrayView<2, float> PreEvaluationMemoryStateVectorBuffer,
		TLearningArrayView<2, float> MemoryStateVectorBuffer,
		TLearningArrayView<1, float> RewardBuffer,
		TLearningArrayView<1, ECompletionMode> CompletionBuffer,
		TLearningArrayView<1, ECompletionMode> EpisodeCompletionBuffer,
		TLearningArrayView<1, ECompletionMode> AllCompletionBuffer,
		const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
		const TFunctionRef<void(const FIndexSet Instances)> ObservationFunction,
		const TFunctionRef<void(const FIndexSet Instances)> PolicyFunction,
		const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
		const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
		const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
		const TFunctionRef<void(const FIndexSet Instances)> CompletionFunction,
		const FIndexSet Instances,
		const Observation::FSchema& ObservationSchema,
		const Observation::FSchemaElement& ObservationSchemaElement,
		const Action::FSchema& ActionSchema,
		const Action::FSchemaElement& ActionSchemaElement,
		const FPPOTrainerTrainingSettings& TrainerSettings,
		TAtomic<bool>* bRequestTrainingStopSignal,
		FRWLock* PolicyNetworkLock,
		FRWLock* CriticNetworkLock,
		FRWLock* EncoderNetworkLock,
		FRWLock* DecoderNetworkLock,
		TAtomic<bool>* bPolicyNetworkUpdatedSignal,
		TAtomic<bool>* bCriticNetworkUpdatedSignal,
		TAtomic<bool>* bEncoderNetworkUpdatedSignal,
		TAtomic<bool>* bDecoderNetworkUpdatedSignal,
		const ELogSetting LogSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Learning::PPOTrainer::Train);

		ETrainerResponse Response = ETrainerResponse::Success;

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending initial Policy...");
		}

		const int32 PolicyNetworkId = ExternalTrainer->AddNetwork(PolicyNetwork);
		const int32 CriticNetworkId = ExternalTrainer->AddNetwork(CriticNetwork);
		const int32 EncoderNetworkId = ExternalTrainer->AddNetwork(EncoderNetwork);
		const int32 DecoderNetworkId = ExternalTrainer->AddNetwork(DecoderNetwork);
		const int32 ReplayBufferId = ExternalTrainer->AddReplayBuffer(ReplayBuffer);

		const int32 ObservationVectorDimensionNum = ReplayBuffer.GetObservations(0).Num<1>();
		const int32 ActionVectorDimensionNum = ReplayBuffer.GetActions(0).Num<1>();
		const int32 MemoryStateVectorDimensionNum = ReplayBuffer.GetMemoryStates(0).Num<1>();

		// Write Data Config
		TSharedRef<FJsonObject> DataConfigObject = MakeShared<FJsonObject>();

		const int32 ObservationSchemaId = 0;
		const int32 ActionSchemaId = 0;

		// Add Replay Buffers Config Entries
		TArray<TSharedPtr<FJsonValue>> ReplayBufferObjects;
		TSharedRef<FJsonValueObject> ReplayBufferJsonValue = MakeShared<FJsonValueObject>(ReplayBuffer.AsJsonConfig(ReplayBufferId));
		ReplayBufferObjects.Add(ReplayBufferJsonValue);
		DataConfigObject->SetArrayField(TEXT("ReplayBuffers"), ReplayBufferObjects);

		// Schemas
		TSharedPtr<FJsonObject> SchemasObject = MakeShared<FJsonObject>();

		// Add the observation schemas
		TArray<TSharedPtr<FJsonValue>> ObservationSchemaObjects;
		{
			// For this PPO trainer, add the one observation schema we have
			TSharedPtr<FJsonObject> ObservationSchemaObject = MakeShared<FJsonObject>();
			ObservationSchemaObject->SetNumberField(TEXT("Id"), ObservationSchemaId);
			ObservationSchemaObject->SetStringField(TEXT("Name"), "Default");
			ObservationSchemaObject->SetObjectField(TEXT("Schema"),
				UE::Learning::Trainer::ConvertObservationSchemaToJSON(ObservationSchema, ObservationSchemaElement));

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ObservationSchemaObject);
			ObservationSchemaObjects.Add(JsonValue);
		}
		SchemasObject->SetArrayField(TEXT("Observations"), ObservationSchemaObjects);

		// Add the action schemas
		TArray<TSharedPtr<FJsonValue>> ActionSchemaObjects;
		{
			// For this PPO trainer, add the one action schema we have
			TSharedPtr<FJsonObject> ActionSchemaObject = MakeShared<FJsonObject>();
			ActionSchemaObject->SetNumberField(TEXT("Id"), ActionSchemaId);
			ActionSchemaObject->SetStringField(TEXT("Name"), "Default");
			ActionSchemaObject->SetObjectField(TEXT("Schema"),
				UE::Learning::Trainer::ConvertActionSchemaToJSON(ActionSchema, ActionSchemaElement));

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ActionSchemaObject);
			ActionSchemaObjects.Add(JsonValue);
		}
		SchemasObject->SetArrayField(TEXT("Actions"), ActionSchemaObjects);

		DataConfigObject->SetObjectField(TEXT("Schemas"), SchemasObject);

		// Write Model Config
		TSharedRef<FJsonObject> ModelConfigObject = MakeShared<FJsonObject>();

		TArray<TSharedPtr<FJsonValue>> NetworkObjects;

		// Policy
		{
			TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
			NetworkObject->SetNumberField(TEXT("Id"), PolicyNetworkId);
			NetworkObject->SetStringField(TEXT("Name"), "Policy");
			NetworkObject->SetNumberField(TEXT("MaxByteNum"), PolicyNetwork.GetSnapshotByteNum());

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
			NetworkObjects.Add(JsonValue);
		}

		// Critic
		{
			TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
			NetworkObject->SetNumberField(TEXT("Id"), CriticNetworkId);
			NetworkObject->SetStringField(TEXT("Name"), "Critic");
			NetworkObject->SetNumberField(TEXT("MaxByteNum"), CriticNetwork.GetSnapshotByteNum());
			NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
			NetworkObjects.Add(JsonValue);
		}

		// Encoder
		{
			TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
			NetworkObject->SetNumberField(TEXT("Id"), EncoderNetworkId);
			NetworkObject->SetStringField(TEXT("Name"), "Encoder");
			NetworkObject->SetNumberField(TEXT("MaxByteNum"), EncoderNetwork.GetSnapshotByteNum());
			NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
			NetworkObjects.Add(JsonValue);
		}

		// Decoder
		{
			TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
			NetworkObject->SetNumberField(TEXT("Id"), DecoderNetworkId);
			NetworkObject->SetStringField(TEXT("Name"), "Decoder");
			NetworkObject->SetNumberField(TEXT("MaxByteNum"), DecoderNetwork.GetSnapshotByteNum());
			NetworkObject->SetNumberField(TEXT("OutputSchemaId"), ActionSchemaId);

			TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
			NetworkObjects.Add(JsonValue);
		}

		ModelConfigObject->SetArrayField(TEXT("Networks"), NetworkObjects);

		// Add PPO Specific Config Entries
		TSharedRef<FJsonObject> TrainingConfigObject = MakeShared<FJsonObject>();

		TrainingConfigObject->SetStringField(TEXT("TaskName"), TEXT("Training"));
		TrainingConfigObject->SetStringField(TEXT("TrainerMethod"), TEXT("PPO"));
		TrainingConfigObject->SetStringField(TEXT("TimeStamp"), *FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S")));

		TrainingConfigObject->SetNumberField(TEXT("IterationNum"), TrainerSettings.IterationNum);
		TrainingConfigObject->SetNumberField(TEXT("LearningRatePolicy"), TrainerSettings.LearningRatePolicy);
		TrainingConfigObject->SetNumberField(TEXT("LearningRateCritic"), TrainerSettings.LearningRateCritic);
		TrainingConfigObject->SetNumberField(TEXT("LearningRateDecay"), TrainerSettings.LearningRateDecay);
		TrainingConfigObject->SetNumberField(TEXT("WeightDecay"), TrainerSettings.WeightDecay);
		TrainingConfigObject->SetNumberField(TEXT("PolicyBatchSize"), TrainerSettings.PolicyBatchSize);
		TrainingConfigObject->SetNumberField(TEXT("CriticBatchSize"), TrainerSettings.CriticBatchSize);
		TrainingConfigObject->SetNumberField(TEXT("PolicyWindow"), TrainerSettings.PolicyWindow);
		TrainingConfigObject->SetNumberField(TEXT("IterationsPerGather"), TrainerSettings.IterationsPerGather);
		TrainingConfigObject->SetNumberField(TEXT("CriticWarmupIterations"), TrainerSettings.CriticWarmupIterations);
		TrainingConfigObject->SetNumberField(TEXT("EpsilonClip"), TrainerSettings.EpsilonClip);
		TrainingConfigObject->SetNumberField(TEXT("ActionSurrogateWeight"), TrainerSettings.ActionSurrogateWeight);
		TrainingConfigObject->SetNumberField(TEXT("ActionRegularizationWeight"), TrainerSettings.ActionRegularizationWeight);
		TrainingConfigObject->SetNumberField(TEXT("ActionEntropyWeight"), TrainerSettings.ActionEntropyWeight);
		TrainingConfigObject->SetNumberField(TEXT("ReturnRegularizationWeight"), TrainerSettings.ReturnRegularizationWeight);
		TrainingConfigObject->SetNumberField(TEXT("GaeLambda"), TrainerSettings.GaeLambda);
		TrainingConfigObject->SetBoolField(TEXT("AdvantageNormalization"), TrainerSettings.bAdvantageNormalization);
		TrainingConfigObject->SetNumberField(TEXT("AdvantageMin"), TrainerSettings.AdvantageMin);
		TrainingConfigObject->SetNumberField(TEXT("AdvantageMax"), TrainerSettings.AdvantageMax);
		TrainingConfigObject->SetBoolField(TEXT("UseGradNormMaxClipping"), TrainerSettings.bUseGradNormMaxClipping);
		TrainingConfigObject->SetNumberField(TEXT("GradNormMax"), TrainerSettings.GradNormMax);
		TrainingConfigObject->SetNumberField(TEXT("TrimEpisodeStartStepNum"), TrainerSettings.TrimEpisodeStartStepNum);
		TrainingConfigObject->SetNumberField(TEXT("TrimEpisodeEndStepNum"), TrainerSettings.TrimEpisodeEndStepNum);
		TrainingConfigObject->SetNumberField(TEXT("Seed"), TrainerSettings.Seed);
		TrainingConfigObject->SetNumberField(TEXT("DiscountFactor"), TrainerSettings.DiscountFactor);
		TrainingConfigObject->SetStringField(TEXT("Device"), UE::Learning::Trainer::GetDeviceString(TrainerSettings.Device));
		TrainingConfigObject->SetBoolField(TEXT("UseTensorBoard"), TrainerSettings.bUseTensorboard);
		TrainingConfigObject->SetBoolField(TEXT("SaveSnapshots"), TrainerSettings.bSaveSnapshots);

		ExternalTrainer->SendConfigs(DataConfigObject, ModelConfigObject, TrainingConfigObject, LogSettings);

		Response = ExternalTrainer->SendNetwork(PolicyNetworkId, PolicyNetwork, PolicyNetworkLock);

		if (Response != ETrainerResponse::Success)
		{
			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Error, "Error sending initial policy to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
			}

			ExternalTrainer->Terminate();
			return Response;
		}

		// Send initial Critic

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending initial Critic...");
		}

		Response = ExternalTrainer->SendNetwork(CriticNetworkId, CriticNetwork, CriticNetworkLock);

		if (Response != ETrainerResponse::Success)
		{
			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Error, "Error sending initial critic to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
			}

			ExternalTrainer->Terminate();
			return Response;
		}

		// Send initial Encoder

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending initial Encoder...");
		}

		Response = ExternalTrainer->SendNetwork(EncoderNetworkId, EncoderNetwork, EncoderNetworkLock);

		if (Response != ETrainerResponse::Success)
		{
			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Error, "Error sending initial encoder to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
			}

			ExternalTrainer->Terminate();
			return Response;
		}

		// Send initial Decoder

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending initial Decoder...");
		}

		Response = ExternalTrainer->SendNetwork(DecoderNetworkId, DecoderNetwork, DecoderNetworkLock);

		if (Response != ETrainerResponse::Success)
		{
			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Error, "Error sending initial decoder to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
			}

			ExternalTrainer->Terminate();
			return Response;
		}

		// Start Training Loop
		while (true)
		{
			if (bRequestTrainingStopSignal && (*bRequestTrainingStopSignal))
			{
				*bRequestTrainingStopSignal = false;

				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Display, "Stopping Training...");
				}

				Response = ExternalTrainer->SendStop();

				if (Response != ETrainerResponse::Success)
				{
					if (LogSettings != ELogSetting::Silent)
					{
						UE_LOGF(LogLearning, Error, "Error sending stop signal to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
					}

					ExternalTrainer->Terminate();
					return Response;
				}

				break;
			}
			else
			{
				Experience::GatherExperienceUntilReplayBufferFull(
					ReplayBuffer,
					EpisodeBuffer,
					ResetBuffer,
					{ ObservationVectorBuffer },
					{ ActionVectorBuffer},
					{ PreEvaluationMemoryStateVectorBuffer },
					{ MemoryStateVectorBuffer },
					{ RewardBuffer },
					CompletionBuffer,
					EpisodeCompletionBuffer,
					AllCompletionBuffer,
					ResetFunction,
					{ ObservationFunction },
					{ PolicyFunction },
					{ ActionFunction },
					{ UpdateFunction },
					{ RewardFunction },
					CompletionFunction,
					Instances);

				Response = ExternalTrainer->SendReplayBuffer(ReplayBufferId, ReplayBuffer);

				if (Response != ETrainerResponse::Success)
				{
					if (LogSettings != ELogSetting::Silent)
					{
						UE_LOGF(LogLearning, Error, "Error sending replay buffer to trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
					}

					ExternalTrainer->Terminate();
					return Response;
				}
			}

			// Update Policy

			Response = ExternalTrainer->ReceiveNetwork(PolicyNetworkId, PolicyNetwork, PolicyNetworkLock);

			if (Response == ETrainerResponse::Completed)
			{
				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Display, "Trainer completed training.");
				}
				break;
			}
			else if (Response != ETrainerResponse::Success)
			{
				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Error, "Error receiving policy from trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
				}
				break;
			}

			if (bPolicyNetworkUpdatedSignal)
			{
				*bPolicyNetworkUpdatedSignal = true;
			}

			// Update Critic

			Response = ExternalTrainer->ReceiveNetwork(CriticNetworkId, CriticNetwork, CriticNetworkLock);

			if (Response != ETrainerResponse::Success)
			{
				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Error, "Error receiving critic from trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
				}
				break;
			}

			if (bCriticNetworkUpdatedSignal)
			{
				*bCriticNetworkUpdatedSignal = true;
			}

			// Update Encoder

			Response = ExternalTrainer->ReceiveNetwork(EncoderNetworkId, EncoderNetwork, EncoderNetworkLock);

			if (Response != ETrainerResponse::Success)
			{
				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Error, "Error receiving encoder from trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
				}
				break;
			}

			if (bEncoderNetworkUpdatedSignal)
			{
				*bEncoderNetworkUpdatedSignal = true;
			}

			// Update Decoder

			Response = ExternalTrainer->ReceiveNetwork(DecoderNetworkId, DecoderNetwork, DecoderNetworkLock);

			if (Response != ETrainerResponse::Success)
			{
				if (LogSettings != ELogSetting::Silent)
				{
					UE_LOGF(LogLearning, Error, "Error receiving decoder from trainer: %ls. Check log for errors.", Trainer::GetResponseString(Response));
				}
				break;
			}

			if (bDecoderNetworkUpdatedSignal)
			{
				*bDecoderNetworkUpdatedSignal = true;
			}
		}

		// Allow some time for trainer to shut down gracefully before we kill it...
			
		Response = ExternalTrainer->Wait();

		if (Response != ETrainerResponse::Success)
		{
			if (LogSettings != ELogSetting::Silent)
			{
				UE_LOGF(LogLearning, Error, "Error waiting for trainer to exit: %ls. Check log for errors.", Trainer::GetResponseString(Response));
			}
		}

		ExternalTrainer->Terminate();

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Training Task Done!");
		}

		return ETrainerResponse::Success;
	}
}
