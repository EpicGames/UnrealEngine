// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningSharedMemoryTraining.h"

#include "LearningProgress.h"
#include "LearningNeuralNetwork.h"
#include "LearningExperience.h"
#include "LearningCompletion.h"

#include "HAL/PlatformProcess.h"

namespace UE::Learning::SharedMemoryTraining
{
	uint8 GetControlNum()
	{
		return (uint8)EControls::ControlNum;
	}

	ETrainerResponse SendStop(TLearningArrayView<1, volatile int32> Controls)
	{
		Controls[(uint8)EControls::StopSignal] = true;
		return ETrainerResponse::Success;
	}

	bool HasNetworkOrCompleted(TLearningArrayView<1, volatile int32> Controls)
	{
		return Controls[(uint8)EControls::NetworkSignal] || Controls[(uint8)EControls::CompleteSignal];
	}

	ETrainerResponse SendConfigSignal(
		TLearningArrayView<1, volatile int32> Controls,
		const ELogSetting LogSettings)
	{
		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending config signal...");
		}

		Controls[(uint8)EControls::ConfigSignal] = true;

		return ETrainerResponse::Success;
	}

	ETrainerResponse RecvNetwork(
		TLearningArrayView<1, volatile int32> Controls,
		const int32 NetworkId,
		ULearningNeuralNetworkData& OutNetwork,
		FSubprocess* Process,
		const TLearningArrayView<1, const uint8> NetworkData,
		const float Timeout,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		const float SleepTime = 0.001f;
		float WaitTime = 0.0f;

		// Wait until the network is done being written by the training process
		while (!Controls[(uint8)EControls::NetworkSignal])
		{
			// Check if Completed Signal has been raised
			if (Controls[(uint8)EControls::CompleteSignal])
			{
				// If so set low to confirm we have read it
				Controls[(uint8)EControls::CompleteSignal] = false;
				return ETrainerResponse::Completed;
			}

			// If we're monitoring a process, then has it has exited?
			if (Process && !Process->Update())
			{
				return ETrainerResponse::Unexpected;
			}

			// Check if we've timed out
			if (WaitTime > Timeout)
			{
				return ETrainerResponse::Timeout;
			}

			// Check if ping has been sent
			if (Controls[(uint8)EControls::PingSignal])
			{
				Controls[(uint8)EControls::PingSignal] = false;
				WaitTime = 0.0f;
			}

			// Sleep for some time
			FPlatformProcess::Sleep(SleepTime);
			WaitTime += SleepTime;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pulling network...");
		}

		// Read the network
		bool bSuccess = false;
		{
			FScopeNullableWriteLock ScopeLock(NetworkLock);

			checkf(Controls[(uint8)EControls::NetworkId] == NetworkId, TEXT("Received unexpected NetworkId!"));

			if (NetworkData.Num() != OutNetwork.GetSnapshotByteNum())
			{
				UE_LOGF(LogLearning, Error, "Error receiving network. Incorrect buffer size. Buffer is %lli bytes, expected %i.", NetworkData.Num(), OutNetwork.GetSnapshotByteNum());
				bSuccess = false;
			}
			else
			{
				if (!OutNetwork.LoadFromSnapshot(MakeArrayView(NetworkData.GetData(), NetworkData.Num())))
				{
					UE_LOGF(LogLearning, Error, "Error receiving network. Invalid Format.");
					bSuccess = false;
				}
				else
				{
					bSuccess = true;
				}
			}
		}

		// Confirm we have read the network
		Controls[(uint8)EControls::NetworkId] = -1;
		Controls[(uint8)EControls::NetworkSignal] = false;

		return bSuccess ? ETrainerResponse::Success : ETrainerResponse::Unexpected;
	}

	ETrainerResponse SendNetwork(
		TLearningArrayView<1, volatile int32> Controls,
		const int32 NetworkId,
		TLearningArrayView<1, uint8> NetworkData,
		FSubprocess* Process,
		const ULearningNeuralNetworkData& Network,
		const float Timeout,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		const float SleepTime = 0.001f;
		float WaitTime = 0.0f;

		// Wait until the policy is requested by the training process
		while (!Controls[(uint8)EControls::NetworkSignal])
		{
			// If we're monitoring a process, then has it has exited?
			if (Process && !Process->Update())
			{
				return ETrainerResponse::Unexpected;
			}

			// Check if we've timed out
			if (WaitTime > Timeout)
			{
				return ETrainerResponse::Timeout;
			}

			// Check if ping has been sent
			if (Controls[(uint8)EControls::PingSignal])
			{
				Controls[(uint8)EControls::PingSignal] = false;
				WaitTime = 0.0f;
			}

			// Sleep for some time
			FPlatformProcess::Sleep(SleepTime);
			WaitTime += SleepTime;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pushing network...");
		}

		// Write the network
		bool bSuccess = false;
		{
			FScopeNullableReadLock ScopeLock(NetworkLock);
			if (NetworkData.Num() != Network.GetSnapshotByteNum())
			{
				UE_LOGF(LogLearning, Error, "Error sending network. Incorrect buffer size. Buffer is %lli bytes, expected %i.", NetworkData.Num(), Network.GetSnapshotByteNum());
				bSuccess = false;
			}
			else
			{
				Network.SaveToSnapshot(MakeArrayView(NetworkData.GetData(), NetworkData.Num()));
				bSuccess = true;
			}
		}

		// Confirm we have written the network
		Controls[(uint8)EControls::NetworkId] = NetworkId;
		Controls[(uint8)EControls::NetworkSignal] = false;

		return bSuccess ? ETrainerResponse::Success : ETrainerResponse::Unexpected;
	}

	ETrainerResponse SendExperience(
		TLearningArrayView<1, int32> EpisodeStarts,
		TLearningArrayView<1, int32> EpisodeLengths,
		TLearningArrayView<1, ECompletionMode> EpisodeCompletionModes,
		TArrayView<TLearningArrayView<2, float>> EpisodeFinalObservations,
		TArrayView<TLearningArrayView<2, float>> EpisodeFinalMemoryStates,
		TArrayView<TLearningArrayView<2, float>> Observations,
		TArrayView<TLearningArrayView<2, float>> Actions,
		TArrayView<TLearningArrayView<2, float>> ActionModifiers,
		TArrayView<TLearningArrayView<2, float>> MemoryStates,
		TArrayView<TLearningArrayView<2, float>> Rewards,
		TLearningArrayView<1, volatile int32> Controls,
		FSubprocess* Process,
		const int32 ReplayBufferId,
		const FReplayBuffer& ReplayBuffer,
		const float Timeout,
		const ELogSetting LogSettings)
	{
		const float SleepTime = 0.001f;
		float WaitTime = 0.0f;

		// Wait until the training process is done reading any experience
		while (Controls[(uint8)EControls::ExperienceSignal])
		{
			// If we're monitoring a process, then has it has exited?
			if (Process && !Process->Update())
			{
				return ETrainerResponse::Unexpected;
			}

			// Check if we've timed out
			if (WaitTime > Timeout)
			{
				return ETrainerResponse::Timeout;
			}

			// Check if ping has been sent
			if (Controls[(uint8)EControls::PingSignal])
			{
				Controls[(uint8)EControls::PingSignal] = false;
				WaitTime = 0.0f;
			}

			// Sleep for some time
			FPlatformProcess::Sleep(SleepTime);
			WaitTime += SleepTime;
		}

		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pushing Experience...");
		}

		const int32 EpisodeNum = ReplayBuffer.GetEpisodeNum();
		const int32 StepNum = ReplayBuffer.GetStepNum();

		// Write experience to the shared memory
		Array::Copy(EpisodeStarts.Slice(0, EpisodeNum), ReplayBuffer.GetEpisodeStarts());
		Array::Copy(EpisodeLengths.Slice(0, EpisodeNum), ReplayBuffer.GetEpisodeLengths());

		if (ReplayBuffer.HasCompletions())
		{
			Array::Copy(EpisodeCompletionModes.Slice(0, EpisodeNum), ReplayBuffer.GetEpisodeCompletionModes());
		}

		if (ReplayBuffer.HasFinalObservations())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
			{
				Array::Copy(EpisodeFinalObservations[Index].Slice(0, EpisodeNum), ReplayBuffer.GetEpisodeFinalObservations(Index));
			}
		}

		if (ReplayBuffer.HasFinalMemoryStates())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
			{
				Array::Copy(EpisodeFinalMemoryStates[Index].Slice(0, EpisodeNum), ReplayBuffer.GetEpisodeFinalMemoryStates(Index));
			}
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
		{
			Array::Copy(Observations[Index].Slice(0, StepNum), ReplayBuffer.GetObservations(Index));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionsNum(); Index++)
		{
			Array::Copy(Actions[Index].Slice(0, StepNum), ReplayBuffer.GetActions(Index));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionModifiersNum(); Index++)
		{
			Array::Copy(ActionModifiers[Index].Slice(0, StepNum), ReplayBuffer.GetActionModifiers(Index));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
		{
			Array::Copy(MemoryStates[Index].Slice(0, StepNum), ReplayBuffer.GetMemoryStates(Index));
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetRewardsNum(); Index++)
		{
			Array::Copy(Rewards[Index].Slice(0, StepNum), ReplayBuffer.GetRewards(Index));
		}

		// Indicate that experience is written
		Controls[(uint8)EControls::ExperienceEpisodeNum] = EpisodeNum;
		Controls[(uint8)EControls::ExperienceStepNum] = StepNum;
		Controls[(uint8)EControls::ReplayBufferId] = ReplayBufferId;
		Controls[(uint8)EControls::ExperienceSignal] = true;

		return ETrainerResponse::Success;
	}
}