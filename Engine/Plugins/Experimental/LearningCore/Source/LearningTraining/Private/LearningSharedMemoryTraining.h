// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningTrainer.h"

class ULearningNeuralNetworkData;

namespace UE::Learning
{
	enum class ECompletionMode : uint8;
	struct FReplayBuffer;

	namespace SharedMemoryTraining
	{
		enum class EControls : uint8
		{
			ExperienceEpisodeNum	= 0,
			ExperienceStepNum		= 1,
			ExperienceSignal		= 2,
			ConfigSignal			= 3,
			NetworkSignal			= 4,
			CompleteSignal			= 5,
			StopSignal				= 6,
			PingSignal				= 7,
			NetworkId				= 8,
			ReplayBufferId			= 9,

			ControlNum				= 10,
		};

		LEARNINGTRAINING_API uint8 GetControlNum();

		LEARNINGTRAINING_API ETrainerResponse SendConfigSignal(
			TLearningArrayView<1, volatile int32> Controls,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse RecvNetwork(
			TLearningArrayView<1, volatile int32> Controls,
			const int32 NetworkId,
			ULearningNeuralNetworkData& OutNetwork,
			FSubprocess* Process,
			const TLearningArrayView<1, const uint8> NetworkData,
			const float Timeout = Trainer::DefaultTimeout,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse SendStop(
			TLearningArrayView<1, volatile int32> Controls);

		LEARNINGTRAINING_API bool HasNetworkOrCompleted(TLearningArrayView<1, volatile int32> Controls);

		LEARNINGTRAINING_API ETrainerResponse SendNetwork(
			TLearningArrayView<1, volatile int32> Controls,
			const int32 NetworkId,
			TLearningArrayView<1, uint8> NetworkData,
			FSubprocess* Process,
			const ULearningNeuralNetworkData& Network,
			const float Timeout = Trainer::DefaultTimeout,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse SendExperience(
			TLearningArrayView<1, int32> EpisodeStarts,
			TLearningArrayView<1, int32> EpisodeLengths,
			TLearningArrayView<1, ECompletionMode> EpisodeCompletionModes,
			TArrayView<TLearningArrayView<2, float>> EpisodeFinalObservations,
			TArrayView<TLearningArrayView<2, float>> EpisodeFinalMemoryStates,
			TArrayView<TLearningArrayView<2, float>> Observations,
			TArrayView<TLearningArrayView<2, float>> Actions,
			TArrayView<TLearningArrayView<2, float>> ActionModifiers,
			TArrayView<TLearningArrayView<2, float>> MemoryStates,
			TArrayView<TLearningArrayView<2, float>> Rewards,
			TLearningArrayView<1, volatile int32> Controls,
			FSubprocess* Process,
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			const float Timeout = Trainer::DefaultTimeout,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);
	}
}
