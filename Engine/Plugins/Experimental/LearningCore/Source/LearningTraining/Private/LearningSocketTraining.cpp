// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningSocketTraining.h"

#include "LearningProgress.h"
#include "LearningNeuralNetwork.h"
#include "LearningExperience.h"

#include "Sockets.h"
#include "HAL/PlatformProcess.h"

namespace UE::Learning::SocketTraining
{
	ETrainerResponse WaitForConnection(FSocket& Socket, FSubprocess* Process, const FInternetAddr& Addr, const float Timeout)
	{
		float WaitTime = 0.0f;
		const float SleepTime = 0.001f; // 1 millisecond

		while (!Process || Process->Update())
		{
			if (Socket.Connect(Addr))
			{
				return ETrainerResponse::Success;
			}
			else
			{
				FPlatformProcess::Sleep(SleepTime);
				WaitTime += SleepTime;

				if (WaitTime > Timeout)
				{
					return  ETrainerResponse::Timeout;
				}
			}
		}

		return ETrainerResponse::Unexpected;
	}

	ETrainerResponse RecvWithTimeout(FSocket& Socket, FSubprocess* Process, uint8* Bytes, const int32 ByteNum, const float Timeout)
	{
		float WaitTime = 0.0f;

		int32 BytesRead = 0;
		int32 TotalBytesRead = 0;

		while (!Process || Process->Update())
		{
			if (Socket.Wait(ESocketWaitConditions::WaitForRead, FTimespan(10000))) // 1 millisecond, as FTimespan are measured in 0.1 microseconds
			{
				if (Socket.Recv(Bytes + TotalBytesRead, ByteNum - TotalBytesRead, BytesRead))
				{
					TotalBytesRead += BytesRead;

					if (TotalBytesRead == ByteNum)
					{
						return ETrainerResponse::Success;
					}
				}
			}

			WaitTime += 0.001f; // 1 millisecond

			if (WaitTime > Timeout)
			{
				return ETrainerResponse::Timeout;
			}
		}

		return ETrainerResponse::Unexpected;
	}

	ETrainerResponse RecvNetwork(
		FSocket& Socket,
		const int32 NetworkId,
		int32& OutNetworkVersion,
		ULearningNeuralNetworkData& OutNetwork,
		FSubprocess* Process,
		TLearningArrayView<1, uint8> OutNetworkBuffer,
		const float Timeout,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings,
		const float SignalTimeout)
	{
		check(OutNetworkBuffer.Num() == OutNetwork.GetSnapshotByteNum());

		uint8 Signal = (uint8)ESignal::Invalid;
		ETrainerResponse Response = ETrainerResponse::Unexpected;

		while (true)
		{
			Response = RecvWithTimeout(Socket, Process, &Signal, 1, SignalTimeout);
			if (Response == ETrainerResponse::Timeout) { return ETrainerResponse::NetworkSignalTimeout; }
			if (Response != ETrainerResponse::Success) { return Response; }

			if (Signal == (uint8)ESignal::RecvComplete)
			{
				return ETrainerResponse::Completed;
			}

			if (Signal == (uint8)ESignal::RecvPing)
			{
				continue;
			}

			if (Signal != (uint8)ESignal::RecvNetwork)
			{
				return ETrainerResponse::Unexpected;
			}

			break;
		}

		int32 Id = -1;
		Response = RecvWithTimeout(Socket, Process, (uint8*)&Id, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		checkf(Id == NetworkId, TEXT("Received unexpected network id!"))

		Response = RecvWithTimeout(Socket, Process, (uint8*)&OutNetworkVersion, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = RecvWithTimeout(Socket, Process, OutNetworkBuffer.GetData(), OutNetworkBuffer.Num(), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		bool bSuccess = false;
		{
			FScopeNullableWriteLock ScopeLock(NetworkLock);

			if (OutNetworkBuffer.Num() != OutNetwork.GetSnapshotByteNum())
			{
				UE_LOGF(LogLearning, Error, "Error receiving network. Incorrect buffer size. Buffer is %lli bytes, expected %i.", OutNetworkBuffer.Num(), OutNetwork.GetSnapshotByteNum());
				bSuccess = false;
			}
			else
			{
				if (!OutNetwork.LoadFromSnapshot(MakeArrayView(OutNetworkBuffer.GetData(), OutNetworkBuffer.Num())))
				{
					UE_LOGF(LogLearning, Error, "Error receiving network. Invalid Format.");
					bSuccess = false;
				}
				else
				{
					bSuccess = true;
				}
			}
		}

		return bSuccess ? ETrainerResponse::Success : ETrainerResponse::Unexpected;
	}

	ETrainerResponse SendWithTimeout(FSocket& Socket, FSubprocess* Process, const uint8* Bytes, const int32 ByteNum, const float Timeout)
	{
		float WaitTime = 0.0f;

		int32 BytesSent = 0;
		int32 TotalBytesSent = 0;

		while (!Process || Process->Update())
		{
			if (Socket.Wait(ESocketWaitConditions::WaitForWrite, FTimespan(10000))) // 1 millisecond, as FTimespan are measured in 0.1 microseconds
			{
				if (Socket.Send(Bytes + TotalBytesSent, ByteNum - TotalBytesSent, BytesSent))
				{
					TotalBytesSent += BytesSent;

					if (TotalBytesSent == ByteNum)
					{
						return ETrainerResponse::Success;
					}
				}
			}

			WaitTime += 0.001f; // 1 millisecond

			if (WaitTime > Timeout)
			{
				return ETrainerResponse::Timeout;
			}
		}

		return ETrainerResponse::Unexpected;
	}

	ETrainerResponse SendConfig(
		FSocket& Socket,
		const FString& ConfigString,
		FSubprocess* Process,
		const float Timeout,
		const ELogSetting LogSettings)
	{
		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Sending config...");
		}

		const uint8 Signal = (uint8)ESignal::SendConfig;
		ETrainerResponse Response = SendWithTimeout(Socket, Process, &Signal, 1, Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		const FTCHARToUTF8 UTF8String(*ConfigString);
		const int32 ConfigLength = UTF8String.Length();
		
		Response = SendWithTimeout(Socket, Process, (uint8*)&ConfigLength, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		
		Response = SendWithTimeout(Socket, Process, (uint8*)UTF8String.Get(), UTF8String.Length(), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		return ETrainerResponse::Success;
	}

	ETrainerResponse SendStop(FSocket& Socket, FSubprocess* Process, const float Timeout)
	{
		const uint8 Signal = (uint8)ESignal::SendStop;
		return SendWithTimeout(Socket, Process, &Signal, 1, Timeout);
	}

	bool HasNetworkOrCompleted(FSocket& Socket, FSubprocess* Process)
	{
		// If we're monitoring a process, then has it has exited?
		if (Process && !Process->Update())
		{
			return false;
		}

		uint32 PendingDataSize;
		bool bHasPendingData = Socket.HasPendingData(PendingDataSize);

		uint8 Signal = (uint8)ESignal::Invalid;
		int32 BytesRead = 0;
		if (bHasPendingData)
		{ 
			Socket.Recv(&Signal, 1, BytesRead, ESocketReceiveFlags::Type::Peek);

			if (Signal == (uint8)ESignal::RecvPing)
			{
				// Consume the ping
				Socket.Recv(&Signal, 1, BytesRead);
				return false;
			}
		}

		return Signal == (uint8)ESignal::RecvComplete || Signal == (uint8)ESignal::RecvNetwork;
	}

	ETrainerResponse SendNetwork(
		FSocket& Socket,
		TLearningArrayView<1, uint8> NetworkBuffer,
		FSubprocess* Process,
		const int32 NetworkId,
		const ULearningNeuralNetworkData& Network,
		const float Timeout,
		FRWLock* NetworkLock,
		const ELogSetting LogSettings)
	{
		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pushing Network...");
		}

		bool bSuccess = false;
		{
			FScopeNullableReadLock ScopeLock(NetworkLock);
			if (NetworkBuffer.Num() != Network.GetSnapshotByteNum())
			{
				UE_LOGF(LogLearning, Error, "Error sending network. Incorrect buffer size. Buffer is %lli bytes, expected %i.", NetworkBuffer.Num(), Network.GetSnapshotByteNum());
				bSuccess = false;
			}
			else
			{
				Network.SaveToSnapshot(MakeArrayView(NetworkBuffer.GetData(), NetworkBuffer.Num()));
				bSuccess = true;
			}
		}

		const uint8 Signal = (uint8)ESignal::SendNetwork;
		ETrainerResponse Response = SendWithTimeout(Socket, Process, &Signal, 1, Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)&NetworkId, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, NetworkBuffer.GetData(), NetworkBuffer.Num(), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		return bSuccess ? Response : ETrainerResponse::Unexpected;
	}

	ETrainerResponse SendExperience(
		FSocket& Socket,
		const int32 NetworksVersion,
		const int32 ReplayBufferId,
		const FReplayBuffer& ReplayBuffer,
		FSubprocess* Process,
		const float Timeout,
		const ELogSetting LogSettings)
	{
		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pushing Experience...");
		}

		const uint8 Signal = (uint8)ESignal::SendExperience;
		ETrainerResponse Response = SendWithTimeout(Socket, Process, &Signal, 1, Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)&NetworksVersion, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)&ReplayBufferId, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		const int32 EpisodeNum = ReplayBuffer.GetEpisodeNum();
		Response = SendWithTimeout(Socket, Process, (const uint8*)&EpisodeNum, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		
		const int32 StepNum = ReplayBuffer.GetStepNum();
		Response = SendWithTimeout(Socket, Process, (const uint8*)&StepNum, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		
		Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetEpisodeStarts().GetData(), ReplayBuffer.GetEpisodeStarts().Num() * sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		
		Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetEpisodeLengths().GetData(), ReplayBuffer.GetEpisodeLengths().Num() * sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }
		
		if (ReplayBuffer.HasCompletions())
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetEpisodeCompletionModes().GetData(), ReplayBuffer.GetEpisodeCompletionModes().Num() * sizeof(ECompletionMode), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}
		
		if (ReplayBuffer.HasFinalObservations())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
			{
				Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetEpisodeFinalObservations(Index).GetData(), ReplayBuffer.GetEpisodeFinalObservations(Index).Num() * sizeof(float), Timeout);
				if (Response != ETrainerResponse::Success) { return Response; }
			}
		}
		
		if (ReplayBuffer.HasFinalMemoryStates())
		{
			for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
			{
				Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetEpisodeFinalMemoryStates(Index).GetData(), ReplayBuffer.GetEpisodeFinalMemoryStates(Index).Num() * sizeof(float), Timeout);
				if (Response != ETrainerResponse::Success) { return Response; }
			}
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetObservationsNum(); Index++)
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetObservations(Index).GetData(), ReplayBuffer.GetObservations(Index).Num() * sizeof(float), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}
		
		for (int32 Index = 0; Index < ReplayBuffer.GetActionsNum(); Index++)
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetActions(Index).GetData(), ReplayBuffer.GetActions(Index).Num() * sizeof(float), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetActionModifiersNum(); Index++)
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetActionModifiers(Index).GetData(), ReplayBuffer.GetActionModifiers(Index).Num() * sizeof(float), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}
		
		for (int32 Index = 0; Index < ReplayBuffer.GetMemoryStatesNum(); Index++)
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetMemoryStates(Index).GetData(), ReplayBuffer.GetMemoryStates(Index).Num() * sizeof(float), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}

		for (int32 Index = 0; Index < ReplayBuffer.GetRewardsNum(); Index++)
		{
			Response = SendWithTimeout(Socket, Process, (const uint8*)ReplayBuffer.GetRewards(Index).GetData(), ReplayBuffer.GetRewards(Index).Num() * sizeof(float), Timeout);
			if (Response != ETrainerResponse::Success) { return Response; }
		}

		return ETrainerResponse::Success;
	}

	ETrainerResponse SendExperience(
		FSocket& Socket,
		const TLearningArrayView<1, const int32> EpisodeStartsExperience,
		const TLearningArrayView<1, const int32> EpisodeLengthsExperience,
		const TLearningArrayView<2, const float> ObservationExperience,
		const TLearningArrayView<2, const float> ActionExperience,
		FSubprocess* Process,
		const float Timeout,
		const ELogSetting LogSettings)
	{
		if (LogSettings != ELogSetting::Silent)
		{
			UE_LOGF(LogLearning, Display, "Pushing Experience...");
		}

		const int32 EpisodeNum = EpisodeStartsExperience.Num<0>();
		const int32 StepNum = ObservationExperience.Num<0>();

		const uint8 Signal = (uint8)ESignal::SendExperience;
		ETrainerResponse Response = SendWithTimeout(Socket, Process, &Signal, 1, Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)&EpisodeNum, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)&StepNum, sizeof(int32), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)EpisodeStartsExperience.GetData(), EpisodeStartsExperience.Num() * sizeof(float), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)EpisodeLengthsExperience.GetData(), EpisodeLengthsExperience.Num() * sizeof(float), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)ObservationExperience.GetData(), ObservationExperience.Num() * sizeof(float), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		Response = SendWithTimeout(Socket, Process, (const uint8*)ActionExperience.GetData(), ActionExperience.Num() * sizeof(float), Timeout);
		if (Response != ETrainerResponse::Success) { return Response; }

		return ETrainerResponse::Success;
	}

}