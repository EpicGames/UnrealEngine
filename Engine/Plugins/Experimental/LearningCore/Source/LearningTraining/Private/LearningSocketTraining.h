// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningTrainer.h"

class FInternetAddr;
class FSocket;
class ULearningNeuralNetworkData;

namespace UE::Learning
{
	struct FReplayBuffer;

	namespace SocketTraining
	{
		enum class ESignal : uint8
		{
			Invalid				= 0,
			SendConfig			= 1,
			SendExperience		= 2,
			RecvNetwork			= 3,
			SendNetwork			= 4,
			RecvComplete		= 5,
			SendStop			= 6,
			RecvPing			= 7,
		};

		LEARNINGTRAINING_API ETrainerResponse WaitForConnection(
			FSocket& Socket,
			FSubprocess* Process,
			const FInternetAddr& Addr,
			const float Timeout = Trainer::DefaultTimeout);

		LEARNINGTRAINING_API ETrainerResponse RecvWithTimeout(
			FSocket& Socket,
			FSubprocess* Process,
			uint8* Bytes, 
			const int32 ByteNum, 
			const float Timeout = Trainer::DefaultTimeout);

		LEARNINGTRAINING_API ETrainerResponse RecvNetwork(
			FSocket& Socket,
			const int32 NetworkId,
			int32& OutNetworkVersion,
			ULearningNeuralNetworkData& OutNetwork,
			FSubprocess* Process,
			TLearningArrayView<1, uint8> OutNetworkBuffer,
			const float Timeout = Trainer::DefaultTimeout,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings,
			const float SignalTimeout = Trainer::DefaultTimeout);

		LEARNINGTRAINING_API ETrainerResponse SendWithTimeout(
			FSocket& Socket,
			FSubprocess* Process,
			const uint8* Bytes, 
			const int32 ByteNum, 
			const float Timeout = Trainer::DefaultTimeout);

		LEARNINGTRAINING_API ETrainerResponse SendConfig(
			FSocket& Socket,
			const FString& ConfigString,
			FSubprocess* Process,
			const float Timeout = Trainer::DefaultTimeout,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse SendStop(
			FSocket& Socket,
			FSubprocess* Process,
			const float Timeout = Trainer::DefaultTimeout);

		LEARNINGTRAINING_API bool HasNetworkOrCompleted(
			FSocket& Socket,
			FSubprocess* Process);

		LEARNINGTRAINING_API ETrainerResponse SendNetwork(
			FSocket& Socket,
			TLearningArrayView<1, uint8> NetworkBuffer,
			FSubprocess* Process,
			const int32 NetworkId,
			const ULearningNeuralNetworkData& Network,
			const float Timeout = Trainer::DefaultTimeout,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse SendExperience(
			FSocket& Socket,
			const int32 NetworksVersion,
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			FSubprocess* Process,
			const float Timeout = Trainer::DefaultTimeout,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		LEARNINGTRAINING_API ETrainerResponse SendExperience(
			FSocket& Socket,
			const TLearningArrayView<1, const int32> EpisodeStartsExperience,
			const TLearningArrayView<1, const int32> EpisodeLengthsExperience,
			const TLearningArrayView<2, const float> ObservationExperience,
			const TLearningArrayView<2, const float> ActionExperience,
			FSubprocess* Process,
			const float Timeout = Trainer::DefaultTimeout,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);
	}
}