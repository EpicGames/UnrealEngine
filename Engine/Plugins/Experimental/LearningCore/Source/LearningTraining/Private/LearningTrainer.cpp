// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningTrainer.h"

#include "LearningObservation.h"
#include "LearningAction.h"

#include "Dom/JsonObject.h"
#include "HAL/Platform.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

namespace UE::Learning
{
	FSubprocess::~FSubprocess()
	{
		Terminate();
	}

	bool FSubprocess::Launch(const FString& Path, const FString& Params, const ESubprocessFlags Flags)
	{
		ensureMsgf(!bIsLaunched, TEXT("Subprocess already launched."));

		Terminate();

		const bool bCreatePipes = !(Flags & ESubprocessFlags::NoRedirectOutput);
		const bool bHideWindow = !(Flags & ESubprocessFlags::ShowWindow);

		if (bCreatePipes && !FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
		{
			return false;
		}

		ProcessHandle = FPlatformProcess::CreateProc(*Path, *Params, false, bHideWindow, bHideWindow, nullptr, 0, *FPaths::RootDir(), WritePipe);
		bIsLaunched = true;
		return true;
	}

	bool FSubprocess::IsRunning() const
	{
		return bIsLaunched && FPlatformProcess::IsProcRunning(const_cast<FProcHandle&>(ProcessHandle));
	}

	void FSubprocess::Terminate()
	{
		if (IsRunning())
		{
			UE_LOGF(LogLearning, Display, "Terminating Subprocess...");

			FPlatformProcess::TerminateProc(ProcessHandle, true);
		}

		Update();
	}

	bool FSubprocess::Update()
	{
		// Do nothing if the process is not launched
		if (!bIsLaunched)
		{
			return false;
		}

		// Append the process stdout to the buffer
		OutputBuffer += FPlatformProcess::ReadPipe(ReadPipe);

		// Output all the complete lines
		int32 LineStartIdx = 0;
		for (int32 Idx = 0; Idx < OutputBuffer.Len(); Idx++)
		{
			if (OutputBuffer[Idx] == '\r' || OutputBuffer[Idx] == '\n')
			{
				UE_LOGF(LogLearning, Display, "Subprocess: %ls", *OutputBuffer.Mid(LineStartIdx, Idx - LineStartIdx));

				if (OutputBuffer[Idx] == '\r' && Idx + 1 < OutputBuffer.Len() && OutputBuffer[Idx + 1] == '\n')
				{
					Idx++;
				}

				LineStartIdx = Idx + 1;
			}
		}

		// Remove all the complete lines from the buffer
		OutputBuffer.MidInline(LineStartIdx, MAX_int32, EAllowShrinking::Yes);

		// If the process is no longer running then close the pipes
		if (!IsRunning())
		{
			FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
			ReadPipe = nullptr;
			WritePipe = nullptr;
			bIsLaunched = false;
			return false;
		}
		
		return true;
	}

}

namespace UE::Learning::Trainer
{
	TSharedPtr<FJsonObject> ConvertObservationSchemaToJSON(
		const Observation::FSchema& ObservationSchema,
		const Observation::FSchemaElement& ObservationSchemaElement)
	{
		TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
		Object->SetNumberField(TEXT("VectorSize"), ObservationSchema.GetObservationVectorSize(ObservationSchemaElement));
		Object->SetNumberField(TEXT("EncodedSize"), ObservationSchema.GetEncodedVectorSize(ObservationSchemaElement));

		switch (ObservationSchema.GetType(ObservationSchemaElement))
		{
		case Observation::EType::Null:
		{
			Object->SetStringField(TEXT("Type"), TEXT("Null"));
			break;
		}

		case Observation::EType::Continuous:
		{
			const Observation::FSchemaContinuousParameters Parameters = ObservationSchema.GetContinuous(ObservationSchemaElement);
			Object->SetStringField(TEXT("Type"), TEXT("Continuous"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			Object->SetStringField(TEXT("Normalization"), Observation::GetNormalizationOptionAsString(Parameters.NormalizationOption));
			break;
		}

		case Observation::EType::DiscreteExclusive:
		{
			const Observation::FSchemaDiscreteExclusiveParameters Parameters = ObservationSchema.GetDiscreteExclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("DiscreteExclusive"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			break;
		}

		case Observation::EType::DiscreteInclusive:
		{
			const Observation::FSchemaDiscreteInclusiveParameters Parameters = ObservationSchema.GetDiscreteInclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("DiscreteInclusive"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			break;
		}

		case Observation::EType::NamedDiscreteExclusive:
		{
			const Observation::FSchemaNamedDiscreteExclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteExclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("NamedDiscreteExclusive"));

			TArray<TSharedPtr<FJsonValue>> ElementNames;
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				ElementNames.Add(MakeShared<FJsonValueString>(Parameters.ElementNames[ElementIdx].ToString()));
			}

			Object->SetArrayField(TEXT("ElementNames"), ElementNames);
			break;
		}

		case Observation::EType::NamedDiscreteInclusive:
		{
			const Observation::FSchemaNamedDiscreteInclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteInclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("NamedDiscreteInclusive"));

			TArray<TSharedPtr<FJsonValue>> ElementNames;
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				ElementNames.Add(MakeShared<FJsonValueString>(Parameters.ElementNames[ElementIdx].ToString()));
			}

			Object->SetArrayField(TEXT("ElementNames"), ElementNames);
			break;
		}

		case Observation::EType::And:
		{
			const Observation::FSchemaAndParameters Parameters = ObservationSchema.GetAnd(ObservationSchemaElement);
			
			Object->SetStringField(TEXT("Type"), TEXT("And"));

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Observation::EType::OrExclusive:
		{
			const Observation::FSchemaOrExclusiveParameters Parameters = ObservationSchema.GetOrExclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("OrExclusive"));
			Object->SetNumberField(TEXT("EncodingSize"), Parameters.EncodingSize);

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Observation::EType::OrInclusive:
		{
			const Observation::FSchemaOrInclusiveParameters Parameters = ObservationSchema.GetOrInclusive(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("OrInclusive"));
			Object->SetNumberField(TEXT("AttentionEncodingSize"), Parameters.AttentionEncodingSize);
			Object->SetNumberField(TEXT("AttentionHeadNum"), Parameters.AttentionHeadNum);
			Object->SetNumberField(TEXT("ValueEncodingSize"), Parameters.ValueEncodingSize);

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Observation::EType::Array:
		{
			const Observation::FSchemaArrayParameters Parameters = ObservationSchema.GetArray(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Array"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			Object->SetObjectField(TEXT("Element"), ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Element));
			break;
		}

		case Observation::EType::Set:
		{
			const Observation::FSchemaSetParameters Parameters = ObservationSchema.GetSet(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Set"));
			Object->SetNumberField(TEXT("MaxNum"), Parameters.MaxNum);
			Object->SetNumberField(TEXT("AttentionEncodingSize"), Parameters.AttentionEncodingSize);
			Object->SetNumberField(TEXT("AttentionHeadNum"), Parameters.AttentionHeadNum);
			Object->SetNumberField(TEXT("ValueEncodingSize"), Parameters.ValueEncodingSize);
			Object->SetObjectField(TEXT("Element"), ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Element));
			break;
		}

		case Observation::EType::Encoding:
		{
			const Observation::FSchemaEncodingParameters Parameters = ObservationSchema.GetEncoding(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Encoding"));
			Object->SetNumberField(TEXT("EncodingSize"), Parameters.EncodingSize);
			Object->SetObjectField(TEXT("Element"), ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Element));
			break;
		}

		case Observation::EType::Conv1d:
		{
			const Observation::FSchemaConv1dParameters Parameters = ObservationSchema.GetConv1d(ObservationSchemaElement);
			Object->SetStringField(TEXT("Type"), TEXT("Conv1d"));
			Object->SetNumberField(TEXT("InputLength"), Parameters.InputLength);
			Object->SetNumberField(TEXT("InChannels"), Parameters.InChannels);
			Object->SetNumberField(TEXT("OutChannels"), Parameters.OutChannels);
			Object->SetNumberField(TEXT("KernelSize"), Parameters.KernelSize);
			Object->SetNumberField(TEXT("Padding"), Parameters.Padding);
			Object->SetNumberField(TEXT("PaddingMode"), (uint32)Parameters.PaddingMode);
			Object->SetObjectField(TEXT("Element"), ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Element));
			break;
		}

		case Observation::EType::Conv2d:
		{
			const Observation::FSchemaConv2dParameters Parameters = ObservationSchema.GetConv2d(ObservationSchemaElement);
			Object->SetStringField(TEXT("Type"), TEXT("Conv2d"));
			Object->SetNumberField(TEXT("InputHeight"), Parameters.InputHeight);
			Object->SetNumberField(TEXT("InputWidth"), Parameters.InputWidth);
			Object->SetNumberField(TEXT("InChannels"), Parameters.InChannels);
			Object->SetNumberField(TEXT("OutChannels"), Parameters.OutChannels);
			Object->SetNumberField(TEXT("KernelSize"), Parameters.KernelSize);
			Object->SetNumberField(TEXT("Stride"), Parameters.Stride);
			Object->SetNumberField(TEXT("Padding"), Parameters.Padding);
			Object->SetNumberField(TEXT("PaddingMode"), (uint32)Parameters.PaddingMode);
			Object->SetObjectField(TEXT("Element"), ConvertObservationSchemaToJSON(ObservationSchema, Parameters.Element));
			break;
		}

		case Observation::EType::Sparse:
		{
			const Observation::FSchemaSparseParameters Parameters = ObservationSchema.GetSparse(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Sparse"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			break;
		}

		case Observation::EType::NamedSparse:
		{
			const Observation::FSchemaNamedSparseParameters Parameters = ObservationSchema.GetNamedSparse(ObservationSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("NamedSparse"));

			TArray<TSharedPtr<FJsonValue>> ElementNames;
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				ElementNames.Add(MakeShared<FJsonValueString>(Parameters.ElementNames[ElementIdx].ToString()));
			}

			Object->SetArrayField(TEXT("ElementNames"), ElementNames);
			break;
		}

		default:
			checkNoEntry();
		}

		return Object;
	}

	TSharedPtr<FJsonObject> ConvertActionSchemaToJSON(
		const Action::FSchema& ActionSchema,
		const Action::FSchemaElement& ActionSchemaElement)
	{
		TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
		Object->SetNumberField(TEXT("VectorSize"), ActionSchema.GetActionVectorSize(ActionSchemaElement));
		Object->SetNumberField(TEXT("DistributionSize"), ActionSchema.GetActionDistributionVectorSize(ActionSchemaElement));
		Object->SetNumberField(TEXT("EncodedSize"), ActionSchema.GetEncodedVectorSize(ActionSchemaElement));
		Object->SetNumberField(TEXT("ModifierSize"), ActionSchema.GetActionModifierVectorSize(ActionSchemaElement));

		switch (ActionSchema.GetType(ActionSchemaElement))
		{
		case Action::EType::Null:
		{
			Object->SetStringField(TEXT("Type"), TEXT("Null"));
			break;
		}

		case Action::EType::Continuous:
		{
			const Action::FSchemaContinuousParameters Parameters = ActionSchema.GetContinuous(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Continuous"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			Object->SetStringField(TEXT("Normalization"), Action::GetNormalizationOptionAsString(Parameters.NormalizationOption));

			break;
		}

		case Action::EType::DiscreteExclusive:
		{
			const Action::FSchemaDiscreteExclusiveParameters Parameters = ActionSchema.GetDiscreteExclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("DiscreteExclusive"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			break;
		}

		case Action::EType::DiscreteInclusive:
		{
			const Action::FSchemaDiscreteInclusiveParameters Parameters = ActionSchema.GetDiscreteInclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("DiscreteInclusive"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			break;
		}

		case Action::EType::NamedDiscreteExclusive:
		{
			const Action::FSchemaNamedDiscreteExclusiveParameters Parameters = ActionSchema.GetNamedDiscreteExclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("NamedDiscreteExclusive"));

			TArray<TSharedPtr<FJsonValue>> ElementNames;
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				ElementNames.Add(MakeShared<FJsonValueString>(Parameters.ElementNames[ElementIdx].ToString()));
			}

			Object->SetArrayField(TEXT("ElementNames"), ElementNames);
			break;
		}

		case Action::EType::NamedDiscreteInclusive:
		{
			const Action::FSchemaNamedDiscreteInclusiveParameters Parameters = ActionSchema.GetNamedDiscreteInclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("NamedDiscreteInclusive"));

			TArray<TSharedPtr<FJsonValue>> ElementNames;
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				ElementNames.Add(MakeShared<FJsonValueString>(Parameters.ElementNames[ElementIdx].ToString()));
			}

			Object->SetArrayField(TEXT("ElementNames"), ElementNames);
			break;
		}

		case Action::EType::And:
		{
			const Action::FSchemaAndParameters Parameters = ActionSchema.GetAnd(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("And"));

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertActionSchemaToJSON(ActionSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Action::EType::OrExclusive:
		{
			const Action::FSchemaOrExclusiveParameters Parameters = ActionSchema.GetOrExclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("OrExclusive"));

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertActionSchemaToJSON(ActionSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Action::EType::OrInclusive:
		{
			const Action::FSchemaOrInclusiveParameters Parameters = ActionSchema.GetOrInclusive(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("OrInclusive"));

			TSharedPtr<FJsonObject> SubObject = MakeShared<FJsonObject>();
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = ConvertActionSchemaToJSON(ActionSchema, Parameters.Elements[SubElementIdx]);
				SubElement->SetNumberField(TEXT("Index"), SubElementIdx);
				SubObject->SetObjectField(Parameters.ElementNames[SubElementIdx].ToString(), SubElement);
			}

			Object->SetObjectField(TEXT("Elements"), SubObject);
			break;
		}

		case Action::EType::Array:
		{
			const Action::FSchemaArrayParameters Parameters = ActionSchema.GetArray(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Array"));
			Object->SetNumberField(TEXT("Num"), Parameters.Num);
			Object->SetObjectField(TEXT("Element"), ConvertActionSchemaToJSON(ActionSchema, Parameters.Element));
			break;
		}

		case Action::EType::Encoding:
		{
			const Action::FSchemaEncodingParameters Parameters = ActionSchema.GetEncoding(ActionSchemaElement);

			Object->SetStringField(TEXT("Type"), TEXT("Encoding"));
			Object->SetNumberField(TEXT("EncodingSize"), Parameters.EncodingSize);
			Object->SetNumberField(TEXT("EncodingDepth"), Parameters.LayerNum);
			Object->SetStringField(TEXT("ActivationFunction"), UE::Learning::Action::GetEncodingActivationAsString(Parameters.ActivationFunction));
			Object->SetObjectField(TEXT("Element"), ConvertActionSchemaToJSON(ActionSchema, Parameters.Element));
			break;
		}

		default:
			checkNoEntry();
		}

		return Object;
	}

	bool IsObservationSchemaSubsetCompatible(const FString& SourceJsonString, const Observation::FSchema& ObservationSchema, const Observation::FSchemaElement& ObservationSchemaElement)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(SourceJsonString);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			return IsObservationSchemaSubsetCompatible(JsonObject.ToSharedRef(), ObservationSchema, ObservationSchemaElement);
		}

		return false;
	}

	bool IsObservationSchemaSubsetCompatible(const TSharedRef<FJsonObject> Object, const Observation::FSchema& ObservationSchema, const Observation::FSchemaElement& ObservationSchemaElement)
	{
		if (Object->GetNumberField(TEXT("VectorSize")) < ObservationSchema.GetObservationVectorSize(ObservationSchemaElement) ||
			Object->GetNumberField(TEXT("EncodedSize")) < ObservationSchema.GetEncodedVectorSize(ObservationSchemaElement))
		{
			return false;
		}

		switch (ObservationSchema.GetType(ObservationSchemaElement))
		{
		case Observation::EType::Null:
		{
			return Object->GetStringField(TEXT("Type")) == TEXT("Null");
		}

		case Observation::EType::Continuous:
		{
			const Observation::FSchemaContinuousParameters Parameters = ObservationSchema.GetContinuous(ObservationSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("Continuous") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Observation::EType::DiscreteExclusive:
		{
			const Observation::FSchemaDiscreteExclusiveParameters Parameters = ObservationSchema.GetDiscreteExclusive(ObservationSchemaElement);
			
			return
				Object->GetStringField(TEXT("Type")) == TEXT("DiscreteExclusive") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Observation::EType::DiscreteInclusive:
		{
			const Observation::FSchemaDiscreteInclusiveParameters Parameters = ObservationSchema.GetDiscreteInclusive(ObservationSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("DiscreteInclusive") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Observation::EType::NamedDiscreteExclusive:
		{
			const Observation::FSchemaNamedDiscreteExclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteExclusive(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("NamedDiscreteExclusive"))
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				bool bFound = false;
				for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					return false;
				}
			}

			return true;
		}

		case Observation::EType::NamedDiscreteInclusive:
		{
			const Observation::FSchemaNamedDiscreteInclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteInclusive(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("NamedDiscreteInclusive"))
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				bool bFound = false;
				for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					return false;
				}
			}

			return true;
		}

		case Observation::EType::And:
		{
			const Observation::FSchemaAndParameters Parameters = ObservationSchema.GetAnd(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("And"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsObservationSchemaSubsetCompatible(SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Observation::EType::OrExclusive:
		{
			const Observation::FSchemaOrExclusiveParameters Parameters = ObservationSchema.GetOrExclusive(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("OrExclusive"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsObservationSchemaSubsetCompatible(SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Observation::EType::OrInclusive:
		{
			const Observation::FSchemaOrInclusiveParameters Parameters = ObservationSchema.GetOrInclusive(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("OrInclusive"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsObservationSchemaSubsetCompatible(SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Observation::EType::Array:
		{
			const Observation::FSchemaArrayParameters Parameters = ObservationSchema.GetArray(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Array") ||
				Object->GetNumberField(TEXT("Num")) != Parameters.Num)
			{
				return false;
			}

			return IsObservationSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
		}

		case Observation::EType::Set:
		{
			const Observation::FSchemaSetParameters Parameters = ObservationSchema.GetSet(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Set") ||
				Object->GetNumberField(TEXT("MaxNum")) != Parameters.MaxNum)
			{
				return false;
			}

			return IsObservationSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
		}

		case Observation::EType::Encoding:
		{
			const Observation::FSchemaEncodingParameters Parameters = ObservationSchema.GetEncoding(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Encoding"))
			{
				return false;
			}

			return IsObservationSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
		}

		case Observation::EType::Conv1d:
		{
			const Observation::FSchemaConv1dParameters Parameters = ObservationSchema.GetConv1d(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Conv1d"))
			{
				return false;
			}

			return IsObservationSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
		}

		case Observation::EType::Conv2d:
		{
			const Observation::FSchemaConv2dParameters Parameters = ObservationSchema.GetConv2d(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Conv2d"))
			{
				return false;
			}

			return IsObservationSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
		}

		case Observation::EType::Sparse:
		{
			const Observation::FSchemaSparseParameters Parameters = ObservationSchema.GetSparse(ObservationSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("Sparse") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Observation::EType::NamedSparse:
		{
			const Observation::FSchemaNamedSparseParameters Parameters = ObservationSchema.GetNamedSparse(ObservationSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("NamedSparse"))
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				bool bFound = false;
				for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					return false;
				}
			}

			return true;
		}

		default:
			checkNoEntry();
		}

		return false;
	}

	TArray<int32> ComputeObservationSchemaSubsetIndices(const FString& SourceJsonString, const Observation::FSchema& ObservationSchema, const Observation::FSchemaElement& ObservationSchemaElement)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(SourceJsonString);

		TArray<int32> Output;

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			int32 NextIndex = 0;
			ComputeObservationSchemaSubsetIndices(NextIndex, Output, JsonObject.ToSharedRef(), ObservationSchema, ObservationSchemaElement);
		}
		
		return Output;
	}

	void ComputeObservationSchemaSubsetIndices(
		int32& NextIndex,
		TArray<int32>& Indices,
		const TSharedRef<FJsonObject> Object,
		const Observation::FSchema& ObservationSchema,
		const Observation::FSchemaElement& ObservationSchemaElement)
	{
		if (Object->GetStringField(TEXT("Type")) == TEXT("Null"))
		{
			// Do nothing since Null is zero-width
			return;
		}
		if ((Object->GetStringField(TEXT("Type")) != TEXT("Null")) && ObservationSchema.GetType(ObservationSchemaElement) == Observation::EType::Null)
		{
			// Skipping element: schema has Null but source JSON has a non null type. This is an acceptable method to skip using certain actions for training
			int32 VectorSize = Object->GetNumberField(TEXT("VectorSize"));
			NextIndex += VectorSize;
			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Continuous"))
		{
			const Observation::FSchemaContinuousParameters Parameters = ObservationSchema.GetContinuous(ObservationSchemaElement);

			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("DiscreteExclusive"))
		{
			const Observation::FSchemaDiscreteExclusiveParameters Parameters = ObservationSchema.GetDiscreteExclusive(ObservationSchemaElement);
			
			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("DiscreteInclusive"))
		{
			const Observation::FSchemaDiscreteInclusiveParameters Parameters = ObservationSchema.GetDiscreteInclusive(ObservationSchemaElement);

			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("NamedDiscreteExclusive"))
		{
			const Observation::FSchemaNamedDiscreteExclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteExclusive(ObservationSchemaElement);

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
			{
				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Indices.Add(NextIndex);
				}

				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("NamedDiscreteInclusive"))
		{
			const Observation::FSchemaNamedDiscreteInclusiveParameters Parameters = ObservationSchema.GetNamedDiscreteInclusive(ObservationSchemaElement);

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
			{
				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Indices.Add(NextIndex);
				}

				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("And"))
		{
			const Observation::FSchemaAndParameters Parameters = ObservationSchema.GetAnd(ObservationSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeObservationSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("OrExclusive"))
		{
			const Observation::FSchemaOrExclusiveParameters Parameters = ObservationSchema.GetOrExclusive(ObservationSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeObservationSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("OrInclusive"))
		{
			const Observation::FSchemaOrInclusiveParameters Parameters = ObservationSchema.GetOrInclusive(ObservationSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeObservationSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ObservationSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Array"))
		{
			const Observation::FSchemaArrayParameters Parameters = ObservationSchema.GetArray(ObservationSchemaElement);

			for (int i = 0; i < Parameters.Num; i++)
			{
				ComputeObservationSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Set"))
		{
			const Observation::FSchemaSetParameters Parameters = ObservationSchema.GetSet(ObservationSchemaElement);

			for (int i = 0; i < Parameters.MaxNum; i++)
			{
				ComputeObservationSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Encoding"))
		{
			const Observation::FSchemaEncodingParameters Parameters = ObservationSchema.GetEncoding(ObservationSchemaElement);
			
			ComputeObservationSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Conv1d"))
		{
			const Observation::FSchemaConv1dParameters Parameters = ObservationSchema.GetConv1d(ObservationSchemaElement);

			ComputeObservationSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Conv2d"))
		{
			const Observation::FSchemaConv2dParameters Parameters = ObservationSchema.GetConv2d(ObservationSchemaElement);

			ComputeObservationSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ObservationSchema, Parameters.Element);

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Sparse"))
		{
			const Observation::FSchemaSparseParameters Parameters = ObservationSchema.GetSparse(ObservationSchemaElement);

			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("NamedSparse"))
		{
			const Observation::FSchemaNamedSparseParameters Parameters = ObservationSchema.GetNamedSparse(ObservationSchemaElement);

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
			{
				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Indices.Add(NextIndex);
				}

				NextIndex++;
			}

			return;
			}
		// We didn't return earlier so we received an invalid Type
		checkNoEntry();
	}

	bool IsActionSchemaSubsetCompatible(const FString& SourceJsonString, const Action::FSchema& ActionSchema, const Action::FSchemaElement& ActionSchemaElement)
	{		
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(SourceJsonString);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			return IsActionSchemaSubsetCompatible(JsonObject.ToSharedRef(), ActionSchema, ActionSchemaElement);
		}

		return false;
	}

	bool IsActionSchemaSubsetCompatible(const TSharedRef<FJsonObject> Object, const Action::FSchema& ActionSchema, const Action::FSchemaElement& ActionSchemaElement)
	{
		if (Object->GetNumberField(TEXT("VectorSize")) < ActionSchema.GetActionVectorSize(ActionSchemaElement) ||
			Object->GetNumberField(TEXT("EncodedSize")) < ActionSchema.GetEncodedVectorSize(ActionSchemaElement))
		{
			return false;
		}

		switch (ActionSchema.GetType(ActionSchemaElement))
		{
		case Action::EType::Null:
		{
			return Object->GetStringField(TEXT("Type")) == TEXT("Null");
		}

		case Action::EType::Continuous:
		{
			const Action::FSchemaContinuousParameters Parameters = ActionSchema.GetContinuous(ActionSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("Continuous") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Action::EType::DiscreteExclusive:
		{
			const Action::FSchemaDiscreteExclusiveParameters Parameters = ActionSchema.GetDiscreteExclusive(ActionSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("DiscreteExclusive") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Action::EType::DiscreteInclusive:
		{
			const Action::FSchemaDiscreteInclusiveParameters Parameters = ActionSchema.GetDiscreteInclusive(ActionSchemaElement);

			return
				Object->GetStringField(TEXT("Type")) == TEXT("DiscreteInclusive") &&
				Object->GetNumberField(TEXT("Num")) == Parameters.Num;
		}

		case Action::EType::NamedDiscreteExclusive:
		{
			const Action::FSchemaNamedDiscreteExclusiveParameters Parameters = ActionSchema.GetNamedDiscreteExclusive(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("NamedDiscreteExclusive"))
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				bool bFound = false;
				for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					return false;
				}
			}

			return true;
		}

		case Action::EType::NamedDiscreteInclusive:
		{
			const Action::FSchemaNamedDiscreteInclusiveParameters Parameters = ActionSchema.GetNamedDiscreteInclusive(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("NamedDiscreteInclusive"))
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
			{
				bool bFound = false;
				for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					return false;
				}
			}

			return true;
		}

		case Action::EType::And:
		{
			const Action::FSchemaAndParameters Parameters = ActionSchema.GetAnd(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("And"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsActionSchemaSubsetCompatible(SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Action::EType::OrExclusive:
		{
			const Action::FSchemaOrExclusiveParameters Parameters = ActionSchema.GetOrExclusive(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("OrExclusive"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsActionSchemaSubsetCompatible(SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Action::EType::OrInclusive:
		{
			const Action::FSchemaOrInclusiveParameters Parameters = ActionSchema.GetOrInclusive(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("OrInclusive"))
			{
				return false;
			}

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(Parameters.ElementNames[SubElementIdx].ToString());
				if (SubElement.IsValid())
				{
					bool bResult = IsActionSchemaSubsetCompatible(SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[SubElementIdx]);
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		case Action::EType::Array:
		{
			const Action::FSchemaArrayParameters Parameters = ActionSchema.GetArray(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Array") ||
				Object->GetNumberField(TEXT("Num")) != Parameters.Num)
			{
				return false;
			}

			return IsActionSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ActionSchema, Parameters.Element);
		}

		case Action::EType::Encoding:
		{
			const Action::FSchemaEncodingParameters Parameters = ActionSchema.GetEncoding(ActionSchemaElement);

			if (Object->GetStringField(TEXT("Type")) != TEXT("Encoding") ||
				Object->GetNumberField(TEXT("EncodingSize")) != Parameters.EncodingSize ||
				Object->GetNumberField(TEXT("EncodingDepth")) != Parameters.LayerNum ||
				Object->GetStringField(TEXT("ActivationFunction")) != UE::Learning::Action::GetEncodingActivationAsString(Parameters.ActivationFunction))
			{
				return false;
			}

			return IsActionSchemaSubsetCompatible(Object->GetObjectField(TEXT("Element")).ToSharedRef(), ActionSchema, Parameters.Element);
		}

		default:
			checkNoEntry();
		}

		return false;
	}

	TArray<int32> ComputeActionSchemaSubsetIndices(const FString& SourceJsonString, const Action::FSchema& ActionSchema, const Action::FSchemaElement& ActionSchemaElement)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(SourceJsonString);

		TArray<int32> Output;

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			int32 NextIndex = 0;
			ComputeActionSchemaSubsetIndices(NextIndex, Output, JsonObject.ToSharedRef(), ActionSchema, ActionSchemaElement);
		}

		return Output;
	}

	void ComputeActionSchemaSubsetIndices(int32& NextIndex, TArray<int32>& Indices,	const TSharedRef<FJsonObject> Object, const Action::FSchema& ActionSchema, const Action::FSchemaElement& ActionSchemaElement)
	{

		if (Object->GetStringField(TEXT("Type")) == TEXT("Null"))
		{
			// Do nothing since Null is zero-width
			return;
		}
		if ((Object->GetStringField(TEXT("Type")) != TEXT("Null")) && ActionSchema.GetType(ActionSchemaElement) == Action::EType::Null)
		{
			// Skipping element: schema has Null but source JSON has a non null type. This is an acceptable method to skip using certain actions for training
			int32 VectorSize = Object->GetNumberField(TEXT("VectorSize"));
			NextIndex += VectorSize;
			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Continuous"))
		{
			const Action::FSchemaContinuousParameters Parameters = ActionSchema.GetContinuous(ActionSchemaElement);

			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("DiscreteExclusive"))
		{
			const Action::FSchemaDiscreteExclusiveParameters Parameters = ActionSchema.GetDiscreteExclusive(ActionSchemaElement);

			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("DiscreteInclusive"))
		{
			const Action::FSchemaDiscreteInclusiveParameters Parameters = ActionSchema.GetDiscreteInclusive(ActionSchemaElement);
			
			for (int32 i = 0; i < Parameters.Num; i++)
			{
				Indices.Add(NextIndex);
				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("NamedDiscreteExclusive"))
		{
			const Action::FSchemaNamedDiscreteExclusiveParameters Parameters = ActionSchema.GetNamedDiscreteExclusive(ActionSchemaElement);

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
			{
				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Indices.Add(NextIndex);
				}

				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("NamedDiscreteInclusive"))
		{
			const Action::FSchemaNamedDiscreteInclusiveParameters Parameters = ActionSchema.GetNamedDiscreteInclusive(ActionSchemaElement);

			TArray<TSharedPtr<FJsonValue>> ElementNames = Object->GetArrayField(TEXT("ElementNames"));
			for (const TSharedPtr<FJsonValue>& ElementName : ElementNames)
			{
				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName->Type == EJson::String && ElementName->AsString() == Parameters.ElementNames[ElementIdx].ToString())
					{
						bFound = true;
						break;
					}
				}

				if (bFound)
				{
					Indices.Add(NextIndex);
				}

				NextIndex++;
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("And"))
		{
			const Action::FSchemaAndParameters Parameters = ActionSchema.GetAnd(ActionSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeActionSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("OrExclusive"))
		{
			const Action::FSchemaOrExclusiveParameters Parameters = ActionSchema.GetOrExclusive(ActionSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeActionSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("OrInclusive"))
		{
			const Action::FSchemaOrInclusiveParameters Parameters = ActionSchema.GetOrInclusive(ActionSchemaElement);

			TSharedPtr<FJsonObject> SubObject = Object->GetObjectField(TEXT("Elements"));
			TArray<FJsonObject::FStringType> ElementNames;
			SubObject->Values.GetKeys(ElementNames);

			for (const FJsonObject::FStringType& ElementName : ElementNames)
			{
				TSharedPtr<FJsonObject> SubElement = SubObject->GetObjectField(ElementName);

				bool bFound = false;
				for (int32 ElementIdx = 0; ElementIdx < Parameters.ElementNames.Num(); ElementIdx++)
				{
					if (ElementName == Parameters.ElementNames[ElementIdx])
					{
						bFound = true;
						ComputeActionSchemaSubsetIndices(NextIndex, Indices, SubElement.ToSharedRef(), ActionSchema, Parameters.Elements[ElementIdx]);
						break;
					}
				}

				if (!bFound)
				{
					int32 VectorSize = SubElement->GetNumberField(TEXT("VectorSize"));
					NextIndex += VectorSize;
				}
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Array"))
		{
			const Action::FSchemaArrayParameters Parameters = ActionSchema.GetArray(ActionSchemaElement);

			for (int i = 0; i < Parameters.Num; i++)
			{
				ComputeActionSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ActionSchema, Parameters.Element);
			}

			return;
		}
		else if (Object->GetStringField(TEXT("Type")) == TEXT("Encoding"))
		{
			const Action::FSchemaEncodingParameters Parameters = ActionSchema.GetEncoding(ActionSchemaElement);

			ComputeActionSchemaSubsetIndices(NextIndex, Indices, Object->GetObjectField(TEXT("Element")).ToSharedRef(), ActionSchema, Parameters.Element);
			
			return;
		}

		// We didn't return earlier so we received an invalid Type
		checkNoEntry();
	}

	const TCHAR* GetDeviceString(const ETrainerDevice Device)
	{
		switch (Device)
		{
		case ETrainerDevice::GPU: return TEXT("GPU");
		case ETrainerDevice::CPU: return TEXT("CPU");
		default: checkNoEntry(); return TEXT("Unknown");
		}
	}

	const TCHAR* GetResponseString(const ETrainerResponse Response)
	{
		switch (Response)
		{
		case ETrainerResponse::Success: return TEXT("Success");
		case ETrainerResponse::Unexpected: return TEXT("Unexpected communication received");
		case ETrainerResponse::Completed: return TEXT("Training completed");
		case ETrainerResponse::Stopped: return TEXT("Training stopped");
		case ETrainerResponse::Timeout: return TEXT("Communication timeout");
		default: checkNoEntry(); return TEXT("Unknown");
		}
	}

	float DiscountFactorFromHalfLife(const float HalfLife, const float DeltaTime)
	{
		return FMath::Pow(0.5f, DeltaTime / FMath::Max(HalfLife, UE_SMALL_NUMBER));
	}

	float DiscountFactorFromHalfLifeSteps(const int32 HalfLifeSteps)
	{
		checkf(HalfLifeSteps >= 1, TEXT("Number of HalfLifeSteps should be at least 1 but got %i"), HalfLifeSteps);

		return FMath::Pow(0.5f, 1.0f / FMath::Max(HalfLifeSteps, 1));
	}

	FString GetPythonExecutablePath(const FString& IntermediateDir)
	{
		checkf(PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX, TEXT("Python only supported on Windows, Mac, and Linux."));

		return IntermediateDir / TEXT("PipInstall") / (PLATFORM_WINDOWS ? TEXT("Scripts/python.exe") : TEXT("bin/python3"));
	}

	FString GetSitePackagesPath(const FString& EngineDir)
	{
		checkf(PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX, TEXT("Python only supported on Windows, Mac, and Linux."));

		return EngineDir / TEXT("Plugins/Experimental/PythonFoundationPackages/Content/Python/Lib") / FPlatformMisc::GetUBTPlatform() / TEXT("site-packages");
	}

	FString GetPythonContentPath(const FString& EngineDir)
	{
		return EngineDir / TEXT("Plugins/Experimental/LearningCore/Content/Python/");
	}

	FString GetProjectPythonContentPath()
	{
		return FPaths::ProjectContentDir() / TEXT("Python/");
	}

	FString GetNneCpuContentPath()
	{
		TSharedPtr<IPlugin> NNERuntimeBasicCpuPlugin = IPluginManager::Get().FindEnabledPlugin(TEXT("NNERuntimeBasicCpu"));
		if (NNERuntimeBasicCpuPlugin.IsValid())
		{
			return FPaths::Combine(NNERuntimeBasicCpuPlugin->GetContentDir(), TEXT("Python"));
		}

		UE_LOGF(LogLearning, Error, "GetNneCpuContentPath: NNERuntimeBasicCpu plugin not found");
		return FString();
	}

	FString GetIntermediatePath(const FString& IntermediateDir)
	{
		return IntermediateDir / TEXT("LearningCore");
	}

}