// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningTraining.h"

void FLearningTrainingModule::StartupModule()
{
}

void FLearningTrainingModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FLearningTrainingModule, LearningTraining)
