// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningObservation.h"
#include "LearningAction.h"

#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API LEARNINGTRAINING_API

class FJsonObject;

namespace UE::Learning
{
	enum class ECompletionMode : uint8;
	struct FResetInstanceBuffer;

	/**
	* Buffer storing the observations, actions, and rewards of multiple instances over an episode
	*/
	struct FEpisodeBuffer
	{
		friend struct FReplayBuffer;

		/**
		* Resize the experience buffer
		*
		* @param InMaxInstanceNum				Maximum number of instances
		* @param InMaxStepNum					Maximum number of steps in an episode
		*/
		UE_API void Resize(const int32 InMaxInstanceNum, const int32 InMaxStepNum);

		/**
		* Reset the buffer for the given set of instances
		*/
		UE_API void Reset(const FIndexSet Instances);


		// Observations

		UE_API int32 AddObservations(const FName& Name, const int32 SchemaId, const int32 Size);

		UE_API void PushObservations(const int32 ObservationId, const TLearningArrayView<2, const float> InObservations, const FIndexSet Instances);

		UE_API const TLearningArrayView<2, const float> GetObservations(const int32 ObservationId, const int32 InstanceIdx) const;


		// Actions

		UE_API int32 AddActions(const FName& Name, const int32 SchemaId, const int32 Size);

		UE_API void PushActions(const int32 ActionId, const TLearningArrayView<2, const float> InActions, const FIndexSet Instances);

		UE_API const TLearningArrayView<2, const float> GetActions(const int32 ActionId, const int32 InstanceIdx) const;


		// Action Modifiers

		UE_API int32 AddActionModifiers(const FName& Name, const int32 SchemaId, const int32 Size);

		UE_API void PushActionModifiers(const int32 ActionModifierId, const TLearningArrayView<2, const float> InActionModifiers, const FIndexSet Instances);

		UE_API const TLearningArrayView<2, const float> GetActionModifiers(const int32 ActiActionModifierIdnId, const int32 InstanceIdx) const;


		// Memory States

		UE_API int32 AddMemoryStates(const FName& Name, const int32 Size);

		UE_API void PushMemoryStates(const int32 MemoryStateId, const TLearningArrayView<2, const float> InMemoryStates, const FIndexSet Instances);

		UE_API const TLearningArrayView<2, const float> GetMemoryStates(const int32 MemoryStateId, const int32 InstanceIdx) const;


		// Rewards

		UE_API int32 AddRewards(const FName& Name, const int32 Size);

		// Convenience overload for pushing rewards from a Training Environment
		UE_API void PushRewards(const int32 RewardId, const TLearningArrayView<1, const float> InRewards, const FIndexSet Instances);

		UE_API void PushRewards(const int32 RewardId, const TLearningArrayView<2, const float> InRewards, const FIndexSet Instances);

		UE_API const TLearningArrayView<2, const float> GetRewards(const int32 RewardId, const int32 InstanceIdx) const;


		// Episode Step Nums

		UE_API void IncrementEpisodeStepNums(const FIndexSet Instances);

		UE_API const TLearningArrayView<1, const int32> GetEpisodeStepNums() const;

		UE_API int32 GetMaxInstanceNum() const;
		UE_API int32 GetMaxStepNum() const;

	private:

		bool bHasBeenSized = false;
		int32 MaxInstanceNum = 0;
		int32 MaxStepNum = 0;

		// Observations
		TArray<FName, TInlineAllocator<1>> ObservationNames;
		TArray<int32, TInlineAllocator<1>> ObservationSchemaIds;
		TArray<int32, TInlineAllocator<1>> ObservationSizes;
		TArray<TLearningArray<3, float>, TInlineAllocator<1>> ObservationArrays;

		// Actions
		TArray<FName, TInlineAllocator<1>> ActionNames;
		TArray<int32, TInlineAllocator<1>> ActionSchemaIds;
		TArray<int32, TInlineAllocator<1>> ActionSizes;
		TArray<TLearningArray<3, float>, TInlineAllocator<1>> ActionArrays;

		// Action Modifiers
		TArray<FName, TInlineAllocator<1>> ActionModifierNames;
		TArray<int32, TInlineAllocator<1>> ActionModifierSchemaIds;
		TArray<int32, TInlineAllocator<1>> ActionModifierSizes;
		TArray<TLearningArray<3, float>, TInlineAllocator<1>> ActionModifierArrays;

		// Memory States
		TArray<FName, TInlineAllocator<1>> MemoryStateNames;
		TArray<int32, TInlineAllocator<1>> MemoryStateSizes;
		TArray<TLearningArray<3, float>, TInlineAllocator<1>> MemoryStateArrays;

		// Rewards
		TArray<FName, TInlineAllocator<1>> RewardNames;
		TArray<int32, TInlineAllocator<1>> RewardSizes;
		TArray<TLearningArray<3, float>, TInlineAllocator<1>> RewardArrays;

		// Episode Step Nums
		TLearningArray<1, int32> EpisodeStepNums;
	};

	/**
	* Large buffer that sequentially concatenates a series of episodes in a large
	* flat array. Used to collate episodic data together from multiple instances.
	*/
	struct FReplayBuffer
	{
		/**
		* Resizes the replay buffer.
		*
		* @param EpisodeBuffer		Determines the dimensionality of the various arrays
		* @param MaxEpisodeNum		Maximum number of episodes to be stored in the buffer
		* @param MaxStepNum			Maximum number of steps to be stored in the buffer
		*/
		UE_API void Resize(
			const FEpisodeBuffer& EpisodeBuffer,
			const int32 InMaxEpisodeNum = 2048,
			const int32 InMaxStepNum = 16384);

		/**
		* Reset the replay buffer. Does not free memory - just resets episode and sample num to zero.
		*/
		UE_API void Reset();

		/**
		* Add a set of episodes to the replay buffer
		*
		* @param InEpisodeCompletionModes		Array of completion modes for each instance of shape (InstanceNum)
		* @param InEpisodeFinalObservations		Array of final observations for each instance of shape (InstanceNum, ObservationVectorDimNum)
		* @param InEpisodeFinalMemoryStates		Array of final memory states (post-evaluation) for each instance of shape (InstanceNum, MemoryStateVectorDimNum)
		* @param EpisodeBuffer					Episode buffer to add experience from
		* @param Instances						Instances to add
		* @param bAddTruncatedEpisodeWhenFull	When enabled, this will add a truncated, partial episode to the buffer when full
		*
		* @returns								True when the replay buffer is full
		*/
		UE_API bool AddEpisodes(
			const TLearningArrayView<1, const ECompletionMode> InEpisodeCompletionModes,
			const TArrayView<const TLearningArrayView<2, const float>> InEpisodeFinalObservations,
			const TArrayView<const TLearningArrayView<2, const float>> InEpisodeFinalMemoryStates,
			const FEpisodeBuffer& EpisodeBuffer,
			const FIndexSet Instances,
			const bool bAddTruncatedEpisodeWhenFull = true);

		/**
		 * Alternate way to add data from records generated via imitation learning. Does the resizing needed.
		 * 
		 * Putting this in place until we have more time later to rewrite how imitation learning stores data. At this
		 * time, it seems logical to have the data records be implemented in terms of EpisodeBuffer(s), in which case
		 * this method may no longer be needed.
		 */
		UE_API void AddRecords(
			const int32 InEpisodeNum,
			const int32 InMaxStepNum,
			const int32 ObservationSchemaId,
			const int32 ObservationNum,
			const int32 ActionSchemaId,
			const int32 ActionNum,
			const TLearningArrayView<1, const int32> RecordedEpisodeStarts,
			const TLearningArrayView<1, const int32> RecordedEpisodeLengths,
			const TLearningArrayView<2, const float> RecordedObservations,
			const TLearningArrayView<2, const float> RecordedActions);

		UE_API bool HasCompletions() const;

		UE_API bool HasFinalObservations() const;

		UE_API bool HasFinalMemoryStates() const;

		UE_API int32 GetMaxEpisodeNum() const;
		UE_API int32 GetMaxStepNum() const;
		UE_API int32 GetEpisodeNum() const;
		UE_API int32 GetStepNum() const;

		UE_API const TLearningArrayView<1, const int32> GetEpisodeStarts() const;
		UE_API const TLearningArrayView<1, const int32> GetEpisodeLengths() const;
		UE_API const TLearningArrayView<1, const ECompletionMode> GetEpisodeCompletionModes() const;
		
		UE_API int32 GetObservationsNum() const;
		UE_API const TLearningArrayView<2, const float> GetObservations(const int32 Index) const;
		UE_API const TLearningArrayView<2, const float> GetEpisodeFinalObservations(const int32 Index) const;

		UE_API int32 GetActionsNum() const;
		UE_API const TLearningArrayView<2, const float> GetActions(const int32 Index) const;

		UE_API int32 GetActionModifiersNum() const;
		UE_API const TLearningArrayView<2, const float> GetActionModifiers(const int32 Index) const;

		UE_API int32 GetMemoryStatesNum() const;
		UE_API const TLearningArrayView<2, const float> GetMemoryStates(const int32 Index) const;
		UE_API const TLearningArrayView<2, const float> GetEpisodeFinalMemoryStates(const int32 Index) const;
		
		UE_API int32 GetRewardsNum() const;
		UE_API const TLearningArrayView<2, const float> GetRewards(const int32 Index) const;

		UE_API TSharedRef<FJsonObject> AsJsonConfig(const int32 ReplayBufferId) const;

	private:

		bool bHasCompletions = false;
		bool bHasFinalObservations = false;
		bool bHasFinalMemoryStates = false;

		int32 MaxEpisodeNum = 0;
		int32 MaxStepNum = 0;

		int32 EpisodeNum = 0;
		int32 StepNum = 0;

		TLearningArray<1, int32> EpisodeStarts;
		TLearningArray<1, int32> EpisodeLengths;
		TLearningArray<1, ECompletionMode> EpisodeCompletionModes;
		
		TArray<FName, TInlineAllocator<1>> ObservationNames;
		TArray<int32, TInlineAllocator<1>> ObservationSchemaIds;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> Observations;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> EpisodeFinalObservations;

		TArray<FName, TInlineAllocator<1>> ActionNames;
		TArray<int32, TInlineAllocator<1>> ActionSchemaIds;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> Actions;

		TArray<FName, TInlineAllocator<1>> ActionModifierNames;
		TArray<int32, TInlineAllocator<1>> ActionModifierSchemaIds;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> ActionModifiers;

		TArray<FName, TInlineAllocator<1>> MemoryStateNames;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> MemoryStates;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> EpisodeFinalMemoryStates;

		TArray<FName, TInlineAllocator<1>> RewardNames;
		TArray<TLearningArray<2, float>, TInlineAllocator<1>> Rewards;
	};

	namespace Experience
	{
		/**
		* Resets, and then runs experience gathering until the provided replay buffer is full
		*
		* @param ReplayBuffer							Replay Buffer
		* @param EpisodeBuffer							Episode Buffer
		* @param ResetBuffer							Reset Buffer
		* @param ObservationVectorBuffers				Buffers to read/write observation vectors into
		* @param ActionVectorBuffers					Buffers to read/write action vectors into
		* @param PreEvaluationMemoryStateVectorBuffers	Buffers to read/write pre-evaluation memory state vectors into
		* @param MemoryStateVectorBuffers				Buffers to read/write (post-evaluation) memory state vectors into
		* @param RewardBuffers							Buffers to read/write rewards into
		* @param CompletionBuffer						Buffer to read/write completions into
		* @param EpisodeCompletionBuffer				Additional buffer to record completions from full episode buffers
		* @param AllCompletionBuffer					Additional buffer to record all completions from full episodes and normal completions
		* @param ResetFunction							Function to run for resetting the environment
		* @param ObservationFunctions					Functions to run for evaluating observations
		* @param PolicyFunctions						Functions to run generating actions from observations
		* @param ActionFunctions						Functions to run for evaluating actions
		* @param UpdateFunctions						Functions to run for updating the environment
		* @param RewardFunctions						Functions to run for evaluating rewards
		* @param CompletionFunction						Function to run for evaluating completions
		* @param Instances								Set of instances to gather experience for
		*/
		UE_API void GatherExperienceUntilReplayBufferFull(
			FReplayBuffer& ReplayBuffer,
			FEpisodeBuffer& EpisodeBuffer,
			FResetInstanceBuffer& ResetBuffer,
			const TArrayView<const TLearningArrayView<2, const float>> ObservationVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> ActionVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> PreEvaluationMemoryStateVectorBuffers,
			const TArrayView<const TLearningArrayView<2, const float>> MemoryStateVectorBuffers,
			const TArrayView<const TLearningArrayView<1, const float>> RewardBuffers,
			TLearningArrayView<1, ECompletionMode> CompletionBuffer,
			TLearningArrayView<1, ECompletionMode> EpisodeCompletionBuffer,
			TLearningArrayView<1, ECompletionMode> AllCompletionBuffer,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> ObservationFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> PolicyFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> ActionFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> UpdateFunctions,
			const TArrayView<const TFunctionRef<void(const FIndexSet Instances)>> RewardFunctions,
			const TFunctionRef<void(const FIndexSet Instances)> CompletionFunction,
			const FIndexSet Instances);
	}
};

#undef UE_API
