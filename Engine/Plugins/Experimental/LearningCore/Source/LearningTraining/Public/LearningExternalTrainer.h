// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"
#include "LearningTrainer.h"
#include "LearningSharedMemory.h"

#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"  

#define UE_API LEARNINGTRAINING_API

class FJsonObject;
class FSocket;
class ULearningNeuralNetworkData;

namespace UE::Learning
{
	struct FReplayBuffer;
	enum class ECompletionMode : uint8;

    /** Interface for communicating with an external trainer process. */
	struct IExternalTrainer
	{
		virtual ~IExternalTrainer() = default;

		/** Returns true if this trainer is valid. Otherwise, false. */
		virtual bool IsValid() = 0;

		/** Terminate the trainer immediately. */
		virtual void Terminate() = 0;

		/** Signal for the trainer to stop.	*/
		virtual ETrainerResponse SendStop() = 0;

		/**
		* Wait for the trainer to finish.
		*
		* @param Timeout		Timeout to wait in seconds
		* @returns				Trainer response
		*/
		virtual ETrainerResponse Wait() = 0;

		/** Returns true if we can receive a network or training completed. Otherwise, false. */
		virtual bool HasNetworkOrCompleted() = 0;

		/**
		* Sends the given json configs to the trainer process.
		*
		* @param DataConfigObject		The config to send with data-related settings (replay buffers, schemas)
		* @param ModelConfigObject		The config to send with model-related settings (networks, architecture)
		* @param TrainerConfigObject	The config to send with trainer-specific settings
		* @param LogSettings			The log verbosity level
		* @returns						Trainer response
		*/
		virtual ETrainerResponse SendConfigs(
			const TSharedRef<FJsonObject>& DataConfigObject,
			const TSharedRef<FJsonObject>& ModelConfigObject,
			const TSharedRef<FJsonObject>& TrainerConfigObject,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) = 0;

		/**
		 * Adds the network to this external trainer. Allocates buffers, etc.
		 * Must be called for each network prior to calling Send/Receive.
		 * 
		 * @params Network	The network to be added
		 * @returns			The network's unique id
		 */
		virtual int32 AddNetwork(const ULearningNeuralNetworkData& Network) = 0;

		/**
		* Wait for the trainer to push an updated network.
		*
		* @param NetworkId		Unique network id
		* @param OutNetwork		Network to update
		* @param Timeout		Timeout to wait in seconds
		* @param NetworkLock	Lock to use when updating network
		* @param LogSettings	The log verbosity level
		* @returns				Trainer response
		*/
		virtual ETrainerResponse ReceiveNetwork(
			const int32 NetworkId,
			ULearningNeuralNetworkData& OutNetwork,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) = 0;

		/**
		* Wait for the trainer to push an array of updated networks.
		*
		* @param NetworkIds		Unique network ids
		* @param OutNetworks		Networks to update
		* @param Timeout		Timeout to wait in seconds
		* @param NetworkLocks	Locks to use when updating networks
		* @param LogSettings	The log verbosity level
		* @returns				Trainer response
		*/
		virtual TArray<ETrainerResponse> ReceiveNetworks(
			const TArray<int32>& NetworkIds,
			TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
			TArray<FRWLock*> NetworkLocks = TArray<FRWLock*>(),
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) = 0;

		/**
		* Wait for the trainer to be ready and push the current policy network.
		*
		* @param NetworkId		Unique network id
		* @param Network		Network to push
		* @param Timeout		Timeout to wait in seconds
		* @param NetworkLock	Lock to use when pushing network
		* @param LogSettings	The log verbosity level
		* @returns				Trainer response
		*/
		virtual ETrainerResponse SendNetwork(
			const int32 NetworkId,
			const ULearningNeuralNetworkData& Network,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) = 0;

		/**
		 * Adds a named replay buffer to this external trainer.
		 * Must be called for each buffer prior to calling SendReplayBuffer.
		 *
		 * @params ReplayBuffer		The buffer to be added
		 * @returns					The replay buffer's unique id		
		 */
		virtual int32 AddReplayBuffer(const FReplayBuffer& ReplayBuffer) = 0;

		/**
		* Wait for the trainer to be ready and send new experience.
		*
		* @params ReplayBufferId	Unique replay buffer id
		* @params Name				The unique name of the buffer, used as a key
		* @param ReplayBuffer		Replay buffer to send
		* @param Timeout			Timeout to wait in seconds
		* @param LogSettings		The log verbosity level
		* @returns					Trainer response
		*/
		virtual ETrainerResponse SendReplayBuffer(
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) = 0;
	};

	/**
	* This object allows you to launch the FSharedMemoryTrainer server as a subprocess,
	* which is convenient when you want to train locally.
	*/
	struct FSharedMemoryTrainerServerProcess
	{
		/**
		* Creates a training server as a subprocess using shared memory for communication. This will no-op if this UE
		* process has a non-zero "LearningProcessIdx".
		*
		* @param TaskName					The name of this training task (used to disambiguate filenames, etc.)
		* @param CustomTrainerPath			Path to check for custom trainer files
		* @param TrainerFileName			The name of the training file to use
		* @param PythonExecutablePath		Path to the python executable used for training. In general should be the
		*									python shipped with Unreal Editor.
		* @param PythonContentPath			Path to the Python Content folder provided by the Learning plugin
		* @param InIntermediatePath			Path to the intermediate folder to write temporary files, logs, and
		*									snapshots to
		* @param TrainingProcessFlags		Training server subprocess flags
		* @param LogSettings				Logging settings to use
		*/
		UE_API FSharedMemoryTrainerServerProcess(
			const FString& TaskName,
			const FString& CustomTrainerPath,
			const FString& TrainerFileName,
			const FString& PythonExecutablePath,
			const FString& PythonContentPath,
			const FString& InIntermediatePath,
			const FString& NneCpuContentPath,
			const float InTimeout = Trainer::DefaultTimeout,
			const ESubprocessFlags TrainingProcessFlags = ESubprocessFlags::None,
			const ELogSetting LogSettings = ELogSetting::Normal);

		/** Check if the server process is still running. */
		UE_API bool IsRunning() const;

		/**
		* Wait for the server process to end
		*
		* @param Timeout		Timeout to wait in seconds
		* @returns				true if successful, otherwise false if it times out
		*/
		UE_API bool Wait();

		/** Terminate the server process. */
		UE_API void Terminate();

		/** Get the Controls shared memory array view. */
		UE_API TSharedMemoryArrayView<1, volatile int32> GetControlsSharedMemoryArrayView() const;

		/** Get the intermediate path. */
		UE_API const FString& GetIntermediatePath() const;

		/** Get the config path. */
		UE_API const FString& GetConfigPath() const;

		/** Get the training subprocess. */
		UE_API FSubprocess* GetTrainingSubprocess();

	private:

		/** Free and deallocate all shared memory. */
		UE_API void Deallocate();

		FString IntermediatePath;
		FString ConfigPath;

		TSharedMemoryArrayView<1, volatile int32> Controls; // Mark as volatile to avoid compiler optimizing away reads without writes etc.

		FSubprocess TrainingProcess;
		float Timeout = Trainer::DefaultTimeout;
	};

	/**
	* Trainer that connects to an external training server to perform training
	*
	* This trainer can be used to allow the python training process the run
	* on a different machine to the experience gathering process.
	*/
	struct FSharedMemoryTrainer : public IExternalTrainer
	{
		struct FSharedMemoryExperienceContainer
		{
			TSharedMemoryArrayView<1, int32> EpisodeStarts;
			TSharedMemoryArrayView<1, int32> EpisodeLengths;
			TSharedMemoryArrayView<1, ECompletionMode> EpisodeCompletionModes;

			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> EpisodeFinalObservations;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> EpisodeFinalMemoryStates;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> Observations;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> Actions;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> ActionModifiers;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> MemoryStates;
			TArray<TSharedMemoryArrayView<2, float>, TInlineAllocator<1>> Rewards;

			/** Free and deallocate all shared memory. */
			void Deallocate();
		};

		/**
		* Creates a new SharedMemory trainer
		*
		* @param InTaskName						Unique name for this training task - used to avoid config filename conflicts
		* @param ExternalTrainerProcess			Shared memory used for communicating status to the trainer server process
		* @param bUseExternalTrainingProcess	If true, attach to existing external training process.
		* @param ControlsGuid					If bUseExternalTrainingProcess is true, attach to this controls memory.
		* @param InConfigPath					The path to write the config to.
		* @param InIntermediatePath				The Intermediate Path - for writing temporary files.
		* @param InTimeout						Timeout to wait in seconds for connection and initial data transfer
		*/
		UE_API FSharedMemoryTrainer(
			const FString& InTaskName,
			const TSharedPtr<FSharedMemoryTrainerServerProcess>& ExternalTrainerProcess,
			const bool bUseExternalTrainingProcess = false,
			const FGuid ControlsGuid = FGuid(),
			const FString& InConfigPath = FString(),
			const FString& InIntermediatePath = FString(),
			const float InTimeout = Trainer::DefaultTimeout);

		UE_API ~FSharedMemoryTrainer();

		UE_API virtual bool IsValid() override final;

		UE_API virtual void Terminate() override final;

		UE_API virtual ETrainerResponse SendStop() override final;

		UE_API virtual ETrainerResponse Wait() override final;

		UE_API virtual bool HasNetworkOrCompleted() override final;

		UE_API virtual ETrainerResponse SendConfigs(
			const TSharedRef<FJsonObject>& DataConfigObject,
			const TSharedRef<FJsonObject>& ModelConfigObject,
			const TSharedRef<FJsonObject>& TrainerConfigObject,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddNetwork(const ULearningNeuralNetworkData& Network) override final;

		UE_API virtual ETrainerResponse ReceiveNetwork(
			const int32 NetworkId,
			ULearningNeuralNetworkData& OutNetwork,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual TArray<ETrainerResponse> ReceiveNetworks(
			const TArray<int32>& NetworkIds,
			TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
			TArray<FRWLock*> NetworkLocks = TArray<FRWLock*>(),
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual ETrainerResponse SendNetwork(
			const int32 NetworkId,
			const ULearningNeuralNetworkData& Network,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddReplayBuffer(const FReplayBuffer& ReplayBuffer) override final;

		UE_API virtual ETrainerResponse SendReplayBuffer(
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

	private:

		/** Free and deallocate all shared memory. */
		UE_API void Deallocate();

		FString TaskName;
		FString IntermediatePath;
		FString ConfigPath;

		TSharedPtr<FSharedMemoryTrainerServerProcess> TrainingProcess = nullptr;

		float Timeout = Trainer::DefaultTimeout;

		bool bUseExternalTrainer = false;

		TSharedMemoryArrayView<1, volatile int32> Controls;
		TArray<TSharedMemoryArrayView<1, uint8>> NeuralNetworkSharedMemoryArrayViews;
		TArray<FSharedMemoryExperienceContainer> SharedMemoryExperienceContainers;
	};

	/**
	* This object allows you to launch the FSocketTrainer server as a subprocess,
	* which is convenient when you want to train using it locally.
	*/
	struct FSocketTrainerServerProcess
	{
		/**
		* Creates a training server as a subprocess
		*
		* @param TaskName					The name of this training task
		* @param CustomTrainerPath			Path to check for custom trainer files
		* @param TrainerFileName			The name of the training file to use
		* @param PythonExecutablePath		Path to the python executable used for training. In general should be the python shipped with Unreal Editor.
		* @param PythonContentPath			Path to the Python Content folder provided by the Learning plugin
		* @param IntermediatePath			Path to the intermediate folder to write temporary files, logs, and snapshots to
		* @param IpAddress					Ip address to bind the listening socket to. For a local server you will want to use 127.0.0.1
		* @param Port						Port to use for the listening socket.
		* @param TrainingProcessFlags		Training server subprocess flags
		* @param LogSettings				Logging settings to use
		*/
		UE_API FSocketTrainerServerProcess(
			const FString& TaskName,
			const FString& CustomTrainerPath,
			const FString& TrainerFileName,
			const FString& PythonExecutablePath,
			const FString& PythonContentPath,
			const FString& IntermediatePath,
			const FString& NneCpuContentPath,
			const TCHAR* IpAddress = Trainer::DefaultIp,
			const uint32 Port = Trainer::DefaultPort,
			const float InTimeout = Trainer::DefaultTimeout,
			const ESubprocessFlags TrainingProcessFlags = ESubprocessFlags::None,
			const ELogSetting LogSettings = ELogSetting::Normal);

		/**
		* Check if the server process is still running
		*/
		UE_API bool IsRunning() const;

		/**
		* Wait for the server process to end
		*
		* @param Timeout		Timeout to wait in seconds
		* @returns				true if successful, otherwise false if it times out
		*/
		UE_API bool Wait();

		/**
		* Terminate the server process
		*/
		UE_API void Terminate();

		/** Get the training subprocess. */
		UE_API FSubprocess* GetTrainingSubprocess();

	private:

		FSubprocess TrainingProcess;
		float Timeout = Trainer::DefaultTimeout;
	};

	/**
	* Trainer that connects to an external training server to perform training
	*
	* This trainer can be used to allow the python training process the run
	* on a different machine to the experience gathering process.
	*/
	struct FSocketTrainer : public IExternalTrainer
	{
		/**
		* Creates a new Socket trainer
		*
		* @param OutResponse				Response to the initial connection
		* @param ExternalTrainerProcess		The external trainer process
		* @param IpAddress					Server Ip address
		* @param Port						Server Port
		* @param Timeout					Timeout to wait in seconds for connection and initial data transfer
		*/
		UE_API FSocketTrainer(
			ETrainerResponse& OutResponse,
			const TSharedPtr<FSocketTrainerServerProcess>& ExternalTrainerProcess,
			const bool bUseExternalTrainerProcess,
			const TCHAR* IpAddressOrHostname = Trainer::DefaultIp,
			const uint32 Port = Trainer::DefaultPort,
			const float InTimeout = Trainer::DefaultTimeout,
			const bool IsHostname = false);

		UE_API ~FSocketTrainer();

		UE_API virtual bool IsValid() override final;

		UE_API virtual void Terminate() override final;

		UE_API virtual ETrainerResponse SendStop() override final;

		UE_API virtual ETrainerResponse Wait() override final;

		UE_API virtual bool HasNetworkOrCompleted() override final;

		UE_API virtual ETrainerResponse SendConfigs(
			const TSharedRef<FJsonObject>& DataConfigObject,
			const TSharedRef<FJsonObject>& ModelConfigObject,
			const TSharedRef<FJsonObject>& TrainerConfigObject,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddNetwork(const ULearningNeuralNetworkData& Network) override final;

		UE_API virtual ETrainerResponse ReceiveNetwork(
			const int32 NetworkId,
			ULearningNeuralNetworkData& OutNetwork,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual TArray<ETrainerResponse> ReceiveNetworks(
			const TArray<int32>& NetworkIds,
			TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
			TArray<FRWLock*> NetworkLocks = TArray<FRWLock*>(),
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual TArray<ETrainerResponse> ReceiveQueuedNetworks(
			const TArray<int32>& NetworkIds,
			TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
			TArray<FRWLock*> NetworkLocks = TArray<FRWLock*>(),
			const ELogSetting LogSettings = Trainer::DefaultLogSettings);

		UE_API virtual ETrainerResponse SendNetwork(
			const int32 NetworkId,
			const ULearningNeuralNetworkData& Network,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddReplayBuffer(const FReplayBuffer& ReplayBuffer) override final;

		UE_API virtual ETrainerResponse SendReplayBuffer(
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

	private:

		TArray<TLearningArray<1, uint8>> NetworkBuffers;
		int32 LastReplayBufferId = -1;

		float Timeout = Trainer::DefaultTimeout;

		bool bUseExternalTrainer = false;

		TSharedPtr<FSocketTrainerServerProcess> TrainingProcess = nullptr;

		FSocket* Socket = nullptr;

		int32 NetworksVersion = -1;
	};
	
	struct FFileTrainer : public IExternalTrainer
	{
		/**
		* Creates a new trainer that dumps training materials to file.
		*
		* @param InIntermediatePath				The Intermediate Path - for writing temporary files.
		* @param InExportFlags					Which sections to export (defaults to All).
		*/
		UE_API FFileTrainer(const FString& InIntermediatePath = FString(), ETrainerExportFlags InExportFlags = ETrainerExportFlags::All);

		UE_API ~FFileTrainer();

		virtual bool IsValid() override final { return true; }

		virtual void Terminate() override final {}

		virtual ETrainerResponse SendStop() override final { return ETrainerResponse::Success; }

		virtual ETrainerResponse Wait() override final { return ETrainerResponse::Success; }

		virtual bool HasNetworkOrCompleted() override final { return true; }

		UE_API virtual ETrainerResponse SendConfigs(
			const TSharedRef<FJsonObject>& DataConfigObject,
			const TSharedRef<FJsonObject>& ModelConfigObject,
			const TSharedRef<FJsonObject>& TrainerConfigObject,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddNetwork(const ULearningNeuralNetworkData& Network) override final;

		virtual ETrainerResponse ReceiveNetwork(
			const int32 NetworkId,
			ULearningNeuralNetworkData& OutNetwork,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final { return ETrainerResponse::Completed; }

		virtual TArray<ETrainerResponse> ReceiveNetworks(
			const TArray<int32>& NetworkIds,
			TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks,
			TArray<FRWLock*> NetworkLocks = TArray<FRWLock*>(),
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final { return TArray<ETrainerResponse>(); }

		UE_API virtual ETrainerResponse SendNetwork(
			const int32 NetworkId,
			const ULearningNeuralNetworkData& Network,
			FRWLock* NetworkLock = nullptr,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

		UE_API virtual int32 AddReplayBuffer(const FReplayBuffer& ReplayBuffer) override final;

		UE_API virtual ETrainerResponse SendReplayBuffer(
			const int32 ReplayBufferId,
			const FReplayBuffer& ReplayBuffer,
			const ELogSetting LogSettings = Trainer::DefaultLogSettings) override final;

	private:
		FString IntermediatePath;
		FString FileSaveDirectory;
		ETrainerExportFlags ExportFlags = ETrainerExportFlags::All;
		int32 ReplayBufferCount = 0;
		int32 NetworksCount = 0;

		TArray<TSharedMemoryArrayView<1, uint8>> NeuralNetworkSharedMemoryArrayViews;
	};
}

#undef UE_API
