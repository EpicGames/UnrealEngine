// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"
#include "LearningTrainer.h"
#include "LearningSharedMemory.h"

#include "Commandlets/Commandlet.h"
#include "Templates/SharedPointer.h"

#include "LearningPPOTrainer.generated.h"

#define UE_API LEARNINGTRAINING_API

class ULearningNeuralNetworkData;

UCLASS(MinimalAPI)
class ULearningSocketPPOTrainerServerCommandlet : public UCommandlet
{
	GENERATED_BODY()

	UE_API ULearningSocketPPOTrainerServerCommandlet(const FObjectInitializer& ObjectInitializer);

	/** Runs the commandlet */
	UE_API virtual int32 Main(const FString& Params) override;
};

namespace UE::Learning
{
	struct IExternalTrainer;
	struct FReplayBuffer;
	struct FResetInstanceBuffer;
	struct FEpisodeBuffer;
	enum class ECompletionMode : uint8;

	/**
	* Settings used for training with PPO
	*/
	struct FPPOTrainerTrainingSettings
	{
		// Number of iterations to train the network for. Controls the overall training time.
		// Training for about 100000 iterations should give you well trained network, but
		// closer to 1000000 iterations or more is required for an exhaustively trained network.
		uint32 IterationNum = 1000000;

		// Learning rate of the policy network. Typical values are between 0.001f and 0.0001f
		float LearningRatePolicy = 0.0001f;

		// Learning rate of the critic network. To avoid instability generally the critic 
		// should have a larger learning rate than the policy.
		float LearningRateCritic = 0.001f;

		// Amount by which to multiply the learning rate every 1000 iterations.
		float LearningRateDecay = 1.0f;

		// Amount of weight decay to apply to the network. Larger values encourage network 
		// weights to be smaller but too large a value can cause the network weights to collapse to all zeros.
		float WeightDecay = 0.0001f;

		// Batch size to use for training the policy. Large batch sizes are much more computationally efficient when training on the GPU.
		uint32 PolicyBatchSize = 1024;

		// Batch size to use for training the critic. Large batch sizes are much more computationally efficient when training on the GPU.
		uint32 CriticBatchSize = 4096;

		// The number of consecutive steps of observations and actions over which to train the policy. Increasing this value 
		// will encourage the policy to use its memory effectively. Too large and training can become slow and unstable.
		uint32 PolicyWindow = 16;

		// Number of training iterations to perform per buffer of experience gathered. This should be large enough for
		// the critic and policy to be effectively updated, but too large and it will simply slow down training.
		uint32 IterationsPerGather = 32;

		// Number of iterations of training to perform to warm-up the Critic. This helps speed up and stabilize training
		// at the beginning when the Critic may be producing predictions at the wrong order of magnitude.
		uint32 CriticWarmupIterations = 8;

		// Clipping ratio to apply to policy updates. Keeps the training "on-policy". 
		// Larger values may speed up training at the cost of stability. Conversely, too small 
		// values will keep the policy from being able to learn an optimal policy.
		float EpsilonClip = 0.2f;

		// Weight used to regularize predicted returns. Encourages the critic not to over or under estimate returns.
		float ReturnRegularizationWeight = 0.0001f;

		// Weight for the loss used to train the policy via the PPO surrogate objective.
		float ActionSurrogateWeight = 1.0f;

		// Weight used to regularize actions.Larger values will encourage exploration and smaller actions, but too large will cause
		// noisy actions centered around zero.
		float ActionRegularizationWeight = 0.001f;

		// Weighting used for the entropy bonus. Larger values encourage larger action 
		// noise and therefore greater exploration but can make actions very noisy.
		float ActionEntropyWeight = 0.0f;

		// This is used in the Generalized Advantage Estimation, where larger values will tend to assign more credit to recent actions. Typical 
		// values should be between 0.9 and 1.0.
		float GaeLambda = 0.95f;

		// When true, advantages are normalized. This tends to makes training more robust to 
		// adjustments of the scale of rewards. 
		bool bAdvantageNormalization = true;

		// The minimum advantage to allow. Setting this below zero will encourage the policy to
		// move away from bad actions, but can introduce instability.
		float AdvantageMin = 0.0f;

		// The maximum advantage to allow. Making this smaller may increase training stability
		// at the cost of some training speed.
		float AdvantageMax = 10.0f;

		// If true, uses gradient norm max clipping. Set this as True if training is unstable or leave as False if unused.
		bool bUseGradNormMaxClipping = false;

		// The maximum gradient norm to clip updates to.
		float GradNormMax = 0.5f;

		// Number of steps to trim from the start of each episode during training. This can
		// be useful if some reset process is taking several steps or you know your starting
		// states are not entirely valid for example.
		int32 TrimEpisodeStartStepNum = 0;

		// Number of steps to trim from the end of each episode during training. This can be
		// useful if you know the last few steps of an episode are not valid or contain incorrect
		// information.
		int32 TrimEpisodeEndStepNum = 0;

		// Random Seed to use for training
		uint32 Seed = 1234;

		// The discount factor causes future rewards to be scaled down so that the policy will 
		// favor near-term rewards over potentially uncertain long-term rewards. Larger values 
		// encourage the system to "look-ahead" but make training more difficult.
		float DiscountFactor = 0.99f;
		
		// Which device to use for training
		ETrainerDevice Device = ETrainerDevice::GPU;

		// If to use TensorBoard for logging and tracking the training progress.
		// 
		// TensorBoard will only work if it is installed in Unreal Engine's python environment. This can be done by
		// enabling the "Tensorboard" plugin in your project.
		bool bUseTensorboard = false;

		// If to save snapshots of the trained networks every 1000 iterations
		bool bSaveSnapshots = false;
	};

	namespace PPOTrainer
	{
		/**
		* Train a policy while gathering experience
		*
		* @param ExternalTrainer						External Trainer
		* @param ReplayBuffer							Replay Buffer
		* @param EpisodeBuffer							Episode Buffer
		* @param ResetBuffer							Reset Buffer
		* @param PolicyNetwork							Policy Network to use
		* @param CriticNetwork							Critic Network to use
		* @param EncoderNetwork							Encoder Network to use
		* @param DecoderNetwork							Decoder Network to use
		* @param ObservationVectorBuffer				Buffer to read/write observation vectors into
		* @param ActionVectorBuffer						Buffer to read/write action vectors into
		* @param PreEvaluationMemoryStateVectorBuffer	Buffer to read/write pre-evaluation memory state vectors into
		* @param MemoryStateVectorBuffer				Buffer to read/write (post-evaluation) memory state vectors into
		* @param RewardBuffer							Buffer to read/write rewards into
		* @param CompletionBuffer						Buffer to read/write completions into
		* @param EpisodeCompletionBuffer				Additional buffer to record completions from full episode buffers
		* @param AllCompletionBuffer					Additional buffer to record all completions from full episodes and normal completions
		* @param ResetFunction							Function to run for resetting the environment
		* @param ObservationFunction					Function to run for evaluating observations
		* @param PolicyFunction							Function to run for evaluating the policy
		* @param ActionFunction							Function to run for evaluating actions
		* @param UpdateFunction							Function to run for updating the environment
		* @param RewardFunction							Function to run for evaluating rewards
		* @param CompletionFunction						Function to run for evaluating completions
		* @param Instances								Set of instances to run training for
		* @param bRequestTrainingStopSignal				Optional signal that can be set to indicate training should be stopped
		* @param PolicyNetworkLock						Optional Lock to use when updating the policy network
		* @param CriticNetworkLock						Optional Lock to use when updating the critic network
		* @param EncoderNetworkLock						Optional Lock to use when updating the encoder network
		* @param DecoderNetworkLock						Optional Lock to use when updating the decoder network
		* @param bPolicyNetworkUpdatedSignal			Optional signal that will be set when the policy network is updated
		* @param bCriticNetworkUpdatedSignal			Optional signal that will be set when the critic network is updated
		* @param bEncoderNetworkUpdatedSignal			Optional signal that will be set when the encoder network is updated
		* @param bDecoderNetworkUpdatedSignal			Optional signal that will be set when the decoder network is updated
		* @param LogSettings							Logging settings
		* @returns										Trainer response in case of errors during communication otherwise Success
		*/
		UE_API ETrainerResponse Train(
			IExternalTrainer* ExternalTrainer,
			FReplayBuffer& ReplayBuffer,
			FEpisodeBuffer& EpisodeBuffer,
			FResetInstanceBuffer& ResetBuffer,
			ULearningNeuralNetworkData& PolicyNetwork,
			ULearningNeuralNetworkData& CriticNetwork,
			ULearningNeuralNetworkData& EncoderNetwork,
			ULearningNeuralNetworkData& DecoderNetwork,
			TLearningArrayView<2, float> ObservationVectorBuffer,
			TLearningArrayView<2, float> ActionVectorBuffer,
			TLearningArrayView<2, float> PreEvaluationMemoryStateVectorBuffer,
			TLearningArrayView<2, float> MemoryStateVectorBuffer,
			TLearningArrayView<1, float> RewardBuffer,
			TLearningArrayView<1, ECompletionMode> CompletionBuffer,
			TLearningArrayView<1, ECompletionMode> EpisodeCompletionBuffer,
			TLearningArrayView<1, ECompletionMode> AllCompletionBuffer,
			const TFunctionRef<void(const FIndexSet Instances)> ResetFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ObservationFunction,
			const TFunctionRef<void(const FIndexSet Instances)> PolicyFunction,
			const TFunctionRef<void(const FIndexSet Instances)> ActionFunction,
			const TFunctionRef<void(const FIndexSet Instances)> UpdateFunction,
			const TFunctionRef<void(const FIndexSet Instances)> RewardFunction,
			const TFunctionRef<void(const FIndexSet Instances)> CompletionFunction,
			const FIndexSet Instances,
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement,
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement,
			const FPPOTrainerTrainingSettings& TrainerSettings = FPPOTrainerTrainingSettings(),
			TAtomic<bool>* bRequestTrainingStopSignal = nullptr,
			FRWLock* PolicyNetworkLock = nullptr,
			FRWLock* CriticNetworkLock = nullptr,
			FRWLock* EncoderNetworkLock = nullptr,
			FRWLock* DecoderNetworkLock = nullptr,
			TAtomic<bool>* bPolicyNetworkUpdatedSignal = nullptr,
			TAtomic<bool>* bCriticNetworkUpdatedSignal = nullptr,
			TAtomic<bool>* bEncoderNetworkUpdatedSignal = nullptr,
			TAtomic<bool>* bDecoderNetworkUpdatedSignal = nullptr,
			const ELogSetting LogSettings = ELogSetting::Normal);
	}

}

#undef UE_API
