// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "GenericPlatform/GenericPlatformMemory.h"
#include "Misc/Guid.h"

/**
* Shared memory is used by Learning to efficiently share both experience 
* and policy data between a Python training sub-process and potentially 
* multiple unreal processes (which may be gathering that experience).
*/

namespace UE::Learning
{
	/**
	* Basic struct containing a view to a shared memory region, its guid, 
	* and a pointer to that region which is used for deallocation.
	*/
	template<uint8 DimNum, typename ElementType>
	struct TSharedMemoryArrayView
	{
		FGuid Guid;
		TLearningArrayView<DimNum, ElementType> View;
		FPlatformMemory::FSharedMemoryRegion* Region = nullptr;

		/** Gets a platform specific Guid string. This is required as guid strings on Mac must shorter to be used with shared memory */
		FString GetPlatformGuidString() const
		{
#if PLATFORM_MAC
			return Guid.ToString(EGuidFormats::Short);
#else
			return Guid.ToString(EGuidFormats::DigitsWithHyphensInBraces);
#endif
		}
	};

	namespace SharedMemory
	{
		/**
		* Map a view of a region of shared memory.
		* 
		* @param Guid			Guid of the shared memory region
		* @param Shape			Shape of the shared memory region
		* @param bCreate		If this shared memory region should be created or if it already exists
		* @returns				View of the shared memory region, guid, and pointer required for unmapping
		*/
		template<uint8 DimNum, typename ElementType>
		TSharedMemoryArrayView<DimNum, ElementType> Map(const FGuid Guid, const TLearningArrayShape<DimNum>& Shape, const bool bCreate = false)
		{
			const SIZE_T TotalSize = sizeof(ElementType) * (SIZE_T)Shape.Total();

			if (TotalSize == 0)
			{
				return { FGuid(), TLearningArrayView<DimNum, ElementType>(nullptr, Shape), nullptr };
			}

			FPlatformMemory::FSharedMemoryRegion* RegionPointer = FPlatformMemory::MapNamedSharedMemoryRegion(
				Guid.ToString(PLATFORM_MAC ? EGuidFormats::Short : EGuidFormats::DigitsWithHyphensInBraces),
				bCreate,
				// I was getting odd issues allocating read-only memory so for now we always allocate as read-write
				FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write,
				TotalSize);

			if (ensureMsgf(RegionPointer != nullptr, TEXT("Unable to map shared memory.")))
			{
				return { Guid, TLearningArrayView<DimNum, ElementType>((ElementType*)RegionPointer->GetAddress(), Shape), RegionPointer };
			}

			return TSharedMemoryArrayView<DimNum, ElementType>();
		}

		/*
		* Unmaps the view of the region of shared memory
		*/
		template<uint8 DimNum, typename ElementType>
		void Unmap(TSharedMemoryArrayView<DimNum, ElementType>& Memory)
		{
			if (Memory.Region != nullptr)
			{
				ensureMsgf(FPlatformMemory::UnmapNamedSharedMemoryRegion(Memory.Region), TEXT("Failed to unmap shared memory."));
			}

			Memory = TSharedMemoryArrayView<DimNum, ElementType>();
		}

		/**
		* Allocate a region of shared memory.
		*
		* @param Shape			Shape of the shared memory region
		* @returns				View of the shared memory region, guid, and pointer required for unmapping
		*/
		template<uint8 DimNum, typename ElementType>
		TSharedMemoryArrayView<DimNum, ElementType> Allocate(const TLearningArrayShape<DimNum>& Shape)
		{
			return Map<DimNum, ElementType>(FGuid::NewGuid(), Shape, true);
		}

		/*
		* Deallocate the region of shared memory
		*/
		template<uint8 DimNum, typename ElementType>
		void Deallocate(TSharedMemoryArrayView<DimNum, ElementType>& Memory)
		{
			Unmap<DimNum, ElementType>(Memory);
		}
	}
}