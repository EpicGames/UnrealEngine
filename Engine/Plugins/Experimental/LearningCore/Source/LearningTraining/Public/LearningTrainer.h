// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningLog.h"

#include "Templates/SharedPointer.h"
#include "HAL/PlatformProcess.h"

#define UE_API LEARNINGTRAINING_API

class FJsonObject;

namespace UE::Learning
{
	namespace Observation
	{
		struct FSchema;
		struct FSchemaElement;
	}

	namespace Action
	{
		struct FSchema;
		struct FSchemaElement;
	}

	/**
	* Training Device
	*/
	enum class ETrainerDevice : uint8
	{
		CPU = 0,
		GPU = 1,
	};

	/**
	* Type of response from a Trainer
	*/
	enum class ETrainerResponse : uint8
	{
		// The communication was successful
		Success = 0,

		// The communication send or received was unexpected
		Unexpected = 1,

		// Training is complete
		Completed = 2,

		// Training is stopped
		Stopped = 3,

		// The communication timed-out
		Timeout = 4,

		// The communication timed-out for a network signal
		NetworkSignalTimeout = 5,
	};

	/**
	* Subprocess flags
	*/
	enum class ESubprocessFlags : uint8
	{
		None = 0,

		// If to show the sub-process console window
		ShowWindow = 1 << 0,

		// If to avoid redirecting the sub-process output to the output log
		NoRedirectOutput = 1 << 1,
	};
	ENUM_CLASS_FLAGS(ESubprocessFlags)

	/** Flags controlling which sections a trainer exports. */
	enum class ETrainerExportFlags : uint8
	{
		None           = 0,
		ReplayBuffers  = 1 << 0,	// ReplayBuffers/ dir: replay binaries + data-config.json
		Networks       = 1 << 1,	// Networks/ dir: network binaries + model-config.json
		TrainerConfig  = 1 << 2,	// trainer-config.json
		All            = ReplayBuffers | Networks | TrainerConfig,
	};
	ENUM_CLASS_FLAGS(ETrainerExportFlags)

	/**
	* Simple managed subprocess similar to FMonitoredProcess
	*/
	struct FSubprocess
	{
		/** Will Terminate the subprocess if it is running. */
		UE_API ~FSubprocess();

		/**
		 * Launches a new subprocess.
		 *
		 * @param Path The path of the executable to launch.
		 * @param Params The command line parameters.
		 * @param Flags Subprocess flags.
		 * @returns true if the launch was successful, otherwise false
		 */
		UE_API bool Launch(const FString& Path, const FString& Params, const ESubprocessFlags Flags);

		/** 
		* Return true if the Subprocess is launched and running, otherwise false.
		*/
		UE_API bool IsRunning() const;

		/**
		* Terminates the subprocess
		*/
		UE_API void Terminate();

		/**
		* Outputs anything the subprocess has written to stdout to the log line-by-line and returns true if the subprocess is still running.
		*/
		UE_API bool Update();

	private:

		// Buffer for the subprocess' stdout
		FString OutputBuffer;

		// If a subprocess has been launched
		bool bIsLaunched = false;

		// Subprocess handle
		FProcHandle ProcessHandle;

		// Read pipe for subprocess stdout
		void* ReadPipe = nullptr;

		// Write pipe for subprocess stdin
		void* WritePipe = nullptr;
	};

	namespace Trainer
	{
		/**
		* Default Timeout to use during communication.
		*/
		static constexpr float DefaultTimeout = 10.0f;

		/**
		* Default Log Settings to use during communication.
		*/
		static constexpr ELogSetting DefaultLogSettings = ELogSetting::Normal;

		/**
		* Default IP to use for networked training
		*/
		static constexpr const TCHAR* DefaultIp = TEXT("127.0.0.1");

		/**
		* Default Port to use for networked training
		*/
		static constexpr uint32 DefaultPort = 48491;

		/**
		* Converts a ETrainerDevice into a string.
		*/
		UE_API const TCHAR* GetDeviceString(const ETrainerDevice Device);

		/**
		* Converts a ETrainerResponse into a string for use in logging and error messages.
		*/
		UE_API const TCHAR* GetResponseString(const ETrainerResponse Response);

		/**
		* Compute the discount factor that corresponds to a particular HalfLife and DeltaTime.
		*
		* @param HalfLife		Time by which the reward should be discounted by half
		* @param DeltaTime		DeltaTime taken upon each step of the environment
		* @returns				Corresponding discount factor
		*/
		UE_API float DiscountFactorFromHalfLife(const float HalfLife, const float DeltaTime);

		/**
		* Compute the discount factor that corresponds to a particular HalfLife provided in terms of number of steps
		*
		* @param HalfLifeSteps	Number of steps taken at which the reward should be discounted by half
		* @returns				Corresponding discount factor
		*/
		UE_API float DiscountFactorFromHalfLifeSteps(const int32 HalfLifeSteps);

		/**
		* Gets the python executable path from the engine directory.
		*/
		UE_API FString GetPythonExecutablePath(const FString& EngineDir);

		/**
		* Gets the PythonFoundationPackages site-packages path from the engine directory.
		*/
		UE_API FString GetSitePackagesPath(const FString& EngineDir);

		/**
		* Gets the LearningAgents Content path from the engine directory.
		*/
		UE_API FString GetPythonContentPath(const FString& EngineDir);

		/**
		* Gets the project's Python Content path.
		*/
		UE_API FString GetProjectPythonContentPath();

		/**
		* Gets the NNE Runtime Basic CPU Python content path from the plugin.
		*/
		UE_API FString GetNneCpuContentPath();

		/**
		* Gets the LearningAgents Intermediate path from the intermediate directory.
		*/
		UE_API FString GetIntermediatePath(const FString& IntermediateDir);

		/**
		* Converts an observation schema element into a JSON representation.
		*/
		UE_API TSharedPtr<FJsonObject> ConvertObservationSchemaToJSON(
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement);

		/**
		* Converts an action schema element into a JSON representation.
		*/
		UE_API TSharedPtr<FJsonObject> ConvertActionSchemaToJSON(
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement);

		/* Returns true if the schema and element are a subset of the provided JSON. */
		UE_API bool IsObservationSchemaSubsetCompatible(
			const FString& SourceJsonString,
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement);

		/* Returns true if the schema and element are a subset of the provided JSON. */
		UE_API bool IsObservationSchemaSubsetCompatible(
			const TSharedRef<FJsonObject> Object,
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement);

		/* Computes the indices to copy from a subset compatible JSON. */
		UE_API TArray<int32> ComputeObservationSchemaSubsetIndices(
			const FString& SourceJsonString,
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement);

		/* Computes the indices to copy from a subset compatible JSON. */
		UE_API void ComputeObservationSchemaSubsetIndices(
			int32& NextIndex,
			TArray<int32>& Indices,
			const TSharedRef<FJsonObject> Object,
			const Observation::FSchema& ObservationSchema,
			const Observation::FSchemaElement& ObservationSchemaElement);

		/* Returns true if the schema and element are a subset of the provided JSON. */
		UE_API bool IsActionSchemaSubsetCompatible(
			const FString& SourceJsonString,
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement);

		/* Returns true if the schema and element are a subset of the provided JSON. */
		UE_API bool IsActionSchemaSubsetCompatible(
			const TSharedRef<FJsonObject> Object,
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement);

		/* Computes the indices to copy from a subset compatible JSON. */
		UE_API TArray<int32> ComputeActionSchemaSubsetIndices(
			const FString& SourceJsonString,
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement);

		/* Computes the indices to copy from a subset compatible JSON. */
		UE_API void ComputeActionSchemaSubsetIndices(
			int32& NextIndex,
			TArray<int32>& Indices,
			const TSharedRef<FJsonObject> Object,
			const Action::FSchema& ActionSchema,
			const Action::FSchemaElement& ActionSchemaElement);
	}

}

#undef UE_API
